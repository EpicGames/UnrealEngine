{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0.0",
	"FriendlyName": "Capture Manager Core",
	"Description": "The Capture Manager Core plugin contains utility modules that are shared between Capture Manager App plugin and Capture Manager Editor plugin.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"EnabledByDefault": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Hidden": true,
	"Modules": [
		{
			"Name": "CaptureUtils",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "CaptureProtocolStack",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CaptureManagerCPSClient",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CaptureManagerPipeline",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CaptureManagerMediaRW",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CaptureDataConverter",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataIngestCore",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LiveLinkHubCaptureMessaging",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CaptureManagerStyle",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CaptureManagerTakeMetadata",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CaptureMetadataExtraction",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "CaptureData",
			"Enabled": true
		},
		{
			"Name": "ControlFlows",
			"Enabled": true
		},
		{
			"Name": "NamingTokens",
			"Enabled": true
		},
		{
			"Name": "ElectraPlayer",
			"Enabled": true
		}
	]
}