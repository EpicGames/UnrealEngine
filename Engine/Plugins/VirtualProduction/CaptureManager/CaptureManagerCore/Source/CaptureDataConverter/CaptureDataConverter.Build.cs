// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class CaptureDataConverter : ModuleRules
{
	public CaptureDataConverter(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CaptureUtils",
			"CaptureManagerTakeMetadata",
			"CaptureManagerPipeline",
			"CaptureManagerMediaRW"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CoreUObject",
			"Engine",
			"CaptureDataUtils",
			"ImageWrapper",
			"Json",
			"DataIngestCore",
			"Media",
			"NamingTokens"
		});
	}
}
