// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerConversionTypes.h"

namespace UE::CaptureManager
{

EMediaTexturePixelFormat ToMediaPixelFormat(ECaptureManagerPixelFormat InPixelFormat)
{
	switch (InPixelFormat)
	{
		case ECaptureManagerPixelFormat::U8_RGB:   return EMediaTexturePixelFormat::U8_RGB;
		case ECaptureManagerPixelFormat::U8_BGR:   return EMediaTexturePixelFormat::U8_BGR;
		case ECaptureManagerPixelFormat::U8_RGBA:  return EMediaTexturePixelFormat::U8_RGBA;
		case ECaptureManagerPixelFormat::U8_BGRA:  return EMediaTexturePixelFormat::U8_BGRA;
		case ECaptureManagerPixelFormat::U8_I444:  return EMediaTexturePixelFormat::U8_I444;
		case ECaptureManagerPixelFormat::U8_I420:  return EMediaTexturePixelFormat::U8_I420;
		case ECaptureManagerPixelFormat::U8_NV12:  return EMediaTexturePixelFormat::U8_NV12;
		case ECaptureManagerPixelFormat::U8_Mono:  return EMediaTexturePixelFormat::U8_Mono;
		case ECaptureManagerPixelFormat::U16_Mono: return EMediaTexturePixelFormat::U16_Mono;
		case ECaptureManagerPixelFormat::F_Mono:   return EMediaTexturePixelFormat::F_Mono;
		case ECaptureManagerPixelFormat::Undefined:
		default:
			check(false);
			return EMediaTexturePixelFormat::Undefined;
	}
}

EMediaOrientation ToMediaOrientation(ECaptureManagerRotation InRotation)
{
	switch (InRotation)
	{
		case ECaptureManagerRotation::CW_90:  return EMediaOrientation::CW90;
		case ECaptureManagerRotation::CW_180: return EMediaOrientation::CW180;
		case ECaptureManagerRotation::CW_270: return EMediaOrientation::CW270;
		case ECaptureManagerRotation::None:   return EMediaOrientation::Original;
		case ECaptureManagerRotation::Auto:
			ensureMsgf(false, TEXT("ToMediaOrientation does not support Auto"));
			return EMediaOrientation::Original;
		default:
			ensureMsgf(false, TEXT("ToMediaOrientation: unhandled rotation value %d"), static_cast<uint8>(InRotation));
			return EMediaOrientation::Original;
	}
}

EMediaOrientation ToUprightRotation(FTakeMetadata::FVideo::EOrientation InOrientation)
{
	switch (InOrientation)
	{
		case FTakeMetadata::FVideo::EOrientation::CW90:  return EMediaOrientation::CW270;
		case FTakeMetadata::FVideo::EOrientation::CW180: return EMediaOrientation::CW180;
		case FTakeMetadata::FVideo::EOrientation::CW270: return EMediaOrientation::CW90;
		case FTakeMetadata::FVideo::EOrientation::Original:
		default:
			return EMediaOrientation::Original;
	}
}

} // namespace UE::CaptureManager
