// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaSample.h"
#include "CaptureManagerTakeMetadata.h"

#include "CaptureManagerConversionTypes.generated.h"

#define UE_API CAPTUREDATACONVERTER_API

UENUM(BlueprintType)
enum class ECaptureManagerPixelFormat : uint8
{
	// Packed RGB
	U8_RGB = 0		UMETA(DisplayName = "RGB 8-bit"),
	U8_BGR			UMETA(DisplayName = "BGR 8-bit"),
	U8_RGBA			UMETA(DisplayName = "RGBA 8-bit"),
	U8_BGRA			UMETA(DisplayName = "BGRA 8-bit"),
	// Planar YUV — only supported when a third-party encoder (e.g. FFmpeg) is configured.
	U8_I444			UMETA(DisplayName = "I444 8-bit"),
	U8_I420			UMETA(DisplayName = "I420 8-bit"),
	U8_NV12			UMETA(DisplayName = "NV12 8-bit"),
	// Grayscale
	U8_Mono			UMETA(DisplayName = "Mono 8-bit"),
	U16_Mono		UMETA(DisplayName = "Mono 16-bit"),
	F_Mono			UMETA(DisplayName = "Mono Float"),

	Default = U8_RGB	UMETA(DisplayName = "Default (U8_RGB)"),
	Undefined = 255		UMETA(Hidden),
};

UENUM(BlueprintType)
enum class ECaptureManagerRotation : uint8
{
	None = 0	UMETA(DisplayName = "None"),
	CW_90		UMETA(DisplayName = "Clockwise 90°"),
	CW_180		UMETA(DisplayName = "Clockwise 180°"),
	CW_270		UMETA(DisplayName = "Clockwise 270°"),
	Auto = 255	UMETA(DisplayName = "Auto"),

	Default = Auto	UMETA(DisplayName = "Default (Auto)"),
};

namespace UE::CaptureManager
{

/** Maps ECaptureManagerPixelFormat to the Core media pipeline type. */
UE_API EMediaTexturePixelFormat ToMediaPixelFormat(ECaptureManagerPixelFormat InPixelFormat);

/** Maps ECaptureManagerRotation to the Core media pipeline type. */
UE_API EMediaOrientation ToMediaOrientation(ECaptureManagerRotation InRotation);

/** Converts a sensor capture orientation to the image rotation needed to display it upright. */
UE_API EMediaOrientation ToUprightRotation(FTakeMetadata::FVideo::EOrientation InOrientation);

} // namespace UE::CaptureManager

#undef UE_API
