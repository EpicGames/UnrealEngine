// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureDataConverter.h"
#include "CaptureManagerEncoderCommands.h"

#include "Nodes/CaptureConvertVideoData.h"
#include "Nodes/CaptureConvertAudioData.h"
#include "Nodes/CaptureConvertDepthData.h"
#include "Nodes/CaptureConvertCalibrationData.h"
#include "Nodes/CaptureValidationNode.h"
#include "Nodes/ThirdPartyEncoder/CaptureConvertVideoDataThirdParty.h"
#include "Nodes/ThirdPartyEncoder/CaptureConvertAudioDataThirdParty.h"

FCaptureDataConverter::FCaptureDataConverter()
	: Pipeline(MakeShared<FCaptureManagerPipeline>(EPipelineExecutionPolicy::Asynchronous))
{
}

FCaptureDataConverter::~FCaptureDataConverter()
{
	Pipeline->Cancel();
}

void FCaptureDataConverter::AddCustomNode(TSharedPtr<FCaptureConvertCustomData> InCustomNode)
{
	CustomNodes.Add(MoveTemp(InCustomNode));
}

void FCaptureDataConverter::AddSyncNode(TSharedPtr<FCaptureConvertCustomData> InCustomNode)
{
	SyncNodes.Add(MoveTemp(InCustomNode));
}

FCaptureDataConverterResult<void> FCaptureDataConverter::Run(FCaptureDataConverterParams InParams, FProgressReporter InProgressReporter)
{
	using namespace UE::CaptureManager;

	const FTakeMetadata& TakeMetadata = InParams.TakeMetadata;

	FTaskProgress::FProgressReporter ProgressReporter = FTaskProgress::FProgressReporter::CreateLambda([this, ProgressReporter = MoveTemp(InProgressReporter)](float InProgress)
	{
		ProgressReporter.ExecuteIfBound(InProgress);
	});

	const uint32 NumberOfTasks = TakeMetadata.Video.Num() + TakeMetadata.Audio.Num() + TakeMetadata.Depth.Num() + TakeMetadata.Calibration.Num();

	TSharedPtr<FTaskProgress> TaskProgress =
		MakeShared<FTaskProgress>(NumberOfTasks, MoveTemp(ProgressReporter));

	FCaptureConvertDataNodeParams NodeParams;
	NodeParams.TaskProgress = TaskProgress;
	NodeParams.TakeOriginDirectory = InParams.TakeOriginDirectory;
	NodeParams.StopToken = InParams.ExternalStopToken.IsSet()
		? *InParams.ExternalStopToken
		: StopRequester.CreateToken();

	if (!TakeMetadata.Video.IsEmpty())
	{
		// It should be params per node. TODO: Should return and edit the node somehow
		checkf(InParams.VideoOutputParams.IsSet(), TEXT("Parameters for the Video MUST be set"));

		const TArray<FTakeMetadata::FVideo>& Videos = TakeMetadata.Video;
		const FCaptureConvertVideoOutputParams& VideoParams = InParams.VideoOutputParams.GetValue();

		for (const FTakeMetadata::FVideo& Video : Videos)
		{
			TSharedPtr<FConvertVideoNode> Node;

			if (InParams.VideoEncoderConfig)
			{
				const FCaptureManagerEncoderConfig& Cfg = *InParams.VideoEncoderConfig;
				Node = MakeShared<FCaptureConvertVideoDataThirdParty>(FVideoEncoderCommand(Cfg.EncoderPath, Cfg.EncoderArgs), Video, InParams.TakeOutputDirectory, NodeParams, VideoParams);
			}
			else
			{
				Node = MakeShared<FCaptureConvertVideoData>(Video, InParams.TakeOutputDirectory, NodeParams, VideoParams);
			}
			
			Pipeline->AddConvertVideoNode(MoveTemp(Node));
		}
	}

	if (!TakeMetadata.Audio.IsEmpty())
	{
		// It should be params per node. TODO: Should return and edit the node somehow
		checkf(InParams.AudioOutputParams.IsSet(), TEXT("Parameters for the Audio MUST be set"));

		const TArray<FTakeMetadata::FAudio>& Audios = TakeMetadata.Audio;
		const FCaptureConvertAudioOutputParams& AudioParams = InParams.AudioOutputParams.GetValue();

		for (const FTakeMetadata::FAudio& Audio : Audios)
		{
			TSharedPtr<FConvertAudioNode> Node;

			if (InParams.AudioEncoderConfig)
			{
				const FCaptureManagerEncoderConfig& Cfg = *InParams.AudioEncoderConfig;
				Node = MakeShared<FCaptureConvertAudioDataThirdParty>(FAudioEncoderCommand(Cfg.EncoderPath, Cfg.EncoderArgs), Audio, InParams.TakeOutputDirectory, NodeParams, AudioParams);
			}
			else
			{
				Node = MakeShared<FCaptureConvertAudioData>(Audio, InParams.TakeOutputDirectory, NodeParams, AudioParams);
			}

			Pipeline->AddConvertAudioNode(MoveTemp(Node));
		}
	}

	if (!TakeMetadata.Depth.IsEmpty())
	{
		// It should be params per node. TODO: Should return and edit the node somehow
		checkf(InParams.DepthOutputParams.IsSet(), TEXT("Parameters for the Depth MUST be set"));

		const TArray<FTakeMetadata::FVideo>& Depths = TakeMetadata.Depth;
		const FCaptureConvertDepthOutputParams& DepthParams = InParams.DepthOutputParams.GetValue();

		for (const FTakeMetadata::FVideo& Depth : Depths)
		{
			TSharedPtr<FCaptureConvertDepthData> Node =
				MakeShared<FCaptureConvertDepthData>(Depth, InParams.TakeOutputDirectory, NodeParams, DepthParams);

			Pipeline->AddConvertDepthNode(MoveTemp(Node));
		}
	}

	if (!TakeMetadata.Calibration.IsEmpty())
	{
		// It should be params per node. TODO: Should return and edit the node somehow
		checkf(InParams.CalibrationOutputParams.IsSet(), TEXT("Parameters for the Calibration MUST be set"));

		const TArray<FTakeMetadata::FCalibration>& Calibrations = TakeMetadata.Calibration;
		const FCaptureConvertCalibrationOutputParams& CalibrationParams = InParams.CalibrationOutputParams.GetValue();

		for (const FTakeMetadata::FCalibration& Calibration : Calibrations)
		{
			TSharedPtr<FCaptureConvertCalibrationData> Node =
				MakeShared<FCaptureConvertCalibrationData>(Calibration, InParams.TakeOutputDirectory, NodeParams, CalibrationParams);

			Pipeline->AddConvertCalibrationNode(MoveTemp(Node));
		}
	}

	for (TSharedPtr<FCaptureConvertCustomData> CustomNode : CustomNodes)
	{
		CustomNode->SetParams(NodeParams);

		Pipeline->AddGenericNode(MoveTemp(CustomNode));
	}

	for (TSharedPtr<FCaptureConvertCustomData> SyncNode : SyncNodes)
	{
		SyncNode->SetParams(NodeParams);

		Pipeline->AddSyncedNode(MoveTemp(SyncNode));
	}

	TSharedPtr<FCaptureValidationNode> ValidationNode = 
		MakeShared<FCaptureValidationNode>(InParams, TakeMetadata);

	Pipeline->AddSyncedNode(MoveTemp(ValidationNode));

	FCaptureManagerPipeline::FResult Result = Pipeline->Run();

	TArray<FText> Errors;
	for (const TPair<FGuid, FCaptureManagerPipelineNode::FResult>& NodeResult : Result)
	{
		if (NodeResult.Value.HasError())
		{
			Errors.Add(NodeResult.Value.GetError().GetMessage());
		}
	}

	CustomNodes.Empty();
	SyncNodes.Empty();

	if (!Errors.IsEmpty())
	{
		return MakeError(MoveTemp(Errors));
	}

	return MakeValue();
}

void FCaptureDataConverter::Cancel()
{
	if (!StopRequester.IsStopRequested())
	{
		StopRequester.RequestStop();

		Pipeline->Cancel();
	}
}