// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureDataConverterError.h"

FCaptureDataConverterError::FCaptureDataConverterError(TArray<FText> InErrors)
	: Errors(MoveTemp(InErrors))
{
}

FCaptureDataConverterError::~FCaptureDataConverterError() = default;

TArray<FText> FCaptureDataConverterError::GetErrors() const
{
    return Errors;
}