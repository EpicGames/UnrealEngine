// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerEncoderCommands.h"

#include "CaptureManagerEncoderConfig.h"

#include "Nodes/ThirdPartyEncoder/CaptureThirdPartyNodeUtils.h"

namespace UE::CaptureManager
{
FAudioEncoderCommand::FAudioEncoderCommand(FString InEncoderPath, FString InEncoderArgs) :
	EncoderPath(MoveTemp(InEncoderPath)),
	EncoderArgs(MoveTemp(InEncoderArgs))
{
}

FProcessRunnerResult FAudioEncoderCommand::Execute(const FString& InInputPath, const FString& InOutputPath, const FStopToken* InStopToken) const
{
	const FStringFormatNamedArguments FormatArgs = {
		{FString(AudioEncoderTokens::InputKey), WrapInQuotes(InInputPath)},
		{FString(AudioEncoderTokens::OutputKey), WrapInQuotes(InOutputPath)}
	};

	const FString Args = FString::Format(*EncoderArgs, FormatArgs);
	return FProcessRunner::Run(EncoderPath, Args, InStopToken);
}

FVideoEncoderCommand::FVideoEncoderCommand(FString InEncoderPath, FString InEncoderArgs) :
	EncoderPath(MoveTemp(InEncoderPath)),
	EncoderArgs(MoveTemp(InEncoderArgs))
{
}

FProcessRunnerResult FVideoEncoderCommand::Execute(const FString& InInputPath, const FString& InOutputPath, const FString& InVideoParams, const FStopToken* InStopToken) const
{
	const FStringFormatNamedArguments FormatArgs = {
		{FString(VideoEncoderTokens::InputKey), WrapInQuotes(InInputPath)},
		{FString(VideoEncoderTokens::OutputKey), WrapInQuotes(InOutputPath)},
		{FString(VideoEncoderTokens::ParamsKey), InVideoParams}
	};

	const FString Args = FString::Format(*EncoderArgs, FormatArgs);
	return FProcessRunner::Run(EncoderPath, Args, InStopToken);
}
}
