// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/StopToken.h"
#include "ProcessRunner/ProcessRunner.h"

// Encoder args use a two-pass resolution scheme:
//
// Pass 1 - Naming tokens (EvaluateTokenString, called once at setup time):
//   Resolves any custom naming tokens the user has embedded in their command pattern
//   (e.g. project-level tokens from UNamingTokens subclasses). The {input}, {output}, and
//   {params} placeholders are intentionally left untouched at this stage because the actual
//   file paths are not yet known. UCaptureManagerVideoEncoderTokens and
//   UCaptureManagerAudioEncoderTokens register these keys with pass-through processors that
//   return the placeholder unchanged, so EvaluateTokenString treats them as resolved-but-
//   identity and leaves them in the string.
//
// Pass 2 - Runtime substitution (FString::Format, called in Execute()):
//   Substitutes {input}, {output}, and {params} with the real file paths and conversion
//   parameters supplied at conversion time.
//
// This split keeps Core encoder logic free of engine-subsystem dependencies: only the
// App-layer setup code (e.g. ApplyThirdPartyEncoderSettings) calls EvaluateTokenString;
// Execute() performs a pure FString::Format with no subsystem access required.

namespace UE::CaptureManager
{

class FAudioEncoderCommand
{
public:
	FAudioEncoderCommand(FString InEncoderPath, FString InEncoderArgs);

	FProcessRunnerResult Execute(const FString& InInputPath, const FString& InOutputPath, const FStopToken* InStopToken = nullptr) const;

private:
	const FString EncoderPath;
	const FString EncoderArgs;
};

class FVideoEncoderCommand
{
public:
	FVideoEncoderCommand(FString InEncoderPath, FString InEncoderArgs);

	FProcessRunnerResult Execute(const FString& InInputPath, const FString& InOutputPath, const FString& InVideoParams, const FStopToken* InStopToken = nullptr) const;

private:
	const FString EncoderPath;
	const FString EncoderArgs;
};

}
