// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerEncoderNamingTokens.h"
#include "CaptureManagerEncoderConfig.h"

#define LOCTEXT_NAMESPACE "CaptureManagerEncoderNamingTokens"

// Encoder command resolution is two-pass:
//   Pass 1: EvaluateTokenString (ResolveCommandArgs) resolves user-facing tokens such as
//           pixel format and frame rate into their values. {input}, {output} and {params} are
//           pass-through tokens - their processor delegates return the key wrapped in braces
//           so they survive Pass 1 unchanged.
//   Pass 2: FString::Format in the encoder Execute() functions substitutes the runtime
//           values for {input}, {output} and {params}.

namespace UE::CaptureManager::Private
{
// Returns the token key wrapped in braces so it survives Pass 1 unchanged and is
// available for Pass 2 FString::Format substitution.
// NOTE: {input}, {output} and {params} are registered as naming tokens purely for
// discoverability - so they appear in the token picker UI alongside the user-facing
// tokens. They do not resolve to a value at evaluation time; their actual substitution
// happens in Pass 2. This is intentional.
FText MakePassthroughToken(FStringView InKey)
{
	return FText::FromString(FString::Printf(TEXT("{%s}"), *FString(InKey)));
}
}

UCaptureManagerVideoEncoderTokens::UCaptureManagerVideoEncoderTokens()
{
	using namespace UE::CaptureManager;
	Namespace = FString(VideoEncoderTokens::Namespace);
}

void UCaptureManagerVideoEncoderTokens::OnCreateDefaultTokens(TArray<FNamingTokenData>& OutTokens)
{
	using namespace UE::CaptureManager::Private;

	Super::OnCreateDefaultTokens(OutTokens);

	OutTokens.Add({
		FString(UE::CaptureManager::VideoEncoderTokens::InputKey),
		LOCTEXT("VideoInputPath", "Input File Path"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([] {
				return MakePassthroughToken(UE::CaptureManager::VideoEncoderTokens::InputKey);
			})
		});

	OutTokens.Add({
		FString(UE::CaptureManager::VideoEncoderTokens::OutputKey),
		LOCTEXT("VideoOutputPath", "Output File Path"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([] {
				return MakePassthroughToken(UE::CaptureManager::VideoEncoderTokens::OutputKey);
			})
		});

	OutTokens.Add({
		FString(UE::CaptureManager::VideoEncoderTokens::ParamsKey),
		LOCTEXT("VideoParams", "Conversion Parameters (e.g. Pixel Format, Rotation etc)"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([] {
				return MakePassthroughToken(UE::CaptureManager::VideoEncoderTokens::ParamsKey);
			})
		});
}

void UCaptureManagerVideoEncoderTokens::OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData)
{
	Super::OnPreEvaluate_Implementation(InEvaluationData);
}

void UCaptureManagerVideoEncoderTokens::OnPostEvaluate_Implementation()
{
	Super::OnPostEvaluate_Implementation();
}


UCaptureManagerAudioEncoderTokens::UCaptureManagerAudioEncoderTokens()
{
	Namespace = FString(UE::CaptureManager::AudioEncoderTokens::Namespace);
}

void UCaptureManagerAudioEncoderTokens::OnCreateDefaultTokens(TArray<FNamingTokenData>& OutTokens)
{
	using namespace UE::CaptureManager::Private;

	Super::OnCreateDefaultTokens(OutTokens);

	OutTokens.Add({
		FString(UE::CaptureManager::AudioEncoderTokens::InputKey),
		LOCTEXT("AudioInputPath", "Input File Path"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([] {
				return MakePassthroughToken(UE::CaptureManager::AudioEncoderTokens::InputKey);
			})
		});

	OutTokens.Add({
		FString(UE::CaptureManager::AudioEncoderTokens::OutputKey),
		LOCTEXT("AudioOutputPath", "Output File Path"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([] {
				return MakePassthroughToken(UE::CaptureManager::AudioEncoderTokens::OutputKey);
			})
		});
}

void UCaptureManagerAudioEncoderTokens::OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData)
{
	Super::OnPreEvaluate_Implementation(InEvaluationData);
}

void UCaptureManagerAudioEncoderTokens::OnPostEvaluate_Implementation()
{
	Super::OnPostEvaluate_Implementation();
}

#undef LOCTEXT_NAMESPACE
