// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureConvertAudioData.h"

#include "Async/TaskProgress.h"
#include "Async/StopToken.h"

#include "CaptureCopyProgressReporter.h"

#include "CaptureManagerMediaRWModule.h"

#include "IMediaAudioSample.h"

#define LOCTEXT_NAMESPACE "CaptureConvertAudioData"

FCaptureConvertAudioData::FCaptureConvertAudioData(const FTakeMetadata::FAudio& InAudio, 
												   const FString& InOutputDirectory, 
												   const FCaptureConvertDataNodeParams& InParams, 
												   const FCaptureConvertAudioOutputParams& InAudioParams)
	: FConvertAudioNode(InAudio, InOutputDirectory)
	, Params(InParams)
	, AudioParams(InAudioParams)
{
}

FCaptureConvertAudioData::FResult FCaptureConvertAudioData::Run()
{
	using namespace UE::CaptureManager;

	if (Params.StopToken.IsStopRequested())
	{
		FText Message = LOCTEXT("CaptureConvertAudioNode_AbortedByUser", "Aborted by user");
		return MakeError(MoveTemp(Message));
	}

	if (FPaths::GetExtension(Audio.Path) == AudioParams.Format)
	{
		return CopyAudioFile();
	}
	
	return ConvertAudioFile();
}

FCaptureConvertAudioData::FResult FCaptureConvertAudioData::CopyAudioFile()
{
	using namespace UE::CaptureManager;

	FTaskProgress::FTask Task = Params.TaskProgress->StartTask();

	FString DestinationDirectory = OutputDirectory / Audio.Name;
	const FString AudioFilePath = FPaths::ConvertRelativePathToFull(Params.TakeOriginDirectory, Audio.Path);

	FCopyProgressReporter ProgressReporter(Task, Params.StopToken);
	FString Destination = DestinationDirectory / FPaths::SetExtension(AudioParams.AudioFileName, AudioParams.Format);
	uint32 Result = IFileManager::Get().Copy(*Destination, *AudioFilePath, true, true, false, &ProgressReporter);

	if (Result == COPY_Fail)
	{
		FText Message = LOCTEXT("CaptureConvertAudioNode_CopyFailed", "Failed to copy the audio file");
		return MakeError(MoveTemp(Message));
	}

	if (Result == COPY_Canceled)
	{
		FText Message = LOCTEXT("CaptureConvertAudioNode_AbortedByUser", "Aborted by user");
		return MakeError(MoveTemp(Message));
	}

	return MakeValue();
}

FCaptureConvertAudioData::FResult FCaptureConvertAudioData::ConvertAudioFile()
{
	using namespace UE::CaptureManager;

	FTaskProgress::FTask Task = Params.TaskProgress->StartTask();

	FString DestinationDirectory = OutputDirectory / Audio.Name;
	const FString AudioFilePath = FPaths::ConvertRelativePathToFull(Params.TakeOriginDirectory, Audio.Path);

	FCaptureManagerMediaRWModule& MediaRWModule =
		FModuleManager::LoadModuleChecked<FCaptureManagerMediaRWModule>("CaptureManagerMediaRW");

	TValueOrError<TUniquePtr<IAudioReader>, FText> AudioReaderResult = MediaRWModule.Get().CreateAudioReader(AudioFilePath);

	if (AudioReaderResult.HasError())
	{
		FText Message = FText::Format(LOCTEXT("CaptureConvertAudioNode_UnsupportedFile", "Audio file format is unsupported {0}. Consider enabling Third Party Encoder in Capture Manager settings."), FText::FromString(AudioFilePath));
		return MakeError(MoveTemp(Message));
	}

	TUniquePtr<IAudioReader> AudioReader = AudioReaderResult.StealValue();

	ON_SCOPE_EXIT
	{
		AudioReader->Close();
	};

	if (AudioReader->GetSampleFormat() != EMediaAudioSampleFormat::Int16)
	{
		FText Message =
			FText::Format(LOCTEXT("CaptureConvertAudioNode_InvalidAudioFormat", "Invalid audio format in file {0}. Only 16-bit PCM is currently supported."), FText::FromString(AudioFilePath));
		return MakeError(MoveTemp(Message));
	}

	TValueOrError<TUniquePtr<IAudioWriter>, FText> AudioMediaWriterResult = MediaRWModule.Get().CreateAudioWriter(DestinationDirectory, AudioParams.AudioFileName, AudioParams.Format);

	if (AudioMediaWriterResult.HasError())
	{
		FText Message = LOCTEXT("CaptureConvertAudioNode_UnsupportedOutputFile", "Output audio file format not supported");
		return MakeError(MoveTemp(Message));
	}

	TUniquePtr<IAudioWriter> AudioMediaWriter = AudioMediaWriterResult.StealValue();

	AudioMediaWriter->Configure(AudioReader->GetSampleRate(), AudioReader->GetNumChannels(), EMediaAudioSampleFormat::Int16); // forcing int16 sample format

	ON_SCOPE_EXIT
	{
		AudioMediaWriter->Close();
	};

	const FTimespan Duration = AudioReader->GetDuration();
	
	while (true)
	{
		TValueOrError<TUniquePtr<UE::CaptureManager::FMediaAudioSample>, FText> SampleResult = AudioReader->Next();

		// Error while reading the file
		if (SampleResult.HasError())
		{
			return MakeError(SampleResult.StealError());
		}

		TUniquePtr<UE::CaptureManager::FMediaAudioSample> Sample = SampleResult.StealValue();

		// End of stream
		if (!Sample)
		{
			break;
		}

		TOptional<FText> AppendResult = AudioMediaWriter->Append(Sample.Get());
		if (AppendResult.IsSet())
		{
			FText Message =
				FText::Format(LOCTEXT("CaptureConvertAudioNode_WavFileWriteFailure", "Failed to write to the audio file: {0}"), AppendResult.GetValue());
			return MakeError(MoveTemp(Message));
		}

		if (Duration.GetTotalSeconds() > 0.0)
		{
			const float LocalProgress = Sample->Time.GetTotalSeconds() / Duration.GetTotalSeconds();
			Task.Update(LocalProgress);
		}

		if (Params.StopToken.IsStopRequested())
		{
			FText Message = LOCTEXT("CaptureConvertAudioNode_AbortedByUser", "Aborted by user");
			return MakeError(MoveTemp(Message));
		}
	}

	return MakeValue();
}

#undef LOCTEXT_NAMESPACE