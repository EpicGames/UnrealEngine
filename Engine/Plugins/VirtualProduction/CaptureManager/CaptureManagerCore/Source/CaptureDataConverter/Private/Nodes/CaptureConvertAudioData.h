// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/ConvertAudioNode.h"

#include "CaptureDataConverterNodeParams.h"

#include "MediaSample.h"

class FCaptureConvertAudioData final : 
	public FConvertAudioNode
{
public:

	FCaptureConvertAudioData(const FTakeMetadata::FAudio& InAudio,
							 const FString& InOutputDirectory,
							 const FCaptureConvertDataNodeParams& InParams,
							 const FCaptureConvertAudioOutputParams& InAudioParams);

private:

	virtual FResult Run() override;

	FResult CopyAudioFile();
	FResult ConvertAudioFile();

	FCaptureConvertDataNodeParams Params;
	FCaptureConvertAudioOutputParams AudioParams;
};