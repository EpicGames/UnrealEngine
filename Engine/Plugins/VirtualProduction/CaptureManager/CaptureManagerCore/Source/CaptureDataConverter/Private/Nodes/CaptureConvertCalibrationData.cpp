// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureConvertCalibrationData.h"

#include "Async/TaskProgress.h"
#include "Async/StopToken.h"

#include "CaptureCopyProgressReporter.h"

#include "CaptureManagerMediaRWModule.h"

#define LOCTEXT_NAMESPACE "CaptureConvertCalibrationData"

FCaptureConvertCalibrationData::FCaptureConvertCalibrationData(const FTakeMetadata::FCalibration& InCalibration,
															   const FString& InOutputDirectory,
															   const FCaptureConvertDataNodeParams& InParams,
															   const FCaptureConvertCalibrationOutputParams& InCalibrationParams)
	: FConvertCalibrationNode(InCalibration, InOutputDirectory)
	, Params(InParams)
	, CalibrationParams(InCalibrationParams)
{
}

FCaptureConvertCalibrationData::FResult FCaptureConvertCalibrationData::Run()
{
	using namespace UE::CaptureManager;

	if (Params.StopToken.IsStopRequested())
	{
		FText Message = LOCTEXT("CaptureConvertCalibrationNode_AbortedByUser", "Aborted by user");
		return MakeError(MoveTemp(Message));
	}

	if (Calibration.Format == TEXT("unreal"))
	{
		return CopyCalibrationFile();
	}

	return ConvertCalibrationFile();
}

FCaptureConvertCalibrationData::FResult FCaptureConvertCalibrationData::CopyCalibrationFile()
{
	using namespace UE::CaptureManager;

	FTaskProgress::FTask Task = Params.TaskProgress->StartTask();

	FString DestinationDirectory = OutputDirectory / Calibration.Name;
	const FString CalibrationFilePath = FPaths::ConvertRelativePathToFull(Params.TakeOriginDirectory, Calibration.Path);

	FCopyProgressReporter ProgressReporter(Task, Params.StopToken);
	FString Destination = DestinationDirectory / FPaths::SetExtension(CalibrationParams.FileName, FPaths::GetExtension(Calibration.Path));
	uint32 Result = IFileManager::Get().Copy(*Destination, *CalibrationFilePath, true, true, false, &ProgressReporter);

	if (Result == COPY_Fail)
	{
		FText Message = 
			FText::Format(LOCTEXT("CaptureConvertCalibrationNode_CopyFailed", "Failed to copy the calibration file to {0}"), FText::FromString(Destination));
		return MakeError(MoveTemp(Message));
	}

	if (Result == COPY_Canceled)
	{
		FText Message = LOCTEXT("CaptureConvertCalibrationNode_AbortedByUser", "Aborted by user");
		return MakeError(MoveTemp(Message));
	}

	return MakeValue();
}

FCaptureConvertCalibrationData::FResult FCaptureConvertCalibrationData::ConvertCalibrationFile()
{
	using namespace UE::CaptureManager;

	if (Params.StopToken.IsStopRequested())
	{
		FText Message = LOCTEXT("CaptureConvertCalibrationNode_AbortedByUser", "Aborted by user");
		return MakeError(MoveTemp(Message));
	}

	FTaskProgress::FTask Task = Params.TaskProgress->StartTask();

	const FString CalibrationFilePath = FPaths::ConvertRelativePathToFull(Params.TakeOriginDirectory, Calibration.Path);
	FString TargetDirectory = OutputDirectory / Calibration.Name;

	FCaptureManagerMediaRWModule& MediaRWModule =
		FModuleManager::LoadModuleChecked<FCaptureManagerMediaRWModule>("CaptureManagerMediaRW");

	TUniquePtr<ICalibrationReader> CalibrationReader = MediaRWModule.Get().CreateCalibrationReaderByFormat(Calibration.Format);
	if (!CalibrationReader)
	{
		FString Format = Calibration.Format;
		if (Format.IsEmpty())
		{
			Format = TEXT("<not specified>");
		}
		FText Message = 
			FText::Format(LOCTEXT("CaptureConvertCalibrationNode_UnsupportedFormat", "Calibration format is unsupported {0} for file: {1}"), 
						  FText::FromString(Format), 
						  FText::FromString(CalibrationFilePath));
		return MakeError(MoveTemp(Message));
	}

	TOptional<FText> Error = CalibrationReader->Open(CalibrationFilePath);
	if (Error.IsSet())
	{
		return MakeError(MoveTemp(Error.GetValue()));
	}

	ON_SCOPE_EXIT
	{
		CalibrationReader->Close();
	};

	TUniquePtr<ICalibrationWriter> CalibrationWriter = MediaRWModule.Get().CreateCalibrationWriterByFormat(TEXT("unreal"));
	if (!CalibrationWriter)
	{
		FText Message = FText::Format(LOCTEXT("CaptureConvertCalibrationNode_WriterCreationFailed", "Calibration writer creation failed while converting file: {0}"), FText::FromString(CalibrationFilePath));
		return MakeError(MoveTemp(Message));
	}

	Error = CalibrationWriter->Open(TargetDirectory, CalibrationParams.FileName, TEXT("unreal"));
	if (Error.IsSet())
	{
		return MakeError(MoveTemp(Error.GetValue()));
	}

	ON_SCOPE_EXIT
	{
		CalibrationWriter->Close();
	};

	while (true)
	{
		TValueOrError<TUniquePtr<UE::CaptureManager::FMediaCalibrationSample>, FText> CalibrationSampleResult = CalibrationReader->Next();

		if (CalibrationSampleResult.HasError())
		{
			return MakeError(CalibrationSampleResult.StealError());
		}

		TUniquePtr<UE::CaptureManager::FMediaCalibrationSample> Sample = CalibrationSampleResult.StealValue();

		if (!Sample)
		{
			break;
		}

		TOptional<FText> AppendResult = CalibrationWriter->Append(Sample.Get());
		if (AppendResult.IsSet())
		{
			FText Message =
				FText::Format(LOCTEXT("CaptureConvertCalibrationNode_CalibFileWriteFailure", "Failed to write to the calibration file: {0}"), AppendResult.GetValue());
			return MakeError(MoveTemp(Message));
		}

		if (Params.StopToken.IsStopRequested())
		{
			FText Message = LOCTEXT("CaptureConvertCalibrationNode_AbortedByUser", "Aborted by user");
			return MakeError(MoveTemp(Message));
		}
	}

	Task.Update(1.0f);

	return MakeValue();
}

#undef LOCTEXT_NAMESPACE