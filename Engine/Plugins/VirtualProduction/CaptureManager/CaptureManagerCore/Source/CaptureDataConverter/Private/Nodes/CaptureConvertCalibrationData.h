// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/ConvertCalibrationNode.h"

#include "CaptureDataConverterNodeParams.h"

#include "MediaSample.h"

class FCaptureConvertCalibrationData final :
	public FConvertCalibrationNode
{
public:

	FCaptureConvertCalibrationData(const FTakeMetadata::FCalibration& InCalibration,
								   const FString& InOutputDirectory,
								   const FCaptureConvertDataNodeParams& InParams,
								   const FCaptureConvertCalibrationOutputParams& InCalibrationParams);

private:

	virtual FResult Run() override;

	FResult CopyCalibrationFile();
	FResult ConvertCalibrationFile();

	FCaptureConvertDataNodeParams Params;
	FCaptureConvertCalibrationOutputParams CalibrationParams;
};