// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureConvertDepthData.h"

#include "Async/TaskProgress.h"
#include "Async/StopToken.h"

#include "TakeDirectoryUtils.h"

#include "CaptureManagerMediaRWModule.h"
#include "CaptureCopyProgressReporter.h"

#include "IImageWrapperModule.h"

#define LOCTEXT_NAMESPACE "CaptureConvertDepthData"

DEFINE_LOG_CATEGORY_STATIC(LogCaptureConvertDepthData, Log, All);

FCaptureConvertDepthData::FCaptureConvertDepthData(const FTakeMetadata::FVideo& InDepth,
												   const FString& InOutputDirectory,
												   const FCaptureConvertDataNodeParams& InParams,
												   const FCaptureConvertDepthOutputParams& InDepthParams)

	: FConvertDepthNode(InDepth, InOutputDirectory)
	, Params(InParams)
	, DepthParams(InDepthParams)
{
}

FCaptureConvertDepthData::~FCaptureConvertDepthData() = default;

FCaptureConvertDepthData::FResult FCaptureConvertDepthData::Run()
{
	using namespace UE::CaptureManager;

	if (Params.StopToken.IsStopRequested())
	{
		FText Message = LOCTEXT("CaptureConvertDepth_AbortedByUser", "Aborted by user");
		return MakeError(MoveTemp(Message));
	}

	if (!Depth.PathType.IsSet())
	{
		Depth.PathType = FTakeMetadataPathUtils::DetectPathType(Depth.Path);
		FString PathTypeString = FTakeMetadataPathUtils::PathTypeToString(Depth.PathType.GetValue());
		UE_LOGF(LogCaptureConvertDepthData, Display, "PathType for %ls is unspecified, setting to detected type %ls", *Depth.Path, *PathTypeString);
	}
	else
	{
		FTakeMetadataPathUtils::ValidatePathType(Depth.Path, Depth.PathType.GetValue());
	}

	if (Depth.PathType.IsSet() && Depth.PathType.GetValue() == FTakeMetadata::FVideo::EPathType::File)
	{
		return ConvertData();
	}
	
	return CopyData();
}

FCaptureConvertDepthData::FResult FCaptureConvertDepthData::ConvertData()
{
	using namespace UE::CaptureManager;

	if (Params.StopToken.IsStopRequested())
	{
		FText Message = LOCTEXT("CaptureConvertDepthData_AbortedByUser", "Aborted by user");
		return MakeError(MoveTemp(Message));
	}

	FTaskProgress::FTask Task = Params.TaskProgress->StartTask();

	const FString DepthFilePath = FPaths::ConvertRelativePathToFull(Params.TakeOriginDirectory, Depth.Path);
	FString TargetDirectory = OutputDirectory / Depth.Name;

	FCaptureManagerMediaRWModule& MediaRWModule =
		FModuleManager::LoadModuleChecked<FCaptureManagerMediaRWModule>("CaptureManagerMediaRW");

	TUniquePtr<IVideoReader> DepthReader = MediaRWModule.Get().CreateVideoReaderByFormat(Depth.Format);
	TValueOrError<TUniquePtr<IImageWriter>, FText> ImageWriterResult = MediaRWModule.Get().CreateImageWriter(TargetDirectory, TEXT("depth"), TEXT("exr"));

	if (!DepthReader || ImageWriterResult.HasError())
	{
		FText Message = FText::Format(LOCTEXT("CaptureConvertDepthData_UnsupportedFile", "Depth file format is unsupported {0}"), FText::FromString(DepthFilePath));
		return MakeError(MoveTemp(Message));
	}

	TOptional<FText> DepthReaderOpen = DepthReader->Open(DepthFilePath);
	if (DepthReaderOpen.IsSet())
	{
		return MakeError(MoveTemp(DepthReaderOpen.GetValue()));
	}

	TUniquePtr<IImageWriter> ImageWriter = ImageWriterResult.StealValue();

	ON_SCOPE_EXIT
	{
		DepthReader->Close();
		ImageWriter->Close();
	};

	while (true)
	{
		TValueOrError<TUniquePtr<UE::CaptureManager::FMediaTextureSample>, FText> DepthSampleResult = DepthReader->Next();

		if (DepthSampleResult.HasError())
		{
			return MakeError(DepthSampleResult.StealError());
		}

		TUniquePtr<UE::CaptureManager::FMediaTextureSample> DepthSample = DepthSampleResult.StealValue();

		// End of stream
		if (!DepthSample.IsValid())
		{
			break;
		}

		FWritingContext Context;
		Context.ReadSample = MoveTemp(DepthSample);
		Context.Writer = ImageWriter.Get();

		if (Depth.FramesCount.IsSet())
		{
			Context.TotalDuration = Depth.FramesCount.GetValue();
		}

		Context.Task = &Task;

		FWriteResult WriteResult = OnWrite(MoveTemp(Context));

		if (WriteResult.HasError())
		{
			return MakeError(WriteResult.StealError());
		}
	}

	return MakeValue();
}

FCaptureConvertDepthData::FResult FCaptureConvertDepthData::CopyData()
{
	using namespace UE::CaptureManager;

	const FString DepthFolderPath = FPaths::ConvertRelativePathToFull(Params.TakeOriginDirectory, Depth.Path);

	IFileManager& FileManager = IFileManager::Get();

	TArray<FString> ImagesToCopy;
	FileManager.FindFiles(ImagesToCopy, *DepthFolderPath, Depth.Format.IsEmpty() ? nullptr : *Depth.Format);

	IImageWrapperModule& ImageWrapperModule = FModuleManager::Get().LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
	ImagesToCopy.RemoveAllSwap([&ImageWrapperModule](const FString& InFilePath)
	{
		return ImageWrapperModule.GetImageFormatFromExtension(*InFilePath) != EImageFormat::EXR;
	});

	if (ImagesToCopy.IsEmpty())
	{
		FText ExtensionPostfix = Depth.Format.IsEmpty() ? FText() : FText::Format(LOCTEXT("CaptureConvertDepth_Extension", " with specified extension .{0}"), FText::FromString(Depth.Format));
		FText Message = FText::Format(LOCTEXT("CaptureConvertDepth_EmptyData", "Copy depth data failed. No depth data found at {0}{1}"), FText::FromString(DepthFolderPath), ExtensionPostfix);
		UE_LOGF(LogCaptureConvertDepthData, Error, "%ls", *Message.ToString());

		return MakeError(MoveTemp(Message));
	}
	
	using namespace UE::CaptureData;

	FRegexPattern Pattern = GetRegexPattern();
	ImagesToCopy.Sort([&Pattern](const FString& InLeft, const FString& InRight)
	{
		FString Dummy;
		FString LeftDigits, RightDigits;
		bool bSuccess = ExtractInfoFromFileName(Pattern, InLeft, Dummy, LeftDigits, Dummy);
		bSuccess = bSuccess && ExtractInfoFromFileName(Pattern, InRight, Dummy, RightDigits, Dummy);

		if (bSuccess)
		{
			return FCString::Atoi(*LeftDigits) < FCString::Atoi(*RightDigits);
		}

		return InLeft < InRight;
	});

	if (!Depth.FramesCount.IsSet() || Depth.FramesCount.GetValue() == 0)
	{
		Depth.FramesCount = ImagesToCopy.Num();
	}

	FTaskProgress::FTask Task = Params.TaskProgress->StartTask();
	TSharedPtr<FTaskProgress> TaskProgress = MakeShared<FTaskProgress>(Depth.FramesCount.GetValue(), FTaskProgress::FProgressReporter::CreateLambda([&Task](float InProgress)
	{
		Task.Update(InProgress);
	}));

	FString DestinationDirectory = OutputDirectory / Depth.Name;

	for (const FString& FilePath : ImagesToCopy)
	{
		FTaskProgress::FTask InnerTask = TaskProgress->StartTask();

		FCopyProgressReporter ProgressReporter(InnerTask, Params.StopToken);

		FString Destination = DestinationDirectory / FilePath;
		FString Source = DepthFolderPath / FilePath;

		uint32 CopyResult = FileManager.Copy(*Destination, *Source, true, true, false, &ProgressReporter);

		if (CopyResult == COPY_Fail)
		{
			FText Message = LOCTEXT("CaptureConvertDepthData_CopyFailed", "Failed to copy the file");
			return MakeError(MoveTemp(Message));
		}

		if (CopyResult == COPY_Canceled)
		{
			FText Message = LOCTEXT("CaptureConvertDepthData_AbortedByUser", "Aborted by user");
			return MakeError(MoveTemp(Message));
		}
	}

	return MakeValue();
}

FCaptureConvertDepthData::FWriteResult FCaptureConvertDepthData::OnWrite(FWritingContext InContext)
{
	TUniquePtr<UE::CaptureManager::FMediaTextureSample>& DepthSample = InContext.ReadSample;
	DepthSample->Rotation = DepthParams.Rotation;

	TOptional<FText> WriterResult = InContext.Writer->Append(DepthSample.Get());

	if (WriterResult.IsSet())
	{
		return MakeError(MoveTemp(WriterResult.GetValue()));
	}

	++CurrentFrame;

	if (InContext.TotalDuration.IsSet() && InContext.TotalDuration.GetValue() > 0)
	{
		// We can only update the progress if we know the total duration
		InContext.Task->Update((float)CurrentFrame / InContext.TotalDuration.GetValue());
	}

	if (Params.StopToken.IsStopRequested())
	{
		FText Message = LOCTEXT("LiveLinkConvertDepthNode_AbortedByUser", "Aborted by user");
		return MakeError(MoveTemp(Message));
	}

	return MakeValue();
}

#undef LOCTEXT_NAMESPACE