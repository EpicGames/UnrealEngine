// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/ConvertDepthNode.h"

#include "CaptureDataConverterNodeParams.h"

#include "MediaSample.h"

class FCaptureConvertDepthData final
	: public FConvertDepthNode
{
public:

	FCaptureConvertDepthData(const FTakeMetadata::FVideo& InDepth,
							 const FString& InOutputDirectory,
							 const FCaptureConvertDataNodeParams& InParams,
							 const FCaptureConvertDepthOutputParams& InDepthParams);

	virtual ~FCaptureConvertDepthData() override;

private:

	virtual FResult Run() override;

	FResult ConvertData();
	FResult CopyData();

	FCaptureConvertDataNodeParams Params;
	FCaptureConvertDepthOutputParams DepthParams;

	struct FWritingContext
	{
		FWritingContext()
			: ReadSample(nullptr)
			, Writer(nullptr)
			, Task(nullptr)
			, TotalDuration(0)
		{
		}

		TUniquePtr<UE::CaptureManager::FMediaTextureSample> ReadSample;
		class IImageWriter* Writer;
		UE::CaptureManager::FTaskProgress::FTask* Task;
		TOptional<int32> TotalDuration;
	};

	using FWriteResult = TValueOrError<void, FText>;
	FWriteResult OnWrite(FWritingContext InContext);

	int32 CurrentFrame = 0;
};