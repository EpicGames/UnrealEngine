// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/CaptureConvertParams.h"

FCaptureConvertParams::FCaptureConvertParams() = default;
FCaptureConvertParams::~FCaptureConvertParams() = default;

void FCaptureConvertParams::SetParams(const FCaptureConvertDataNodeParams& InParams)
{
	Params = InParams;
}
