// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureConvertVideoData.h"

#include "Async/TaskProgress.h"
#include "Async/StopToken.h"

#include "CaptureManagerMediaRWModule.h"
#include "CaptureCopyProgressReporter.h"

#include "TakeDirectoryUtils.h"

#include "IImageWrapperModule.h"

#define LOCTEXT_NAMESPACE "CaptureConvertVideoData"

DEFINE_LOG_CATEGORY_STATIC(LogCaptureConvertVideoData, Log, All);

FCaptureConvertVideoData::FCaptureConvertVideoData(const FTakeMetadata::FVideo& InVideo, 
												   const FString& InOutputDirectory, 
												   const FCaptureConvertDataNodeParams& InParams,
												   const FCaptureConvertVideoOutputParams& InVideoParams)
	: FConvertVideoNode(InVideo, InOutputDirectory)
	, Params(InParams)
	, VideoParams(InVideoParams)
{
	checkf(!VideoParams.Format.IsEmpty(), TEXT("Output image format must be specified"));
}

FCaptureConvertVideoData::~FCaptureConvertVideoData() = default;

FCaptureConvertVideoData::FResult FCaptureConvertVideoData::Run()
{
	using namespace UE::CaptureManager;

	if (Params.StopToken.IsStopRequested())
	{
		FText Message = LOCTEXT("CaptureConvertVideo_AbortedByUser", "Video conversion aborted by user");
		return MakeError(MoveTemp(Message));
	}

	if (!Video.PathType.IsSet())
	{
		Video.PathType = FTakeMetadataPathUtils::DetectPathType(Video.Path);
		FString PathTypeString = FTakeMetadataPathUtils::PathTypeToString(Video.PathType.GetValue());
		UE_LOGF(LogCaptureConvertVideoData, Display, "PathType for %ls is unspecified, setting to detected type %ls", *Video.Path, *PathTypeString);
	}
	else
	{
		FTakeMetadataPathUtils::ValidatePathType(Video.Path, Video.PathType.GetValue());
	}
	
	if (ShouldCopy())
	{
		return CopyData();
	}

	return ConvertData();
}

FCaptureConvertVideoData::FResult FCaptureConvertVideoData::ConvertData()
{
	using namespace UE::CaptureManager;

	FTaskProgress::FTask Task = Params.TaskProgress->StartTask();

	const FString VideoFilePath = FPaths::ConvertRelativePathToFull(Params.TakeOriginDirectory, Video.Path);
	FString DestinationDirectory = OutputDirectory / Video.Name;

	FCaptureManagerMediaRWModule& MediaRWModule =
		FModuleManager::LoadModuleChecked<FCaptureManagerMediaRWModule>("CaptureManagerMediaRW");

	TUniquePtr<IVideoReader> VideoReader = nullptr;
	if (Video.PathType.IsSet() && Video.PathType.GetValue() == FTakeMetadata::FVideo::EPathType::Folder)
	{
		static constexpr FStringView Format = TEXT("image_sequence");
		VideoReader = MediaRWModule.Get().CreateVideoReaderByFormat(FString(Format));

		if (!VideoReader)
		{
			FText Message = FText::Format(
				LOCTEXT("CaptureConvertVideo_ReaderIsntRegistered", "Video reader for image sequences isn't registered {0}"), FText::FromStringView(Format));
			return MakeError(MoveTemp(Message));
		}

		TOptional<FText> Error = VideoReader->Open(VideoFilePath);

		if (Error.IsSet())
		{
			return MakeError(MoveTemp(Error.GetValue()));
		}
	}
	else
	{
		TValueOrError<TUniquePtr<IVideoReader>, FText> VideoReaderResult = MediaRWModule.Get().CreateVideoReader(VideoFilePath);

		if (VideoReaderResult.HasError())
		{
			FText Message = FText::Format(LOCTEXT("CaptureConvertVideo_UnsupportedFile", "Video file format is unsupported {0}. Consider enabling Third Party Encoder in Capture Manager settings."), FText::FromString(VideoFilePath));
			return MakeError(MoveTemp(Message));
		}

		VideoReader = VideoReaderResult.StealValue();
	}
	
	check(VideoReader);

	ON_SCOPE_EXIT
	{
		VideoReader->Close();
	};

	TValueOrError<TUniquePtr<IImageWriter>, FText> ImageWriterResult = MediaRWModule.Get().CreateImageWriter(DestinationDirectory, VideoParams.ImageFileName, VideoParams.Format);

	if (ImageWriterResult.HasError())
	{
		FText Message = FText::Format(LOCTEXT("CaptureConvertVideo_UnsupportedFile", "Video file format is unsupported {0}. Consider enabling Third Party Encoder in Capture Manager settings."), FText::FromString(VideoFilePath));
		return MakeError(MoveTemp(Message));
	}

	TUniquePtr<IImageWriter> ImageWriter = ImageWriterResult.StealValue();

	ON_SCOPE_EXIT
	{
		ImageWriter->Close();
	};

	const double TotalDuration = VideoReader->GetDuration().GetTotalSeconds();

	FResult Result = MakeValue();

	while (true)
	{
		TValueOrError<TUniquePtr<UE::CaptureManager::FMediaTextureSample>, FText> VideoSampleResult = VideoReader->Next();

		if (VideoSampleResult.HasError())
		{
			return MakeError(VideoSampleResult.StealError());
		}

		TUniquePtr<UE::CaptureManager::FMediaTextureSample> VideoSample = VideoSampleResult.StealValue();

		// End of stream
		if (!VideoSample.IsValid())
		{
			break;
		}

		FWritingContext Context;
		Context.ReadSample = MoveTemp(VideoSample);
		Context.Writer = ImageWriter.Get();
		Context.TotalDuration = TotalDuration;
		Context.Task = &Task;

		FWriteResult WriteResult = OnWrite(MoveTemp(Context));

		if (WriteResult.HasError())
		{
			return MakeError(WriteResult.StealError());
		}
	}

	return Result;
}

bool FCaptureConvertVideoData::ShouldCopy() const
{
	FCaptureManagerMediaRWModule& MediaRWModule =
		FModuleManager::LoadModuleChecked<FCaptureManagerMediaRWModule>("CaptureManagerMediaRW");

	if (!Video.PathType.IsSet() || Video.PathType.GetValue() != FTakeMetadata::FVideo::EPathType::Folder)
	{
		return false;
	}

	TUniquePtr<IVideoReader> VideoReader = MediaRWModule.Get().CreateVideoReaderByFormat(TEXT("image_sequence"));

	if (!VideoReader)
	{
		return false;
	}

	const FString VideoFilePath = FPaths::ConvertRelativePathToFull(Params.TakeOriginDirectory, Video.Path);
	TOptional<FText> Error = VideoReader->Open(VideoFilePath);

	if (Error.IsSet())
	{
		return false;
	}

	TValueOrError<TUniquePtr<UE::CaptureManager::FMediaTextureSample>, FText> ReadResult = VideoReader->Next();
	if (!ReadResult.IsValid())
	{
		return false;
	}

	TUniquePtr<UE::CaptureManager::FMediaTextureSample> Sample = ReadResult.StealValue();

	if (!Sample)
	{
		return false;
	}

	IFileManager& FileManager = IFileManager::Get();
	IImageWrapperModule& ImageWrapperModule =
		FModuleManager::Get().LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
	FString Extension;
	FileManager.IterateDirectory(*VideoFilePath, [&Extension, &ImageWrapperModule](const TCHAR* InFilenameOrDirectory, bool bInIsDirectory)
	{
		if (bInIsDirectory)
		{
			return true;
		}

		Extension = FPaths::GetExtension(InFilenameOrDirectory, false);
		if (ImageWrapperModule.GetImageFormatFromExtension(*Extension) != EImageFormat::Invalid)
		{
			// Found the correct extension and return false to exit the iteration
			return false;
		}

		return true;
	});

	if (Sample->CurrentFormat == VideoParams.OutputPixelFormat &&
		VideoParams.Rotation == EMediaOrientation::Original &&
		VideoParams.Format == Extension)
	{
		return true;
	}

	return false;
}

FCaptureConvertVideoData::FResult FCaptureConvertVideoData::CopyData()
{
	using namespace UE::CaptureManager;

	const FString VideoFolderPath = FPaths::ConvertRelativePathToFull(Params.TakeOriginDirectory, Video.Path);

	IFileManager& FileManager = IFileManager::Get();

	TArray<FString> ImagesToCopy;
	FileManager.FindFiles(ImagesToCopy, *VideoFolderPath, Video.Format.IsEmpty() ? nullptr : *Video.Format);

	IImageWrapperModule& ImageWrapperModule = FModuleManager::Get().LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
	ImagesToCopy.RemoveAllSwap([&ImageWrapperModule](const FString& InFilePath)
	{
		return ImageWrapperModule.GetImageFormatFromExtension(*InFilePath) == EImageFormat::Invalid;
	});

	if (ImagesToCopy.IsEmpty())
	{
		FText ExtensionPostfix = Video.Format.IsEmpty() ? FText() : FText::Format(LOCTEXT("CaptureConvertVideo_Extension", " with specified extension .{0}"), FText::FromString(Video.Format));
		FText Message = FText::Format(LOCTEXT("CaptureConvertVideo_EmptyData", "Copy image data failed. No image data found at {0}{1}"), FText::FromString(VideoFolderPath), ExtensionPostfix);
		UE_LOGF(LogCaptureConvertVideoData, Error, "%ls", *Message.ToString());

		return MakeError(MoveTemp(Message));
	}

	using namespace UE::CaptureData;

	FRegexPattern Pattern = GetRegexPattern();
	ImagesToCopy.Sort([&Pattern](const FString& InLeft, const FString& InRight)
	{
		FString Dummy;
		FString LeftDigits, RightDigits;
		bool bSuccess = ExtractInfoFromFileName(Pattern, InLeft, Dummy, LeftDigits, Dummy);
		bSuccess = bSuccess && ExtractInfoFromFileName(Pattern, InRight, Dummy, RightDigits, Dummy);

		if (bSuccess)
		{
			return FCString::Atoi(*LeftDigits) < FCString::Atoi(*RightDigits);
		}

		return InLeft < InRight;
	});

	if (!Video.FramesCount.IsSet() || Video.FramesCount.GetValue() == 0)
	{
		Video.FramesCount = ImagesToCopy.Num();
	}

	FTaskProgress::FTask Task = Params.TaskProgress->StartTask();
	TSharedPtr<FTaskProgress> TaskProgress = MakeShared<FTaskProgress>(Video.FramesCount.GetValue(), FTaskProgress::FProgressReporter::CreateLambda([&Task](float InProgress)
	{
		Task.Update(InProgress);
	}));

	FString DestinationDirectory = OutputDirectory / Video.Name;

	for (const FString& FilePath : ImagesToCopy)
	{
		FTaskProgress::FTask InnerTask = TaskProgress->StartTask();

		FCopyProgressReporter ProgressReporter(InnerTask, Params.StopToken);

		FString Destination = DestinationDirectory / FilePath;
		FString Source = VideoFolderPath / FilePath;

		constexpr bool bReplace = true;
		constexpr bool bEvenIfReadOnly = true;
		constexpr bool bAttributes = false;
		uint32 CopyResult = FileManager.Copy(*Destination, *Source, bReplace, bEvenIfReadOnly, bAttributes, &ProgressReporter);

		if (CopyResult == COPY_Fail)
		{
			FText Message = FText::Format(LOCTEXT("CaptureConvertVideoData_CopyFailed", "Failed to copy file {0} from {1} to {2}"), 
										  FText::FromString(FilePath), FText::FromString(VideoFolderPath), FText::FromString(DestinationDirectory));
			return MakeError(MoveTemp(Message));
		}

		if (CopyResult == COPY_Canceled)
		{
			FText Message = LOCTEXT("CaptureConvertVideoData_AbortedByUser", "Image data copy aborted by user");
			return MakeError(MoveTemp(Message));
		}
	}

	return MakeValue();
}

FCaptureConvertVideoData::FWriteResult FCaptureConvertVideoData::OnWrite(FWritingContext InContext)
{
	IImageWriter* Writer = InContext.Writer;

	TUniquePtr<UE::CaptureManager::FMediaTextureSample>& VideoSample = InContext.ReadSample;

	EMediaOrientation Orientation = ConvertOrientation(Video.Orientation.Get(FTakeMetadata::FVideo::EOrientation::Original));
	VideoSample->Orientation = Orientation;
	VideoSample->Rotation = VideoParams.Rotation;
	VideoSample->DesiredFormat = VideoParams.OutputPixelFormat;

	TOptional<FText> WriterResult = Writer->Append(VideoSample.Get());

	if (WriterResult.IsSet())
	{
		return MakeError(MoveTemp(WriterResult.GetValue()));
	}

	const double Time = VideoSample->Time.GetTotalSeconds();
	if (InContext.TotalDuration > 0.0)
	{
		InContext.Task->Update(Time / InContext.TotalDuration);
	}

	if (Params.StopToken.IsStopRequested())
	{
		FText Message = LOCTEXT("CaptureConvertVideo_AbortedByUser", "Video conversion aborted by user");
		return MakeError(MoveTemp(Message));
	}

	return MakeValue();
}

EMediaOrientation FCaptureConvertVideoData::ConvertOrientation(FTakeMetadata::FVideo::EOrientation InOrientation) const
{
	switch (InOrientation)
	{
		case FTakeMetadata::FVideo::EOrientation::CW90:
			return EMediaOrientation::CW90;
		case FTakeMetadata::FVideo::EOrientation::CW180:
			return EMediaOrientation::CW180;
		case FTakeMetadata::FVideo::EOrientation::CW270:
			return EMediaOrientation::CW270;
		case FTakeMetadata::FVideo::EOrientation::Original:
		default:
			return EMediaOrientation::Original;
	}
}

#undef LOCTEXT_NAMESPACE