// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/ConvertVideoNode.h"

#include "CaptureDataConverterNodeParams.h"

#include "MediaSample.h"

#include "Async/QueueRunner.h"


class FCaptureConvertVideoData final : 
	public FConvertVideoNode
{
public:

	FCaptureConvertVideoData(const FTakeMetadata::FVideo& InVideo,
							 const FString& InOutputDirectory,
							 const FCaptureConvertDataNodeParams& InParams,
							 const FCaptureConvertVideoOutputParams& InVideoParams);

	virtual ~FCaptureConvertVideoData() override;

private:

	virtual FResult Run() override;

	FResult ConvertData();
	bool ShouldCopy() const;
	FResult CopyData();

	EMediaOrientation ConvertOrientation(FTakeMetadata::FVideo::EOrientation InOrientation) const;

	FCaptureConvertDataNodeParams Params;
	FCaptureConvertVideoOutputParams VideoParams;

	struct FWritingContext
	{
		FWritingContext() 
			: ReadSample(nullptr)
			, Writer(nullptr)
			, Task(nullptr)
			, TotalDuration(0.0f)
		{
		}

		TUniquePtr<UE::CaptureManager::FMediaTextureSample> ReadSample;
		class IImageWriter* Writer;
		UE::CaptureManager::FTaskProgress::FTask* Task;

		double TotalDuration;
	};

	using FWriteResult = TValueOrError<void, FText>;
	FWriteResult OnWrite(FWritingContext InContext);
};