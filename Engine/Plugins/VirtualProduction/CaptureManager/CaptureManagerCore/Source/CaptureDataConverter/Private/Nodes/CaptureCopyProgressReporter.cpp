// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureCopyProgressReporter.h"

FCopyProgressReporter::FCopyProgressReporter(UE::CaptureManager::FTaskProgress::FTask& InTask,
											 UE::CaptureManager::FStopToken InStopToken)
	: Task(InTask)
	, StopToken(MoveTemp(InStopToken))
{
}

bool FCopyProgressReporter::Poll(float InProgress)
{
	Task.Update(InProgress);

	return !StopToken.IsStopRequested();
}