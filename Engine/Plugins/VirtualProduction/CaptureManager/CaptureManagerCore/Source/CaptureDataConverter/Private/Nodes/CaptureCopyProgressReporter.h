// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/FileManager.h"

#include "Async/TaskProgress.h"
#include "Async/StopToken.h"

class FCopyProgressReporter final : public FCopyProgress
{
public:

	FCopyProgressReporter(UE::CaptureManager::FTaskProgress::FTask& InTask,
						  UE::CaptureManager::FStopToken InStopToken);

	virtual bool Poll(float InProgress) override;

private:

	UE::CaptureManager::FTaskProgress::FTask& Task;
	UE::CaptureManager::FStopToken StopToken;
};