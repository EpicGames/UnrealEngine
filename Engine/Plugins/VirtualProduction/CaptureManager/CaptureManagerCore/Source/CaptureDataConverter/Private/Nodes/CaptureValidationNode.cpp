// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureValidationNode.h"

#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "Misc/Paths.h"

#include "Modules/ModuleManager.h"
#include "CaptureManagerTakeMetadata.h"

#include "IngestCaptureData.h"

#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "CaptureValidationNode"

namespace UE::CaptureManager::Private
{

FIngestCaptureData::FVideo ConvertTakeMetadataVideoObject(const FString& InVideoDirectory, const FTakeMetadata::FVideo& InVideo)
{
	FIngestCaptureData::FVideo Video;

	Video.Name = InVideo.Name;
	Video.Path = InVideoDirectory;
	Video.FrameRate = InVideo.FrameRate;
	Video.FrameWidth = InVideo.FrameWidth;
	Video.FrameHeight = InVideo.FrameHeight;
	Video.TimecodeStart = InVideo.TimecodeStart;
	Video.DroppedFrames = InVideo.DroppedFrames.Get(TArray<uint32>());

	return Video;
}

FIngestCaptureData::FAudio ConvertTakeMetadataAudioObject(const FString& InAudioDirectory, const FTakeMetadata::FAudio& InAudio)
{
	FIngestCaptureData::FAudio Audio;

	Audio.Name = InAudio.Name;
	Audio.Path = InAudioDirectory;
	Audio.TimecodeStart = InAudio.TimecodeStart;
	Audio.TimecodeRate = InAudio.TimecodeRate;

	return Audio;
}

FIngestCaptureData::FCalibration ConvertTakeMetadataCalibrationObject(const FString& InCalibrationPath, const FTakeMetadata::FCalibration& InCalibration)
{
	FIngestCaptureData::FCalibration Calibration;

	Calibration.Name = InCalibration.Name;
	Calibration.Path = InCalibrationPath;
	
	return Calibration;
}

FString ConvertPathToDir(const FString& InDirectory)
{
	return InDirectory + TEXT("/");
}

}

const FString FCaptureValidationNode::TakeJsonFileName = TEXT("take");

FCaptureValidationNode::FCaptureValidationNode(const FCaptureDataConverterParams& InParams,
											   const FTakeMetadata& InTakeMetadata)
	: FCaptureManagerPipelineNode(TEXT("Validation"))
	, Params(InParams)
	, TakeMetadata(InTakeMetadata)
{
}

FCaptureValidationNode::FResult FCaptureValidationNode::Prepare()
{
	return MakeValue();
}

FCaptureValidationNode::FResult FCaptureValidationNode::Run()
{
	using namespace UE::CaptureManager;

	FString OutputDirectory = Private::ConvertPathToDir(Params.TakeOutputDirectory);

	FIngestCaptureData IngestData;

	for (const FTakeMetadata::FVideo& Video : TakeMetadata.Video)
	{
		static const FString VideoDirectory = TEXT("Video");

		FString OutputVideoDirectory = OutputDirectory / VideoDirectory / Video.Name;

		FCaptureValidationNode::FResult Result = CheckImages(OutputVideoDirectory, {});

		if (Result.HasError())
		{
			return Result;
		}

		
		FPaths::MakePathRelativeTo(OutputVideoDirectory, *OutputDirectory);

		IngestData.Video.Add(Private::ConvertTakeMetadataVideoObject(OutputVideoDirectory, Video));
	}

	for (const FTakeMetadata::FVideo& Depth : TakeMetadata.Depth)
	{
		static const FString DepthDirectory = TEXT("Depth");

		FString OutputDepthDirectory = OutputDirectory / DepthDirectory / Depth.Name;

		FCaptureValidationNode::FResult Result = CheckImages(OutputDepthDirectory, EImageFormat::EXR);

		if (Result.HasError())
		{
			return Result;
		}

		FPaths::MakePathRelativeTo(OutputDepthDirectory, *OutputDirectory);

		IngestData.Depth.Add(Private::ConvertTakeMetadataVideoObject(OutputDepthDirectory, Depth));
	}

	for (const FTakeMetadata::FAudio& Audio : TakeMetadata.Audio)
	{
		static const FString AudioDirectory = TEXT("Audio");

		FString OutputAudioDirectory = OutputDirectory / AudioDirectory / Audio.Name;

		FCaptureConvertAudioOutputParams AudioParams = Params.AudioOutputParams.GetValue();
		FCaptureValidationNode::FResult Result = CheckAudio(AudioParams.AudioFileName, OutputAudioDirectory);

		if (Result.HasError())
		{
			return Result;
		}

		OutputAudioDirectory = OutputAudioDirectory / FPaths::SetExtension(AudioParams.AudioFileName, AudioParams.Format);
		FPaths::MakePathRelativeTo(OutputAudioDirectory, *OutputDirectory);

		IngestData.Audio.Add(Private::ConvertTakeMetadataAudioObject(OutputAudioDirectory, Audio));
	}

	for (const FTakeMetadata::FCalibration& Calibration : TakeMetadata.Calibration)
	{
		static const FString CalibrationDirectory = TEXT("Calibration");

		FCaptureConvertCalibrationOutputParams CalibrationParams = Params.CalibrationOutputParams.GetValue();
		FString OutputCalibrationFile = OutputDirectory / CalibrationDirectory / Calibration.Name / CalibrationParams.FileName;
		OutputCalibrationFile = FPaths::SetExtension(OutputCalibrationFile, TEXT("json"));

		if (!IFileManager::Get().FileExists(*OutputCalibrationFile))
		{
			return MakeError(LOCTEXT("CaptureValidationNode_CalibrationMissing", "The calibration file is missing"));
		}

		FPaths::MakePathRelativeTo(OutputCalibrationFile, *OutputDirectory);

		IngestData.Calibration.Add(Private::ConvertTakeMetadataCalibrationObject(OutputCalibrationFile, Calibration));
	}

	IngestData.Version = 1;
	IngestData.DeviceModel = Params.TakeMetadata.Device.Model;
	IngestData.Slate = Params.TakeMetadata.Slate;
	IngestData.TakeNumber = Params.TakeMetadata.TakeNumber;

	TOptional<FText> Result = IngestCaptureData::Serialize(OutputDirectory, TakeJsonFileName, IngestData);

	if (Result.IsSet())
	{
		return MakeError(Result.GetValue());
	}

	return MakeValue();
}

FCaptureValidationNode::FResult FCaptureValidationNode::Validate()
{
	using namespace UE::CaptureManager;

	FString OutputDirectory = Private::ConvertPathToDir(Params.TakeOutputDirectory);
	const FString TakeJsonFile = OutputDirectory / FPaths::SetExtension(TakeJsonFileName, FIngestCaptureData::Extension);

	bool bTakeJsonExists = IFileManager::Get().FileExists(*TakeJsonFile);

	if (!bTakeJsonExists)
	{
		return MakeError(LOCTEXT("CaptureValidationNode_ValidateTakeJson", "The take.cparch file is missing from the output directory"));
	}

	return MakeValue();
}

FCaptureValidationNode::FResult FCaptureValidationNode::CheckImages(const FString& InImagesPath, TOptional<EImageFormat> InFormat)
{
	IFileManager& FileManager = IFileManager::Get();
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));

	if (!FPaths::DirectoryExists(InImagesPath))
	{
		FText Message = LOCTEXT("CaptureValidationNode_DirectoryMissing", "The output directory is missing");
		return MakeError(MoveTemp(Message));
	}

	bool bDirectoryIsEmpty = true;
	bool bFilesAreValid = FileManager.IterateDirectory(*InImagesPath, [&ImageWrapperModule, &bDirectoryIsEmpty, &InFormat](const TCHAR* InFileName, bool bIsDirectory)
	{
		bDirectoryIsEmpty = false;

		if (bIsDirectory)
		{
			return false;
		}

		EImageFormat Format = ImageWrapperModule.GetImageFormatFromExtension(*FPaths::GetExtension(InFileName));

		if (InFormat.IsSet())
		{
			return Format == InFormat.GetValue();
		}
		
		return Format != EImageFormat::Invalid;
	});

	if (bDirectoryIsEmpty)
	{
		FText Message = FText::Format(LOCTEXT("CaptureValidationNode_EmptyDirectory", "Folder is empty: {0}"), FText::FromString(InImagesPath));
		return MakeError(MoveTemp(Message));
	}

	if (!bFilesAreValid)
	{
		FText Message = LOCTEXT("CaptureValidationNode_InvalidFormat", "The images are in an unsupported format");
		return MakeError(MoveTemp(Message));
	}

	return MakeValue();
}

FCaptureValidationNode::FResult FCaptureValidationNode::CheckAudio(const FString& InExpectedFileName, const FString& InOutputDirectory)
{
	static const TArray<FString> SupportedFormats =
	{
		TEXT("wav")
	};

	IFileManager& FileManager = IFileManager::Get();

	if (!FPaths::DirectoryExists(InOutputDirectory))
	{
		FText Message = LOCTEXT("ConvertAudioNode_Validate_DirectoryMissing", "The output directory is missing");
		return MakeError(MoveTemp(Message));
	}

	FText ResultMessage;
	bool bDirectoryIsEmpty = true;
	bool bFilesAreValid = FileManager.IterateDirectory(*InOutputDirectory, [&ResultMessage, &bDirectoryIsEmpty, &InExpectedFileName](const TCHAR* InFileName, bool bIsDirectory)
	{
		bDirectoryIsEmpty = false;

		if (bIsDirectory)
		{
			ResultMessage = FText::Format(
				LOCTEXT("ConvertAudioNode_Validate_UnexpectedDirectory", "Unexpected directory found: {0}"), FText::FromString(FPaths::GetPathLeaf(InFileName)));

			return false;
		}

		FString FileName = FPaths::GetBaseFilename(InFileName);

		if (FileName != InExpectedFileName)
		{
			ResultMessage = FText::Format(
				LOCTEXT("ConvertAudioNode_Validate_InvalidFileName", "Invalid audio file name: {0}, expected {1}"),
				FText::FromString(FileName),
				FText::FromString(InExpectedFileName));

			return false;
		}

		FString Extension = FPaths::GetExtension(InFileName);

		if (!SupportedFormats.Contains(Extension))
		{
			FString SupportedFormatsString = FString::Join(SupportedFormats, TEXT(", "));

			ResultMessage = FText::Format(
				LOCTEXT("ConvertAudioNode_Validate_InvalidFormat", "Unsupported audio file format: {0}, supported formats: {1}"),
				FText::FromString(Extension),
				FText::FromString(SupportedFormatsString));

			return false;
		}

		return true;
	});

	if (bDirectoryIsEmpty)
	{
		FText Message = FText::Format(LOCTEXT("ConvertVideoNode_Validate_EmptyDirectory", "Folder is empty: {0}"), FText::FromString(InOutputDirectory));
		return MakeError(MoveTemp(Message));
	}

	if (!bFilesAreValid)
	{
		return MakeError(MoveTemp(ResultMessage));
	}

	return MakeValue();
}

#undef LOCTEXT_NAMESPACE