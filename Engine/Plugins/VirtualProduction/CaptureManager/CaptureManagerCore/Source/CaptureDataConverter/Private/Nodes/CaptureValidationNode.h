// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerPipelineNode.h"

#include "CaptureDataConverter.h"
#include "CaptureDataConverterNodeParams.h"

#include "IImageWrapperModule.h"
#include "CaptureManagerTakeMetadata.h"

class FCaptureValidationNode final : 
	public FCaptureManagerPipelineNode
{
public:

	FCaptureValidationNode(const FCaptureDataConverterParams& InParams,
						   const FTakeMetadata& InTakeMetadata);

private:

	static const FString TakeJsonFileName;

	virtual FResult Prepare() override;
	virtual FResult Run() override;
	virtual FResult Validate() override;

	static FResult CheckImages(const FString& InImagesPath, TOptional<EImageFormat> InFormat);
	static FResult CheckAudio(const FString& InExpectedFileName, const FString& InAudio);

	FCaptureDataConverterParams Params;
	FTakeMetadata TakeMetadata;
};