// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureConvertAudioDataThirdParty.h"

#include "MediaSample.h"

#include "Async/HelperFunctions.h"
#include "Async/StopToken.h"
#include "Async/TaskProgress.h"

#include "Nodes/CaptureCopyProgressReporter.h"
#include "CaptureThirdPartyNodeUtils.h"

#include "CaptureManagerMediaRWModule.h"

#include "Engine/Engine.h"
#include "GlobalNamingTokens.h"
#include "NamingTokenData.h"
#include "NamingTokensEngineSubsystem.h"
#include "ProcessRunner/ProcessRunner.h"

#define LOCTEXT_NAMESPACE "CaptureConvertAudioDataTP"

DEFINE_LOG_CATEGORY_STATIC(LogCaptureConvertAudioDataThirdParty, Log, All);

FCaptureConvertAudioDataThirdParty::FCaptureConvertAudioDataThirdParty(UE::CaptureManager::FAudioEncoderCommand InThirdPartyCommand,
																	   const FTakeMetadata::FAudio& InAudio,
																	   const FString& InOutputDirectory,
																	   const FCaptureConvertDataNodeParams& InParams,
																	   const FCaptureConvertAudioOutputParams& InAudioParams

	)
	: FConvertAudioNode(InAudio, InOutputDirectory)
	, ThirdPartyCommand(MoveTemp(InThirdPartyCommand))
	, Params(InParams)
	, AudioParams(InAudioParams)
{
}

FCaptureConvertAudioDataThirdParty::FResult FCaptureConvertAudioDataThirdParty::Run()
{
	using namespace UE::CaptureManager;

	if (Params.StopToken.IsStopRequested())
	{
		FText Message = LOCTEXT("CaptureConvertAudioNodeTP_AbortedByUser", "Aborted by user");
		return MakeError(MoveTemp(Message));
	}

	if (FPaths::GetExtension(Audio.Path) == AudioParams.Format)
	{
		return CopyAudioFile();
	}

	return ConvertAudioFile();
}

FCaptureConvertAudioDataThirdParty::FResult FCaptureConvertAudioDataThirdParty::CopyAudioFile()
{
	using namespace UE::CaptureManager;

	FTaskProgress::FTask Task = Params.TaskProgress->StartTask();

	FString DestinationDirectory = OutputDirectory / Audio.Name;
	const FString AudioFilePath = FPaths::ConvertRelativePathToFull(Params.TakeOriginDirectory, Audio.Path);

	FCopyProgressReporter ProgressReporter(Task, Params.StopToken);
	FString Destination = DestinationDirectory / FPaths::SetExtension(AudioParams.AudioFileName, AudioParams.Format);

	constexpr bool bReplace = true;
	constexpr bool bEvenIfReadOnly = true;
	constexpr bool bAttributes = false;
	uint32 Result = IFileManager::Get().Copy(*Destination, *AudioFilePath, bReplace, bEvenIfReadOnly, bAttributes, &ProgressReporter);

	if (Result == COPY_Fail)
	{
		FText Message = FText::Format(LOCTEXT("CaptureConvertAudioNodeTP_CopyFailed", "Failed to copy the audio file {0}"), 
									  FText::FromString(AudioFilePath));
		return MakeError(MoveTemp(Message));
	}

	if (Result == COPY_Canceled)
	{
		FText Message = LOCTEXT("CaptureConvertAudioNodeTP_AbortedByUser", "Aborted by user");
		return MakeError(MoveTemp(Message));
	}

	return MakeValue();
}

FCaptureConvertAudioDataThirdParty::FResult FCaptureConvertAudioDataThirdParty::ConvertAudioFile()
{
	using namespace UE::CaptureManager;

	FTaskProgress::FTask Task = Params.TaskProgress->StartTask();

	FString DestinationDirectory = OutputDirectory / Audio.Name;
	const FString AudioFilePath = FPaths::ConvertRelativePathToFull(Params.TakeOriginDirectory, Audio.Path);

	const FString AudioOutputFile = FPaths::SetExtension(FPaths::Combine(DestinationDirectory, AudioParams.AudioFileName), AudioParams.Format);
	FProcessRunnerResult RunnerResult = ThirdPartyCommand.Execute(AudioFilePath, AudioOutputFile, &Params.StopToken);

	if (RunnerResult.HasError())
	{
		EProcessRunnerError RunnerError = RunnerResult.StealError();

		FText Message;
		switch (RunnerError)
		{
			case EProcessRunnerError::LaunchFailed:
				Message = LOCTEXT("CaptureConvertAudioNodeTP_ProcessNotFound", "Failed to start the process");
				break;
			case EProcessRunnerError::Cancelled:
				Message = LOCTEXT("CaptureConvertAudioNodeTP_AbortedByUser", "Aborted by user");
				break;
			default:
				Message = LOCTEXT("CaptureConvertAudioNodeTP_ErrorRunning", "Error occurred while running the third party encoder");
				break;
		}
		return MakeError(MoveTemp(Message));
	}

	Task.Update(1.0f);
	return MakeValue();
}

#undef LOCTEXT_NAMESPACE