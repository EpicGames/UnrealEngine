// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/ConvertAudioNode.h"

#include "CaptureManagerEncoderCommands.h"

#include "CaptureDataConverterNodeParams.h"

class FCaptureConvertAudioDataThirdParty final :
	public FConvertAudioNode
{
public:

	FCaptureConvertAudioDataThirdParty(UE::CaptureManager::FAudioEncoderCommand InThirdPartyCommand,
									   const FTakeMetadata::FAudio& InAudio,
									   const FString& InOutputDirectory,
									   const FCaptureConvertDataNodeParams& InParams,
									   const FCaptureConvertAudioOutputParams& InAudioParams
	);

private:

	virtual FResult Run() override;

	FResult CopyAudioFile();
	FResult ConvertAudioFile();

	UE::CaptureManager::FAudioEncoderCommand ThirdPartyCommand;
	FCaptureConvertDataNodeParams Params;
	FCaptureConvertAudioOutputParams AudioParams;
};