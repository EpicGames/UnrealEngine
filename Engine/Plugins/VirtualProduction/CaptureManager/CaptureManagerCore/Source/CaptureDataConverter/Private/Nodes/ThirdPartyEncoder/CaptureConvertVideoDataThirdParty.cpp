// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureConvertVideoDataThirdParty.h"

#include "MediaSample.h"

#include "Async/HelperFunctions.h"
#include "Async/StopToken.h"
#include "Async/TaskProgress.h"

#include "TakeDirectoryUtils.h"

#include "CaptureManagerMediaRWModule.h"
#include "Nodes/CaptureCopyProgressReporter.h"
#include "CaptureThirdPartyNodeUtils.h"

#include "Engine/Engine.h"
#include "GlobalNamingTokens.h"
#include "NamingTokenData.h"
#include "NamingTokensEngineSubsystem.h"
#include "IImageWrapperModule.h"

#include "Internationalization/Regex.h"

#include "ProcessRunner/ProcessRunner.h"

#define LOCTEXT_NAMESPACE "CaptureConvertVideoDataTP"

DEFINE_LOG_CATEGORY_STATIC(LogCaptureConvertVideoDataThirdParty, Log, All);

namespace UE::CaptureManager::Private
{

static void ConvertOrientation(EMediaOrientation InOrientation, TArray<FString>& OutFilterArray)
{
	if (InOrientation == EMediaOrientation::CW90)
	{
		OutFilterArray.Add(TEXT("transpose=clock"));
	}
	else if (InOrientation == EMediaOrientation::CW180)
	{
		OutFilterArray.Add(TEXT("transpose=clock,transpose=clock"));
	}
	else if (InOrientation == EMediaOrientation::CW270)
	{
		OutFilterArray.Add(TEXT("transpose=cclock"));
	}
}

static void ConvertPixelFormat(EMediaTexturePixelFormat InPixelFormat, TArray<FString>& OutFilterArray)
{
	if (InPixelFormat == EMediaTexturePixelFormat::U8_RGB)
	{
		OutFilterArray.Add(TEXT("format=rgb0"));
	}
	else if (InPixelFormat == EMediaTexturePixelFormat::U8_BGR)
	{
		OutFilterArray.Add(TEXT("format=bgr0"));
	}
	else if (InPixelFormat == EMediaTexturePixelFormat::U8_RGBA)
	{
		OutFilterArray.Add(TEXT("format=rgba"));
	}
	else if (InPixelFormat == EMediaTexturePixelFormat::U8_BGRA)
	{
		OutFilterArray.Add(TEXT("format=bgra"));
	}
	else if (InPixelFormat == EMediaTexturePixelFormat::U8_I444)
	{
		OutFilterArray.Add(TEXT("format=yuvj444p"));
	}
	else if (InPixelFormat == EMediaTexturePixelFormat::U8_I420)
	{
		OutFilterArray.Add(TEXT("format=yuvj420p"));
	}
	else if (InPixelFormat == EMediaTexturePixelFormat::U8_YUY2)
	{
		OutFilterArray.Add(TEXT("format=yuyv422"));
	}
	else if (InPixelFormat == EMediaTexturePixelFormat::U8_NV12)
	{
		OutFilterArray.Add(TEXT("format=nv12"));
	}
	else if (InPixelFormat == EMediaTexturePixelFormat::U8_Mono)
	{
		OutFilterArray.Add(TEXT("format=gray"));
	}
	else if (InPixelFormat == EMediaTexturePixelFormat::U16_Mono)
	{
		OutFilterArray.Add(TEXT("format=gray16be"));
	}
	else if (InPixelFormat == EMediaTexturePixelFormat::F_Mono)
	{
		OutFilterArray.Add(TEXT("format=gray32fbe"));
	}
}

static FString CreateConversionArguments(EMediaTexturePixelFormat InPixelFormat, EMediaOrientation InOrientation)
{
	TArray<FString> FilterArray;

	ConvertPixelFormat(InPixelFormat, FilterArray);
	ConvertOrientation(InOrientation, FilterArray);

	if (FilterArray.IsEmpty())
	{
		return FString();
	}

	FString Filters = FString::Join(FilterArray, TEXT(","));

	return FString::Format(TEXT("-vf \"{0}\""), { Filters });
}

}

FCaptureConvertVideoDataThirdParty::FCaptureConvertVideoDataThirdParty(UE::CaptureManager::FVideoEncoderCommand InThirdPartyCommand,
																	   const FTakeMetadata::FVideo& InVideo,
																	   const FString& InOutputDirectory, 
																	   const FCaptureConvertDataNodeParams& InParams,
																	   const FCaptureConvertVideoOutputParams& InVideoParams)
	: FConvertVideoNode(InVideo, InOutputDirectory)
	, ThirdPartyCommand(MoveTemp(InThirdPartyCommand))
	, Params(InParams)
	, VideoParams(InVideoParams)
{
	checkf(!VideoParams.Format.IsEmpty(), TEXT("Video output format MUST be specified"));
}

FCaptureConvertVideoDataThirdParty::FResult FCaptureConvertVideoDataThirdParty::Run()
{
	using namespace UE::CaptureManager;

	if (Params.StopToken.IsStopRequested())
	{
		FText Message = LOCTEXT("CaptureConvertVideoNodeTP_AbortedByUser", "Aborted by user");
		return MakeError(MoveTemp(Message));
	}

	if (!Video.PathType.IsSet())
	{
		Video.PathType = FTakeMetadataPathUtils::DetectPathType(Video.Path);
		FString PathTypeString = FTakeMetadataPathUtils::PathTypeToString(Video.PathType.GetValue());
		UE_LOGF(LogCaptureConvertVideoDataThirdParty, Display, "PathType for %ls is unspecified, setting to detected type %ls", *Video.Path, *PathTypeString);
	}
	else
	{
		FTakeMetadataPathUtils::ValidatePathType(Video.Path, Video.PathType.GetValue());
	}

	if (ShouldCopy())
	{
		return CopyData();
	}

	return ConvertData();
}

FCaptureConvertVideoDataThirdParty::FResult FCaptureConvertVideoDataThirdParty::ConvertData()
{
	using namespace UE::CaptureManager;

	FTaskProgress::FTask Task = Params.TaskProgress->StartTask();

	FString VideoFilePath = FPaths::ConvertRelativePathToFull(Params.TakeOriginDirectory, Video.Path);
	FString DestinationDirectory = OutputDirectory / Video.Name;
	
	if (!FPaths::DirectoryExists(DestinationDirectory))
	{
		bool bSuccess = IFileManager::Get().MakeDirectory(*DestinationDirectory, true);
		if (!bSuccess)
		{
			FText Message = FText::Format(
				LOCTEXT("CaptureConvertVideoNodeTP_FailedToCreateDirectory", "Failed to create encoder output directory: {0}"),
				FText::FromString(DestinationDirectory)
			);

			UE_LOGF(LogCaptureConvertVideoDataThirdParty, Error, "%ls", *Message.ToString());
			return MakeError(MoveTemp(Message));
		}
	}

	if (Video.PathType.IsSet() && Video.PathType.GetValue() == FTakeMetadata::FVideo::EPathType::Folder)
	{
		using namespace UE::CaptureData;

		FString FilePattern = GetFileNameFormat(VideoFilePath);
		if (FilePattern.IsEmpty())
		{
			FText Message = FText::Format(
				LOCTEXT("CaptureConvertVideoNodeTP_FailedToDetectFilePattern", "Failed to detect the file pattern for directory: {0}"),
				FText::FromString(VideoFilePath)
			);

			UE_LOGF(LogCaptureConvertVideoDataThirdParty, Error, "%ls", *Message.ToString());
			
			return MakeError(MoveTemp(Message));
		}

		VideoFilePath = FPaths::Combine(VideoFilePath, FilePattern);
	}

	const FString ImageFileName = FString::Format(TEXT("{0}_%06d.{1}"), { VideoParams.ImageFileName, VideoParams.Format });
	const FString ImageFilePath = FPaths::Combine(DestinationDirectory, ImageFileName);

	const FString ConversionArguments = Private::CreateConversionArguments(VideoParams.OutputPixelFormat, VideoParams.Rotation);
	FProcessRunnerResult RunnerResult = ThirdPartyCommand.Execute(VideoFilePath, ImageFilePath, ConversionArguments, &Params.StopToken);

	if (RunnerResult.HasError())
	{
		EProcessRunnerError RunnerError = RunnerResult.StealError();

		FText Message;
		switch (RunnerError)
		{
			case EProcessRunnerError::LaunchFailed:
				Message = LOCTEXT("CaptureConvertVideoNodeTP_ProcessNotFound", "Failed to start the process");
				break;
			case EProcessRunnerError::Cancelled:
				Message = LOCTEXT("CaptureConvertVideoNodeTP_AbortedByUser", "Aborted by user");
				break;
			default:
				Message = LOCTEXT("CaptureConvertVideoNodeTP_ErrorRunning", "Error occurred while running the third party encoder.");
				break;
		}
		return MakeError(MoveTemp(Message));
	}

	Task.Update(1.0f);
	return MakeValue();
}

bool FCaptureConvertVideoDataThirdParty::ShouldCopy() const
{
	FCaptureManagerMediaRWModule& MediaRWModule =
		FModuleManager::LoadModuleChecked<FCaptureManagerMediaRWModule>("CaptureManagerMediaRW");

	if (!Video.PathType.IsSet() || Video.PathType.GetValue() != FTakeMetadata::FVideo::EPathType::Folder)
	{
		return false;
	}

	TUniquePtr<IVideoReader> VideoReader = MediaRWModule.Get().CreateVideoReaderByFormat(TEXT("image_sequence"));

	if (!VideoReader)
	{
		return false;
	}

	const FString VideoFilePath = FPaths::ConvertRelativePathToFull(Params.TakeOriginDirectory, Video.Path);
	TOptional<FText> Error = VideoReader->Open(VideoFilePath);

	if (Error.IsSet())
	{
		return false;
	}

	TValueOrError<TUniquePtr<UE::CaptureManager::FMediaTextureSample>, FText> ReadResult = VideoReader->Next();
	if (!ReadResult.IsValid())
	{
		return false;
	}

	TUniquePtr<UE::CaptureManager::FMediaTextureSample> Sample = ReadResult.StealValue();

	if (!Sample)
	{
		return false;
	}

	IFileManager& FileManager = IFileManager::Get();
	IImageWrapperModule& ImageWrapperModule =
		FModuleManager::Get().LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
	FString Extension;
	FileManager.IterateDirectory(*VideoFilePath, [&Extension, &ImageWrapperModule](const TCHAR* InFilenameOrDirectory, bool bInIsDirectory)
	{
		if (bInIsDirectory)
		{
			return true;
		}

		Extension = FPaths::GetExtension(InFilenameOrDirectory, false);
		if (ImageWrapperModule.GetImageFormatFromExtension(*Extension) != EImageFormat::Invalid)
		{
			// Found the correct extension and return false to exit the iteration
			return false;
		}

		return true;
	});

	if (Sample->CurrentFormat == VideoParams.OutputPixelFormat &&
		VideoParams.Rotation == EMediaOrientation::Original &&
		VideoParams.Format == Extension)
	{
		return true;
	}

	return false;
}

FCaptureConvertVideoDataThirdParty::FResult FCaptureConvertVideoDataThirdParty::CopyData()
{
	using namespace UE::CaptureManager;

	const FString VideoFolderPath = FPaths::ConvertRelativePathToFull(Params.TakeOriginDirectory, Video.Path);

	IFileManager& FileManager = IFileManager::Get();

	TArray<FString> ImagesToCopy;
	FileManager.FindFiles(ImagesToCopy, *VideoFolderPath, Video.Format.IsEmpty() ? nullptr : *Video.Format);

	IImageWrapperModule& ImageWrapperModule = FModuleManager::Get().LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
	ImagesToCopy.RemoveAllSwap([&ImageWrapperModule](const FString& InFilePath)
	{
		return ImageWrapperModule.GetImageFormatFromExtension(*InFilePath) == EImageFormat::Invalid;
	});

	if (ImagesToCopy.IsEmpty())
	{
		FText ExtensionPostfix = Video.Format.IsEmpty() ? FText() : FText::Format(LOCTEXT("CaptureConvertVideoTP_Extension", " with specified extension .{0}"), FText::FromString(Video.Format));
		FText Message = FText::Format(LOCTEXT("CaptureConvertVideoNodeTP_EmptyData", "Copy image data failed. No image data found at {0}{1}"), FText::FromString(VideoFolderPath), ExtensionPostfix);
		UE_LOGF(LogCaptureConvertVideoDataThirdParty, Error, "%ls", *Message.ToString());

		return MakeError(MoveTemp(Message));
	}

	using namespace UE::CaptureData;

	FRegexPattern Pattern = GetRegexPattern();
	ImagesToCopy.Sort([&Pattern](const FString& InLeft, const FString& InRight)
	{
		FString Dummy;
		FString LeftDigits, RightDigits;
		bool bSuccess = ExtractInfoFromFileName(Pattern, InLeft, Dummy, LeftDigits, Dummy);
		bSuccess = bSuccess && ExtractInfoFromFileName(Pattern, InRight, Dummy, RightDigits, Dummy);

		if (bSuccess)
		{
			return FCString::Atoi(*LeftDigits) < FCString::Atoi(*RightDigits);
		}

		return InLeft < InRight;
	});

	if (!Video.FramesCount.IsSet() || Video.FramesCount.GetValue() == 0)
	{
		Video.FramesCount = ImagesToCopy.Num();
	}

	FTaskProgress::FTask Task = Params.TaskProgress->StartTask();
	TSharedPtr<FTaskProgress> TaskProgress = MakeShared<FTaskProgress>(Video.FramesCount.GetValue(), FTaskProgress::FProgressReporter::CreateLambda([&Task](float InProgress)
	{
		Task.Update(InProgress);
	}));

	FString DestinationDirectory = OutputDirectory / Video.Name;

	for (const FString& FilePath : ImagesToCopy)
	{
		FTaskProgress::FTask InnerTask = TaskProgress->StartTask();

		FCopyProgressReporter ProgressReporter(InnerTask, Params.StopToken);

		FString Destination = DestinationDirectory / FilePath;
		FString Source = VideoFolderPath / FilePath;

		constexpr bool bReplace = true;
		constexpr bool bEvenIfReadOnly = true;
		constexpr bool bAttributes = false;
		uint32 CopyResult = FileManager.Copy(*Destination, *Source, bReplace, bEvenIfReadOnly, bAttributes, &ProgressReporter);

		if (CopyResult == COPY_Fail)
		{
			FText Message = FText::Format(LOCTEXT("CaptureConvertVideoNodeTP_CopyFailed", "Failed to copy file {0} from {1} to {2}"),
										  FText::FromString(FilePath), FText::FromString(VideoFolderPath), FText::FromString(DestinationDirectory));
			return MakeError(MoveTemp(Message));
		}

		if (CopyResult == COPY_Canceled)
		{
			FText Message = LOCTEXT("CaptureConvertVideoNodeTP_CopyAbortedByUser", "Image data copy aborted by user");
			return MakeError(MoveTemp(Message));
		}
	}

	return MakeValue();
}

#undef LOCTEXT_NAMESPACE
