// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/ConvertVideoNode.h"
#include "CaptureManagerEncoderCommands.h"

#include "CaptureDataConverterNodeParams.h"

class FCaptureConvertVideoDataThirdParty final :
	public FConvertVideoNode
{
public:

	FCaptureConvertVideoDataThirdParty(UE::CaptureManager::FVideoEncoderCommand InThirdPartyCommand,
									   const FTakeMetadata::FVideo& InVideo,
									   const FString& InOutputDirectory,
									   const FCaptureConvertDataNodeParams& InParams,
									   const FCaptureConvertVideoOutputParams& InVideoParams);

private:

	virtual FResult Run() override;

	FResult ConvertData();
	bool ShouldCopy() const;
	FResult CopyData();

	UE::CaptureManager::FVideoEncoderCommand ThirdPartyCommand;
	FCaptureConvertDataNodeParams Params;
	FCaptureConvertVideoOutputParams VideoParams;
};
