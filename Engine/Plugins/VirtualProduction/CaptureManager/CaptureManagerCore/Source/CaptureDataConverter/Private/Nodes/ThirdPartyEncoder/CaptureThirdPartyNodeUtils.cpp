// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureThirdPartyNodeUtils.h"

#include "HAL/PlatformProcess.h"

namespace UE::CaptureManager
{

FString WrapInQuotes(const FString& InString)
{
	if (InString.StartsWith(TEXT("\"")) && InString.EndsWith(TEXT("\"")))
	{
		return InString;
	}

	return FString::Format(TEXT("\"{0}\""), { InString });
}

TArray<uint8> ReadPipe(void* InReadPipe)
{
	TArray<uint8> CommandOutput;
	bool Read = FPlatformProcess::ReadPipeToArray(InReadPipe, CommandOutput);
	if (!Read)
	{
		CommandOutput.Empty();
	}

	return CommandOutput;
}

}
