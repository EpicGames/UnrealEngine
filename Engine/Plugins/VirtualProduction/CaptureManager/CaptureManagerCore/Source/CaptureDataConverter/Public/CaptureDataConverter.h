// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerPipeline.h"
#include "CaptureDataConverterError.h"

#include "CaptureManagerTakeMetadata.h"

#include "CaptureDataConverterNodeParams.h"

#include "Nodes/CaptureConvertCustomData.h"

#include "CaptureManagerEncoderConfig.h"

#define UE_API CAPTUREDATACONVERTER_API

struct FCaptureDataConverterParams
{
	FTakeMetadata TakeMetadata;

	FString TakeName;
	FString TakeOriginDirectory;
	FString TakeOutputDirectory;

	TOptional<FCaptureConvertVideoOutputParams> VideoOutputParams;
	TOptional<FCaptureConvertAudioOutputParams> AudioOutputParams;
	TOptional<FCaptureConvertDepthOutputParams> DepthOutputParams;
	TOptional<FCaptureConvertCalibrationOutputParams> CalibrationOutputParams;

	TOptional<FCaptureManagerEncoderConfig> AudioEncoderConfig;
	TOptional<FCaptureManagerEncoderConfig> VideoEncoderConfig;

	/** When set, conversion nodes check this token for cancellation instead of the converter's
	 *  internal stop requester. Cancel() on the converter only signals the internal requester,
	 *  so cooperative node cancellation must be driven by the token's owner. */
	UE_INTERNAL TOptional<UE::CaptureManager::FStopToken> ExternalStopToken;
};

template<typename T>
using FCaptureDataConverterResult = TValueOrError<T, FCaptureDataConverterError>;

class FCaptureDataConverter
{
public:

	DECLARE_DELEGATE_OneParam(FProgressReporter, double InProgress);

	UE_API FCaptureDataConverter();
	UE_API ~FCaptureDataConverter();

	UE_API void AddCustomNode(TSharedPtr<FCaptureConvertCustomData> InCustomNode);
	UE_API void AddSyncNode(TSharedPtr<FCaptureConvertCustomData> InCustomNode);

	[[nodiscard]] UE_API FCaptureDataConverterResult<void> Run(FCaptureDataConverterParams InParams, FProgressReporter InProgressReporter);
	UE_API void Cancel();

private:

	TArray<TSharedPtr<FCaptureConvertCustomData>> CustomNodes;
	TArray<TSharedPtr<FCaptureConvertCustomData>> SyncNodes;

	TSharedPtr<FCaptureManagerPipeline> Pipeline;
	UE::CaptureManager::FStopRequester StopRequester;
};

#undef UE_API
