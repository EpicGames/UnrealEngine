// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"

#define UE_API CAPTUREDATACONVERTER_API

class FCaptureDataConverterError
{
public:

	UE_API explicit FCaptureDataConverterError(TArray<FText> InErrors);
	UE_API ~FCaptureDataConverterError();

	UE_API TArray<FText> GetErrors() const;

private:

	TArray<FText> Errors;
};

#undef UE_API
