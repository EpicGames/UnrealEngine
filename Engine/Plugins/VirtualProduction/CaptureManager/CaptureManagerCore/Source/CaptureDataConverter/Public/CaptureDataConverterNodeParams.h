// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskProgress.h"
#include "Async/StopToken.h"

#include "MediaSample.h"

struct FCaptureConvertDataNodeParams
{
	FString TakeOriginDirectory;
	TSharedPtr<UE::CaptureManager::FTaskProgress> TaskProgress;
	UE::CaptureManager::FStopToken StopToken;
};

struct FCaptureConvertVideoOutputParams
{
	FString ImageFileName = TEXT("frame");
	FString Format;
	UE::CaptureManager::EMediaTexturePixelFormat OutputPixelFormat = UE::CaptureManager::EMediaTexturePixelFormat::Undefined;
	EMediaOrientation Rotation = EMediaOrientation::Original;
};

struct FCaptureConvertAudioOutputParams
{
	FString AudioFileName = TEXT("audio");
	FString Format = TEXT("wav");
};

struct FCaptureConvertDepthOutputParams
{
	FString ImageFileName = TEXT("depth");
	bool bShouldCompressFiles = true;
	EMediaOrientation Rotation = EMediaOrientation::Original;
};

struct FCaptureConvertCalibrationOutputParams
{
	FString FileName = TEXT("calibration");
};
