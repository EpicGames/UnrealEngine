// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"

namespace UE::CaptureManager
{

// Token namespace/key constants used in the two-pass encoder command resolution scheme.
// CaptureManagerEncoderNamingTokens.h/cpp registers UNamingTokens subclasses that are
// auto-discovered by UNamingTokensEngineSubsystem in both LiveLinkHub and UE Editor contexts.
namespace VideoEncoderTokens
{
static constexpr FStringView Namespace = TEXTVIEW("cmvidenc");
static constexpr FStringView InputKey = TEXTVIEW("input");
static constexpr FStringView OutputKey = TEXTVIEW("output");
static constexpr FStringView ParamsKey = TEXTVIEW("params");
}

namespace AudioEncoderTokens
{
static constexpr FStringView Namespace = TEXTVIEW("cmaudenc");
static constexpr FStringView InputKey = TEXTVIEW("input");
static constexpr FStringView OutputKey = TEXTVIEW("output");
}

namespace EncoderDefaults
{
static constexpr FStringView AudioCommandArgs = TEXTVIEW("-i {input} -vn {output}");
static constexpr FStringView VideoCommandArgs = TEXTVIEW("-noautorotate -i {input} -an {params} -q:v 1 -qmax 1 -qmin 1 -start_number 0 {output}");
}

}

// Encoder path and Pass-1-resolved encoder args supplied by the caller.
// CaptureDataConverter constructs the private command objects from this internally.
struct FCaptureManagerEncoderConfig
{
	FString EncoderPath;
	FString EncoderArgs;
};
