// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NamingTokens.h"

#include "CaptureManagerEncoderNamingTokens.generated.h"

UCLASS(MinimalAPI, NotBlueprintable)
class UCaptureManagerVideoEncoderTokens final : public UNamingTokens
{
	GENERATED_BODY()

public:
	UCaptureManagerVideoEncoderTokens();

protected:
	// ~Begin UNamingTokens
	virtual void OnCreateDefaultTokens(TArray<FNamingTokenData>& OutTokens) override;
	virtual void OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData) override;
	virtual void OnPostEvaluate_Implementation() override;
	// ~End UNamingTokens
};


UCLASS(MinimalAPI, NotBlueprintable)
class UCaptureManagerAudioEncoderTokens final : public UNamingTokens
{
	GENERATED_BODY()

public:
	UCaptureManagerAudioEncoderTokens();

protected:
	// ~Begin UNamingTokens
	virtual void OnCreateDefaultTokens(TArray<FNamingTokenData>& OutTokens) override;
	virtual void OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData) override;
	virtual void OnPostEvaluate_Implementation() override;
	// ~End UNamingTokens
};
