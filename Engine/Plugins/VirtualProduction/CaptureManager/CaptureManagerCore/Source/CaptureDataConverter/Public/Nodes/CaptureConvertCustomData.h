// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerPipelineNode.h"
#include "Nodes/CaptureConvertParams.h"

class FCaptureConvertCustomData :
	public FCaptureManagerPipelineNode,
	public FCaptureConvertParams 
{
public:

	using FCaptureManagerPipelineNode::FCaptureManagerPipelineNode;
};