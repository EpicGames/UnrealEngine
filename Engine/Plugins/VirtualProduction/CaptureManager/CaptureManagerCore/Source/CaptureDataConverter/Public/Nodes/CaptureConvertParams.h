// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerPipelineNode.h"

#include "CaptureDataConverterNodeParams.h"

#define UE_API CAPTUREDATACONVERTER_API

class FCaptureConvertParams
{
public:

	UE_API FCaptureConvertParams();
	UE_API virtual ~FCaptureConvertParams();

	UE_API void SetParams(const FCaptureConvertDataNodeParams& InParams);

protected:

	FCaptureConvertDataNodeParams Params;

};

#undef UE_API