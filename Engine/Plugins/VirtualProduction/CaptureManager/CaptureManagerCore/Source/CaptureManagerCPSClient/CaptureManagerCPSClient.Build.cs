// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CaptureManagerCPSClient : ModuleRules
{
	public CaptureManagerCPSClient(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "CMCPSClient";

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CaptureUtils",
			"CaptureProtocolStack",
		});
	}
}
