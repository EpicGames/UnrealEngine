// Copyright Epic Games, Inc. All Rights Reserved.

#include "CPSDataStream.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

namespace UE::CaptureManager {

FCPSDataStream::FCPSDataStream(FFileExportFinished InFileExportFinished)
	: FileExportFinished(MoveTemp(InFileExportFinished))
{
}

FCPSDataStream::~FCPSDataStream() = default;

TProtocolResult<void> FCPSDataStream::StartFile(const FString& InTakeName, const FString& InFileName)
{
	checkf(Data.IsEmpty(), TEXT("Data buffer must be empty"));
	return ResultOk;
}

TProtocolResult<void> FCPSDataStream::ProcessData(const FString& InTakeName, const FString& InFileName, const TConstArrayView<uint8>& InData)
{
	Data.Append(InData);
	return ResultOk;
}

TProtocolResult<void> FCPSDataStream::FinishFile(const FString& InTakeName, const FString& InFileName, const TStaticArray<uint8, 16>& InHash)
{
	using namespace UE::CaptureManager;

	FMD5 MD5;
	MD5.Update(Data.GetData(), Data.Num());

	FMD5Hash Hash;
	Hash.Set(MD5);

	if (FMemory::Memcmp(Hash.GetBytes(), InHash.GetData(), Hash.GetSize()) != 0)
	{
		TMap<FString, UE::CaptureManager::TProtocolResult<FData>>& TakeEntry = ExportResults.FindOrAdd(InTakeName);
		TakeEntry.Add(InFileName, FCaptureProtocolError(TEXT("Invalid file hash")));

		// Continue sending the data regardless of the error
		return ResultOk;
	}

	TMap<FString, UE::CaptureManager::TProtocolResult<FData>>& TakeEntry = ExportResults.FindOrAdd(InTakeName);
	TakeEntry.Add(InFileName, MoveTemp(Data));

	return ResultOk;
}

void FCPSDataStream::Finalize(UE::CaptureManager::TProtocolResult<void> InResult)
{
	FileExportFinished.ExecuteIfBound(MoveTemp(ExportResults));
}

}
