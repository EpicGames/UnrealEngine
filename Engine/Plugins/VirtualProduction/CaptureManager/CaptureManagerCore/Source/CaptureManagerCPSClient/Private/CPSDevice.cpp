// Copyright Epic Games, Inc. All Rights Reserved.

#include "CPSDevice.h"

#include "Control/Messages/Constants.h"

#include "Misc/ScopeExit.h"

#include "CaptureUtilsModule.h"

DEFINE_LOG_CATEGORY_STATIC(LogCPSProtocolDevice, Log, All);

#define CHECK_LOG_RETURN_VAL(InCondition, InFormat, ...) \
	if (InCondition) \
	{ \
		FString Message = FString::Printf(InFormat, ##__VA_ARGS__); \
		UE_LOGF(LogCPSProtocolDevice, Warning, "%ls", *Message); \
		return FCaptureProtocolError(MoveTemp(Message)); \
	}

#define CHECK_LOG_RETURN(InCondition, InFormat, ...) \
	if (InCondition) \
	{ \
		UE_LOG(LogCPSProtocolDevice, Warning, InFormat, ##__VA_ARGS__); \
		return; \
	}

namespace UE::CaptureManager
{

FConnectionStateChangedEvent::FConnectionStateChangedEvent(EState InConnectionState)
	: FCaptureEvent(Name)
	, ConnectionState(InConnectionState)
{
}

FConnectionStateChangedEvent::~FConnectionStateChangedEvent() = default;

FCPSEvent::FCPSEvent(TSharedPtr<FControlUpdate> InUpdateMessage)
	: FCaptureEvent(Name)
	, UpdateMessage(MoveTemp(InUpdateMessage))
{
}

FCPSEvent::~FCPSEvent() = default;

FCPSStateEvent::FCPSStateEvent(FGetStateResponse InGetStateResponse)
	: FCaptureEvent(Name)
	, GetStateResponse(MoveTemp(InGetStateResponse))
{
}

FCPSStateEvent::~FCPSStateEvent() = default;

TSharedPtr<FCPSDevice> FCPSDevice::MakeCPSDevice(FString InDeviceIpAddress, uint16 InDevicePort)
{
	TSharedPtr<FCPSDevice> CPSDevice = MakeShared<FCPSDevice>(FPrivateToken(), MoveTemp(InDeviceIpAddress), InDevicePort);
	CPSDevice->InitializeDelegates();

	return CPSDevice;
}

FCPSDevice::FCPSDevice(FCPSDevice::FPrivateToken,
					   FString InDeviceIpAddress,
					   uint16 InDevicePort)
	: TimerManager(GetTimerManager())
	, DeviceIpAddress(MoveTemp(InDeviceIpAddress))
	, DeviceControlPort(InDevicePort)
	, bIsConnected(false)
{
	RegisterEvent(FConnectionStateChangedEvent::Name); // Connection changed
	RegisterEvent(FCPSStateEvent::Name); // State update
	RegisterEvent(FCPSEvent::Name); // Other events
}

FCPSDevice::~FCPSDevice()
{
	Stop();
}

void FCPSDevice::InitializeDelegates()
{
	ConnThread = MakeUnique<FConnectionThread>(FConnectionThread::FOnProcess::CreateSP(this, &FCPSDevice::ConnectControlClient));

	RegisterForAllEvents();

	ControlMessenger.RegisterDisconnectHandler(FControlMessenger::FOnDisconnect::CreateSP(this, &FCPSDevice::OnDisconnect));
}

void FCPSDevice::InitiateConnect()
{
	PublishEvent<FConnectionStateChangedEvent>(FConnectionStateChangedEvent::EState::Connecting);
	StartConnectTimer();
}

void FCPSDevice::Stop()
{
	bool bIsConnectedLocal = bIsConnected.exchange(false);

	ConnThread->Empty();
	TimerManager->RemoveTimer(ConnectTimerHandle);

	ControlMessenger.Stop();

	if (bIsConnectedLocal)
	{
		PublishEvent<FConnectionStateChangedEvent>(FConnectionStateChangedEvent::EState::Disconnected);
	}
}

bool FCPSDevice::IsConnected() const
{
	return bIsConnected.load();
}

TProtocolResult<void> FCPSDevice::StartRecording(FString SlateName,
												 uint16 TakeNumber,
												 TOptional<FString> Subject,
												 TOptional<FString> Scenario,
												 TOptional<TArray<FString>> Tags)
{
	

	if (!bIsConnected.load())
	{
		FString Message = FString::Format(TEXT("Device is not connected: {0}:{1}"), { DeviceIpAddress, FString::FromInt(DeviceControlPort) });
		return FCaptureProtocolError(MoveTemp(Message));
	}

	FStartRecordingTakeRequest Request(MoveTemp(SlateName),
									   TakeNumber,
									   MoveTemp(Subject),
									   MoveTemp(Scenario),
									   MoveTemp(Tags));

	TProtocolResult<FStartRecordingTakeResponse> Response = ControlMessenger.SendRequest(MoveTemp(Request));

	if (Response.HasError())
	{
		return Response.StealError();
	}

	return ResultOk;
}

TProtocolResult<void> FCPSDevice::StopRecording()
{
	if (!bIsConnected.load())
	{
		FString Message = FString::Format(TEXT("Device is not connected: {0}:{1}"), { DeviceIpAddress, FString::FromInt(DeviceControlPort) });
		return FCaptureProtocolError(MoveTemp(Message));
	}

	FStopRecordingTakeRequest Request;

	TProtocolResult<FStopRecordingTakeResponse> Response = ControlMessenger.SendRequest(MoveTemp(Request));

	if (Response.HasError())
	{
		return Response.StealError();
	}

	return ResultOk;
}

TProtocolResult<TArray<FGetTakeMetadataResponse::FTakeObject>> FCPSDevice::FetchTakeList()
{
	if (!bIsConnected.load())
	{
		FString Message = FString::Format(TEXT("Device is not connected: {0}:{1}"), { DeviceIpAddress, FString::FromInt(DeviceControlPort) });
		return FCaptureProtocolError(MoveTemp(Message));
	}

	TProtocolResult<FGetTakeListResponse> TakeListRequestResult = ControlMessenger.GetTakeList();
	CHECK_LOG_RETURN_VAL(TakeListRequestResult.HasError(), TEXT("Failed to fetch the take list from the remote host."));

	FGetTakeListResponse TakeListResponse = TakeListRequestResult.StealValue();

	FGetTakeMetadataRequest TakeMetadataRequest(TakeListResponse.GetNames());

	TProtocolResult<FGetTakeMetadataResponse> TakeMetadataRequestResult = ControlMessenger.SendRequest(MoveTemp(TakeMetadataRequest));
	CHECK_LOG_RETURN_VAL(TakeMetadataRequestResult.HasError(), TEXT("Failed to fetch the take metadata from the remote host. Note: Please ensure you are using compatible versions of LLF and UE"));

	FGetTakeMetadataResponse TakeMetadataResponse = TakeMetadataRequestResult.StealValue();

	return TakeMetadataResponse.GetTakes();
}

TProtocolResult<FGetTakeMetadataResponse::FTakeObject> FCPSDevice::FetchTake(const FString& InTakeName)
{
	if (!bIsConnected.load())
	{
		FString Message = FString::Format(TEXT("Device is not connected: {0}:{1}"), { DeviceIpAddress, FString::FromInt(DeviceControlPort) });
		return FCaptureProtocolError(MoveTemp(Message));
	}

	FGetTakeMetadataRequest TakeMetadataRequest({ InTakeName });

	TProtocolResult<FGetTakeMetadataResponse> TakeMetadataRequestResult = ControlMessenger.SendRequest(MoveTemp(TakeMetadataRequest));
	CHECK_LOG_RETURN_VAL(TakeMetadataRequestResult.HasError(), TEXT("Failed to fetch the take metadata from the remote host. Please ensure you are using compatible versions of LLF and UE"));

	FGetTakeMetadataResponse TakeMetadataResponse = TakeMetadataRequestResult.StealValue();

	TArray<FGetTakeMetadataResponse::FTakeObject> Take = TakeMetadataResponse.GetTakes();

	if (Take.IsEmpty())
	{
		return FCaptureProtocolError(FString::Format(TEXT("Failed to obtain the take with the specified name: {0}"), { InTakeName }));
	}

	check(Take.Num() == 1);

	return Take[0];
}

void FCPSDevice::AddTakeMetadata(FTakeId InId, FGetTakeMetadataResponse::FTakeObject InTake)
{
	FScopeLock Lock(&Mutex);
	TakeMetadata.Add(InId, MoveTemp(InTake));
}

void FCPSDevice::RemoveTakeMetadata(FTakeId InId)
{
	FScopeLock Lock(&Mutex);
	TakeMetadata.Remove(InId);
}

FGetTakeMetadataResponse::FTakeObject FCPSDevice::GetTake(FTakeId InId)
{
	FScopeLock Lock(&Mutex);
	return TakeMetadata[InId];
}

FTakeId FCPSDevice::GetTakeId(const FString& InTakeName)
{
	FScopeLock Lock(&Mutex);

	for (const TPair<FTakeId, FGetTakeMetadataResponse::FTakeObject>& TakeElement : TakeMetadata)
	{
		if (TakeElement.Value.Name == InTakeName)
		{
			return TakeElement.Key;
		}
	}

	return INDEX_NONE;
}

void FCPSDevice::StartExport(FTakeId InTakeId, TUniquePtr<FBaseStream> InStream)
{
	check(ExportClient);

	const FGetTakeMetadataResponse::FTakeObject& Take = GetTake(InTakeId);

	FExportClient::FTakeFileArray TakeFiles;
	TakeFiles.Reserve(Take.Files.Num());

	for (const FGetTakeMetadataResponse::FFileObject& File : Take.Files)
	{
		TakeFiles.Add({ File.Name, File.Length, 0 });
	}

	FExportClient::FTaskId TaskId = ExportClient->ExportTakeFiles(Take.Name, MoveTemp(TakeFiles), MoveTemp(InStream));
	IdMap.Add(InTakeId, TaskId);
}

void FCPSDevice::CancelExport(FTakeId InTakeId)
{
	if (!ExportClient)
	{
		return;
	}

	if (FExportClient::FTaskId* TaskId = IdMap.Find(InTakeId))
	{
		ExportClient->AbortExport(*TaskId);
	}
}

void FCPSDevice::CancelAllExports()
{
	if (ExportClient)
	{
		ExportClient->AbortAllExports();
	}
}

void FCPSDevice::FetchThumbnailForTake(FTakeId InTakeId, TUniquePtr<FBaseStream> InStream)
{
	FetchFileForTake(InTakeId, MoveTemp(InStream), FString("thumbnail.jpg"));
}

void FCPSDevice::FetchThumbnails(TUniquePtr<FBaseStream> InStream)
{
	FetchFiles(MoveTemp(InStream), TArray<FString>{ FString("thumbnail.jpg") });
}

void FCPSDevice::FetchFileForTake(FTakeId InTakeId, TUniquePtr<FBaseStream> InStream, const FString& InFileName)
{
	FScopeLock Lock(&Mutex);
	if (FGetTakeMetadataResponse::FTakeObject* TakeObjectPtr = TakeMetadata.Find(InTakeId))
	{
		// Intentional copy
		const FGetTakeMetadataResponse::FTakeObject TakeObject = *TakeObjectPtr;

		// Unlock earlier as there is nothing below that needs protecting
		Lock.Unlock();

		uint64 Length = 0;
		for (const FGetTakeMetadataResponse::FFileObject& FileObject : TakeObject.Files)
		{
			if (FileObject.Name == InFileName)
			{
				Length = FileObject.Length;
			}
		}

		if (Length == 0)
		{
			return;
		}

		FTakeFile TakeFile;
		TakeFile.FileName = InFileName;
		TakeFile.Length = Length;
		TakeFile.Offset = 0;

		check(ExportClient);
		ExportClient->ExportTakeFiles(TakeObject.Name, { MoveTemp(TakeFile) }, MoveTemp(InStream));
	}
}

void FCPSDevice::FetchFiles(TUniquePtr<FBaseStream> InStream, TArray<FString> InFileNames)
{
	TMap<FString, FExportClient::FTakeFileArray> TakeFileArrayMap;

	FScopeLock Lock(&Mutex);

	for (const TPair<FTakeId, FGetTakeMetadataResponse::FTakeObject>& TakePair : TakeMetadata)
	{
		const FGetTakeMetadataResponse::FTakeObject& TakeObject = TakePair.Value;

		uint64 Length = 0;
		FString FileName;
		FExportClient::FTakeFileArray FileArray;
		for (const FGetTakeMetadataResponse::FFileObject& FileObject : TakeObject.Files)
		{
			if (InFileNames.Contains(FileObject.Name))
			{
				Length = FileObject.Length;
				FileName = FileObject.Name;

				if (Length == 0)
				{
					continue;
				}

				FTakeFile TakeFile;
				TakeFile.FileName = FileName;
				TakeFile.Length = Length;
				TakeFile.Offset = 0;

				FileArray.Add(MoveTemp(TakeFile));
			}
		}

		TakeFileArrayMap.Add(TakeObject.Name, MoveTemp(FileArray));
	}

	// Unlock earlier as there is no need to lock the ExportFiles function
	Lock.Unlock();

	check(ExportClient);
	ExportClient->ExportFiles(MoveTemp(TakeFileArrayMap), MoveTemp(InStream));
}

void FCPSDevice::ConnectControlClient(FEmpty)
{
	if (bIsConnected.load())
	{
		return;
	}

	ON_SCOPE_EXIT
	{
		if (!bIsConnected.load())
		{
			ControlMessenger.Stop();
		}
	};

	if (ControlMessenger.Start(DeviceIpAddress, DeviceControlPort).HasError())
	{
		UE_LOGF(LogCPSProtocolDevice, Verbose, "Failed to connect to %ls:%d", *DeviceIpAddress, DeviceControlPort);
		return;
	}

	CHECK_LOG_RETURN(ControlMessenger.StartSession().HasError(), TEXT("Failed to start session for the Control client. Please ensure you are using compatible versions of LLF and UE"));

	CHECK_LOG_RETURN(ControlMessenger.SendRequest(FSubscribeRequest()).HasError(), TEXT("Failed to subscribe to events for Control client."));

	TProtocolResult<FGetServerInformationResponse> GetServerInformationResult = ControlMessenger.GetServerInformation();
	CHECK_LOG_RETURN(GetServerInformationResult.HasError(), TEXT("Failed to fetch the Control server information"));
	FGetServerInformationResponse GetServerInformationResponse = GetServerInformationResult.StealValue();

	ExportClient = MakeUnique<FExportClient>(DeviceIpAddress, GetServerInformationResponse.GetExportPort());

	TProtocolResult<FGetStateResponse> GetStateResult = ControlMessenger.SendRequest(FGetStateRequest());
	CHECK_LOG_RETURN(GetStateResult.HasError(), TEXT("Failed to fetch the current state of the Control server"));

	FGetStateResponse GetStateResponse = GetStateResult.StealValue();
	PublishEvent<FCPSStateEvent>(MoveTemp(GetStateResponse));

	PublishEvent<FConnectionStateChangedEvent>(FConnectionStateChangedEvent::EState::Connected);

	bIsConnected.store(true);

	UE_LOGF(LogCPSProtocolDevice, Display, "Connected to the CPS Device: %ls:%d", *DeviceIpAddress, DeviceControlPort);
}

void FCPSDevice::RegisterForAllEvents()
{
	ControlMessenger.RegisterUpdateHandler(CPS::AddressPaths::GTakeAdded, FControlUpdate::FOnUpdateMessage::CreateSP(this, &FCPSDevice::OnCPSEvent));
	ControlMessenger.RegisterUpdateHandler(CPS::AddressPaths::GTakeRemoved, FControlUpdate::FOnUpdateMessage::CreateSP(this, &FCPSDevice::OnCPSEvent));
	ControlMessenger.RegisterUpdateHandler(CPS::AddressPaths::GTakeUpdated, FControlUpdate::FOnUpdateMessage::CreateSP(this, &FCPSDevice::OnCPSEvent));
	ControlMessenger.RegisterUpdateHandler(CPS::AddressPaths::GRecordingStatus, FControlUpdate::FOnUpdateMessage::CreateSP(this, &FCPSDevice::OnCPSEvent));
	ControlMessenger.RegisterUpdateHandler(CPS::AddressPaths::GDiskCapacity, FControlUpdate::FOnUpdateMessage::CreateSP(this, &FCPSDevice::OnCPSEvent));
	ControlMessenger.RegisterUpdateHandler(CPS::AddressPaths::GBattery, FControlUpdate::FOnUpdateMessage::CreateSP(this, &FCPSDevice::OnCPSEvent));
	ControlMessenger.RegisterUpdateHandler(CPS::AddressPaths::GThermalState, FControlUpdate::FOnUpdateMessage::CreateSP(this, &FCPSDevice::OnCPSEvent));
}

void FCPSDevice::OnCPSEvent(TSharedPtr<FControlUpdate> InUpdateMessage)
{
	PublishEvent<FCPSEvent>(MoveTemp(InUpdateMessage));
}

void FCPSDevice::StartConnectTimer(float InDelay)
{
	if (!ConnectTimerHandle.IsValid())
	{
		ConnectTimerHandle = TimerManager->AddTimer(FTimerDelegate::CreateSP(this, &FCPSDevice::OnConnectTick), ConnectInterval, true, InDelay);
	}
}

void FCPSDevice::OnConnectTick()
{
	if (!bIsConnected.load())
	{
		UE_LOGF(LogCPSProtocolDevice, Log, "Connecting to the CPS Device: %ls:%d", *DeviceIpAddress, DeviceControlPort);
		ConnThread->Add(FEmpty()); // Initiate connect
	}
	else
	{
		TimerManager->RemoveTimer(ConnectTimerHandle);
	}
}

void FCPSDevice::OnDisconnect(const FString& InCause)
{
	if (bIsConnected.exchange(false))
	{
		PublishEvent<FConnectionStateChangedEvent>(FConnectionStateChangedEvent::EState::Disconnected);
	}

	PublishEvent<FConnectionStateChangedEvent>(FConnectionStateChangedEvent::EState::Connecting);
	StartConnectTimer(ConnectInterval);
}

TSharedRef<FCaptureTimerManager> FCPSDevice::GetTimerManager()
{
	FCaptureUtilsModule& Module = FModuleManager::LoadModuleChecked<FCaptureUtilsModule>(TEXT("CaptureUtils"));
	return Module.GetTimerManager();
}

}
