// Copyright Epic Games, Inc. All Rights Reserved.

#include "CPSFileStream.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

namespace UE::CaptureManager
{

FCPSFileStream::FCPSFileStream(FString InBaseDir, uint64 InSize)
	: BaseDir(MoveTemp(InBaseDir))
	, TotalExportExpectedSize(InSize)
	, TotalExportArrivedSize(0)
{
}

FCPSFileStream::~FCPSFileStream() = default;

TProtocolResult<void> FCPSFileStream::StartFile(const FString& InTakeName, const FString& InFileName)
{
	FString FilePathToBeSaved = FPaths::Combine(BaseDir, InTakeName, InFileName);

	if (!FPaths::IsUnderDirectory(FilePathToBeSaved, BaseDir))
	{
		return FCaptureProtocolError(FString::Format(TEXT("Unable to write a file to {0}"), { FPaths::ConvertRelativePathToFull(FilePathToBeSaved) }));
	}

	checkf(!Writer, TEXT("Writer must be nullptr when new file arrives"));
	Writer = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*FilePathToBeSaved, EFileWrite::FILEWRITE_None));

	if (!Writer)
	{
		// If writer has not been assigned it is likely another writer is writing to FilePathToBeSaved
		return FCaptureProtocolError(TEXT("Unable to create a writer for file"));
	}

	checkf(!MD5Generator, TEXT("MD5 generator must be nullptr when new file arrives"));
	MD5Generator = MakeUnique<FMD5>();

	return ResultOk;
}

TProtocolResult<void> FCPSFileStream::ProcessData(const FString& InTakeName, const FString& InFileName, const TConstArrayView<uint8>& InData)
{
	Writer->Serialize(const_cast<uint8*>(InData.GetData()), InData.Num());
	ReportProgressStep(InFileName, InData.Num());

	MD5Generator->Update(InData.GetData(), InData.Num());

	return ResultOk;
}

TProtocolResult<void> FCPSFileStream::FinishFile(const FString& InTakeName, const FString& InFileName, const TStaticArray<uint8, 16>& InHash)
{
	checkf(Writer, TEXT("Writer must be valid when file has arrived"));
	Writer->Close();
	Writer = nullptr;

	checkf(MD5Generator, TEXT("MD5 Generator must be valid when file has arrived"));

	static constexpr uint32 HashSize = 16;
	TStaticArray<uint8, HashSize> Hash = MakeUniformStaticArray<uint8, HashSize>(0);

	MD5Generator->Final(Hash.GetData());
	MD5Generator = nullptr;

	if (FMemory::Memcmp(Hash.GetData(), InHash.GetData(), Hash.Num()) != 0)
	{
		return FCaptureProtocolError(TEXT("Invalid hash file"));
	}

	return ResultOk;
}

void FCPSFileStream::Finalize(UE::CaptureManager::TProtocolResult<void> InResult)
{
	// Close the writer if not closed
	if (Writer)
	{
		Writer->Close();
		Writer = nullptr;
	}

	OnExportFinished.ExecuteIfBound(MoveTemp(InResult));
}

void FCPSFileStream::SetExportFinished(FExportFinished InExportFinished)
{
	OnExportFinished = MoveTemp(InExportFinished);
}

void FCPSFileStream::SetProgressHandler(FReportProgress InReportProgress)
{
	OnReportProgress = MoveTemp(InReportProgress);
}

void FCPSFileStream::ReportProgressStep(const FString& InFileName, const uint32 InArrivedSize)
{
	TotalExportArrivedSize += InArrivedSize;
	OnReportProgress.ExecuteIfBound((float) TotalExportArrivedSize / (float) TotalExportExpectedSize);
}

}
