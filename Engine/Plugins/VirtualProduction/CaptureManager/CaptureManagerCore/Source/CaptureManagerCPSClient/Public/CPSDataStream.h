// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExportClient/ExportClient.h"

#include "Containers/Map.h"

#define UE_API CAPTUREMANAGERCPSCLIENT_API

namespace UE::CaptureManager {

class FCPSDataStream : public UE::CaptureManager::FBaseStream
{
public:
	using FData = TArray<uint8>;
	using FResults = TMap<FString, TMap<FString, UE::CaptureManager::TProtocolResult<FData>>>;

	DECLARE_DELEGATE_OneParam(FFileExportFinished, FResults InExportResults);

	UE_API FCPSDataStream(FFileExportFinished InFileExportFinished);
	UE_API ~FCPSDataStream();

	UE_API TProtocolResult<void> StartFile(const FString& InTakeName, const FString& InFileName) override;
	UE_API TProtocolResult<void> ProcessData(const FString& InTakeName, const FString& InFileName, const TConstArrayView<uint8>& InData) override;
	UE_API TProtocolResult<void> FinishFile(const FString& InTakeName, const FString& InFileName, const TStaticArray<uint8, 16>& InHash) override;

	UE_API void Finalize(UE::CaptureManager::TProtocolResult<void> InResult) override;

private:
	FData Data;
	FFileExportFinished FileExportFinished;

	FResults ExportResults;
};

}

#undef UE_API
