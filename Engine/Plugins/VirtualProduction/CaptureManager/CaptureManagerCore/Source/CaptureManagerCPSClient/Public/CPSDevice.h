// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Control/ControlMessenger.h"

#include "ExportClient/ExportClient.h"

#include "Async/EventSourceUtils.h"
#include "Misc/Optional.h"

#include "Async/CaptureTimerManager.h"

#include "CaptureManagerTakeId.h"

#define UE_API CAPTUREMANAGERCPSCLIENT_API

namespace UE::CaptureManager
{

struct FConnectionStateChangedEvent : public FCaptureEvent
{
	inline static const FString Name = TEXT("ConnectionStateChanged");

	enum class EState
	{
		Unknown = 0,
		Connecting,
		Connected,
		Disconnected
	};

	UE_API FConnectionStateChangedEvent(EState InConnectionState);
	UE_API ~FConnectionStateChangedEvent();

	EState ConnectionState;
};

struct FCPSEvent : public FCaptureEvent
{
	inline static const FString Name = TEXT("CPSEvent");

	UE_API FCPSEvent(TSharedPtr<FControlUpdate> InUpdateMessage);
	UE_API ~FCPSEvent();

	TSharedPtr<FControlUpdate> UpdateMessage;
};

struct FCPSStateEvent : public FCaptureEvent
{
	inline static const FString Name = TEXT("CPSStateEvent");

	UE_API FCPSStateEvent(FGetStateResponse InGetStateResponse);
	UE_API ~FCPSStateEvent();

	FGetStateResponse GetStateResponse;
};

class FCPSDevice :
	public FCaptureEventSource,
	public TSharedFromThis<FCPSDevice>
{
private:

	struct FPrivateToken
	{
		explicit FPrivateToken() = default;
	};

public:

	UE_API static TSharedPtr<FCPSDevice> MakeCPSDevice(FString InDeviceIpAddress, uint16 InDevicePort);

	UE_API FCPSDevice(FPrivateToken,
			   FString InDeviceIpAddress,
			   uint16 InDevicePort);

	UE_API virtual ~FCPSDevice() override;

	UE_API void InitiateConnect();
	UE_API void Stop();

	UE_API bool IsConnected() const;

	UE_API TProtocolResult<void> StartRecording(FString SlateName,
										 uint16 TakeNumber,
										 TOptional<FString> Subject = TOptional<FString>(),
										 TOptional<FString> Scenario = TOptional<FString>(),
										 TOptional<TArray<FString>> Tags = TOptional<TArray<FString>>());

	UE_API TProtocolResult<void> StopRecording();

	UE_API TProtocolResult<TArray<FGetTakeMetadataResponse::FTakeObject>> FetchTakeList();
	UE_API TProtocolResult<FGetTakeMetadataResponse::FTakeObject> FetchTake(const FString& InTakeName);

	UE_API void AddTakeMetadata(FTakeId InId, FGetTakeMetadataResponse::FTakeObject InTake);
	UE_API void RemoveTakeMetadata(FTakeId InId);
	UE_API FGetTakeMetadataResponse::FTakeObject GetTake(FTakeId InId);
	UE_API FTakeId GetTakeId(const FString& InTakeName);

	UE_API void StartExport(FTakeId InTakeId, TUniquePtr<FBaseStream> InStream);
	UE_API void CancelExport(FTakeId InTakeId);
	UE_API void CancelAllExports();

	/** Fetches the thumbnail for a single take. */
	UE_API void FetchThumbnailForTake(FTakeId InTakeId, TUniquePtr<FBaseStream> InStream);
	/** Fetches the thumbnails for all takes. */
	UE_API void FetchThumbnails(TUniquePtr<FBaseStream> InStream);

	/** Fetches the specified file for a single take. */
	UE_API void FetchFileForTake(FTakeId InTakeId, TUniquePtr<FBaseStream> InStream, const FString& InFileName);
	/** Fetches the specified files for all takes. */
	UE_API void FetchFiles(TUniquePtr<FBaseStream> InStream, TArray<FString> InFileNames);

private:

	inline static const float ConnectInterval = 5.0f;

	void InitializeDelegates();

	struct FEmpty
	{
	};

	void ConnectControlClient(FEmpty);

	void RegisterForAllEvents();
	void OnCPSEvent(TSharedPtr<FControlUpdate> InUpdateMessage);

	void StartConnectTimer(float InDelay = 0.0f);
	void OnConnectTick();
	void OnDisconnect(const FString& InCause);

	static TSharedRef<FCaptureTimerManager> GetTimerManager();

	TSharedRef<FCaptureTimerManager> TimerManager;

	FString DeviceIpAddress;
	uint16 DeviceControlPort;

	std::atomic_bool bIsConnected;

	using FConnectionThread = TQueueRunner<FEmpty>;
	TUniquePtr<FConnectionThread> ConnThread;
	FControlMessenger ControlMessenger;
	FCaptureTimerManager::FTimerHandle ConnectTimerHandle;

	TUniquePtr<FExportClient> ExportClient;

	FCriticalSection Mutex;
	TMap<FTakeId, FGetTakeMetadataResponse::FTakeObject> TakeMetadata;

	TMap<FTakeId, FExportClient::FTaskId> IdMap;
};

}

#undef UE_API
