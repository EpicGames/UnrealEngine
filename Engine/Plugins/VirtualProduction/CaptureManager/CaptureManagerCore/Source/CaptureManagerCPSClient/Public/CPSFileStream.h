// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExportClient/ExportClient.h"

#define UE_API CAPTUREMANAGERCPSCLIENT_API

namespace UE::CaptureManager
{

class FCPSFileStream : public UE::CaptureManager::FBaseStream
{
public:

	DECLARE_DELEGATE_OneParam(FReportProgress, float InProgress);
	DECLARE_DELEGATE_OneParam(FExportFinished, UE::CaptureManager::TProtocolResult<void> InResult);

	UE_API FCPSFileStream(FString InBaseDir, uint64 InSize);
	UE_API ~FCPSFileStream();

	UE_API virtual TProtocolResult<void> StartFile(const FString& InTakeName, const FString& InFileName) override;

	UE_API virtual TProtocolResult<void> ProcessData(const FString& InTakeName, const FString& InFileName, const TConstArrayView<uint8>& InData) override;
	UE_API virtual TProtocolResult<void> FinishFile(const FString& InTakeName, const FString& InFileName, const TStaticArray<uint8, 16>& InHash) override;

	UE_API virtual void Finalize(UE::CaptureManager::TProtocolResult<void> InResult) override;

	UE_API void SetExportFinished(FExportFinished InExportFinished);
	UE_API void SetProgressHandler(FReportProgress InReportProgress);

private:

	void ReportProgressStep(const FString& InFileName, const uint32 InArrivedSize);

	FString BaseDir;
	TUniquePtr<FArchive> Writer;
	TUniquePtr<FMD5> MD5Generator;

	uint64 TotalExportExpectedSize;
	uint64 TotalExportArrivedSize;

	FExportFinished OnExportFinished;
	FReportProgress OnReportProgress;
};

}

#undef UE_API
