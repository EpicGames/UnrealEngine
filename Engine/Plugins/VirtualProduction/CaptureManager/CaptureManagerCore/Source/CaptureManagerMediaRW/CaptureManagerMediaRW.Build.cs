// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CaptureManagerMediaRW : ModuleRules
{
	public CaptureManagerMediaRW(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"Media"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Engine",
			"RenderCore",
			"ImageWriteQueue",
			"ImageCore",
			"OodleDataCompression",
			"Json",
			"JsonUtilities",
			"DataIngestCore",
			"CaptureDataUtils"
		});

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			// Windows Media Foundation dependencies required for ingest data preparation on the Windows platform
			PublicDelayLoadDLLs.AddRange(new[] {
				"mf.dll", "mfplat.dll", "mfreadwrite.dll", "mfuuid.dll", "propsys.dll"
			});

			PublicSystemLibraries.AddRange(new[]
			{
				"mf.lib", "mfplat.lib", "mfreadwrite.lib", "mfuuid.lib", "propsys.lib"
			});
		}
	}
}