// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerMediaRWModule.h"

#if PLATFORM_WINDOWS && !UE_SERVER

#include "Utils/WindowsRWHelpers.h"

#endif
#include "Readers/MHADepthVideoReader.h"
#include "Readers/MHAICalibrationReader.h"
#include "Readers/OpenCVCalibrationReader.h"
#include "Readers/ImageSequenceReader.h"

#include "Writers/AudioWaveMediaWriter.h"
#include "Writers/DepthImageWriter.h"
#include "Writers/UnrealCalibrationWriter.h"

void FCaptureManagerMediaRWModule::StartupModule()
{
	MediaRWManager = MakeUnique<FMediaRWManager>();

	// Readers
#if PLATFORM_WINDOWS && !UE_SERVER
	if (FWindowsRWHelpers::Init())
	{
		FWindowsRWHelpers::RegisterReaders(*MediaRWManager);
		FWindowsRWHelpers::RegisterWriters(*MediaRWManager);
	}
#endif

	FMHADepthVideoReaderHelpers::RegisterReaders(*MediaRWManager);
	FMHAICalibrationReaderHelpers::RegisterReaders(*MediaRWManager);
	FOpenCvCalibrationReaderHelpers::RegisterReaders(*MediaRWManager);
	FImageSequenceReaderHelper::RegisterReaders(*MediaRWManager);

	// Writers
	FAudioWaveWriterHelpers::RegisterWriters(*MediaRWManager);
	FDepthExrImageWriterHelpers::RegisterWriters(*MediaRWManager);
	FUnrealCalibrationWriterHelpers::RegisterWriters(*MediaRWManager);
}

void FCaptureManagerMediaRWModule::ShutdownModule()
{
#if PLATFORM_WINDOWS && !UE_SERVER
	FWindowsRWHelpers::Deinit();
#endif

	MediaRWManager = nullptr;
}

FMediaRWManager& FCaptureManagerMediaRWModule::Get()
{
	return *MediaRWManager;
}

IMPLEMENT_MODULE(FCaptureManagerMediaRWModule, CaptureManagerMediaRW);