// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaRWManager.h"

#define LOCTEXT_NAMESPACE "MediaRWManager"

FMediaRWManager::FMediaRWManager() = default;

template<typename T>
FString CreateFormatKey(const TMap<FString, T*>& InMap, const FString& InFormat)
{
	uint32 Counter = 0;

	FString FormatKey = FString::Format(TEXT("{0}_{1}"), { InFormat, Counter });

	while (InMap.Contains(FormatKey))
	{
		FormatKey = FString::Format(TEXT("{0}_{1}"), { InFormat, FString::FromInt(++Counter) });
	}

	return FormatKey;
}

template<typename RegistryEntry>
void UpdateFormatRegistry(const TArray<FString>& InFormats, RegistryEntry* InEntry, TMap<FString, RegistryEntry*>& OutFormatRegistry)
{
	for (const FString& Format : InFormats)
	{
		FString FormatKey = CreateFormatKey(OutFormatRegistry, Format);
		OutFormatRegistry.Add(FormatKey, InEntry);
	}
}

FString GetFormatKey(const FString& InFormat, int32 InIndex)
{
	return FString::Format(TEXT("{0}_{1}"), { InFormat, FString::FromInt(InIndex) });
}

void FMediaRWManager::RegisterAudioReader(const TArray<FString>& InFormats, TUniquePtr<IAudioReaderFactory> InReader)
{
	checkf(InReader.IsValid(), TEXT("Reader MUST not be NULL"));

	UpdateFormatRegistry(InFormats, InReader.Get(), AudioReadersPerFormat);
	AudioReaders.Add(MoveTemp(InReader));
}

void FMediaRWManager::RegisterVideoReader(const TArray<FString>& InFormats, TUniquePtr<IVideoReaderFactory> InReader)
{
	checkf(InReader.IsValid(), TEXT("Reader MUST not be NULL"));

	UpdateFormatRegistry(InFormats, InReader.Get(), VideoReadersPerFormat);
	VideoReaders.Add(MoveTemp(InReader));
}

void FMediaRWManager::RegisterCalibrationReader(const TArray<FString>& InFormats, TUniquePtr<ICalibrationReaderFactory> InReader)
{
	checkf(InReader.IsValid(), TEXT("Reader MUST not be NULL"));

	UpdateFormatRegistry(InFormats, InReader.Get(), CalibrationReadersPerFormat);
	CalibrationReaders.Add(MoveTemp(InReader));
}

void FMediaRWManager::RegisterAudioWriter(const TArray<FString>& InFormats, TUniquePtr<IAudioWriterFactory> InWriter)
{
	checkf(InWriter.IsValid(), TEXT("Writer MUST not be NULL"));

	UpdateFormatRegistry(InFormats, InWriter.Get(), AudioWritersPerFormat);
	AudioWriters.Add(MoveTemp(InWriter));
}

void FMediaRWManager::RegisterImageWriter(const TArray<FString>& InFormats, TUniquePtr<IImageWriterFactory> InWriter)
{
	checkf(InWriter.IsValid(), TEXT("Writer MUST not be NULL"));

	UpdateFormatRegistry(InFormats, InWriter.Get(), ImageWritersPerFormat);
	ImageWriters.Add(MoveTemp(InWriter));
}

void FMediaRWManager::RegisterCalibrationWriter(const TArray<FString>& InFormats, TUniquePtr<ICalibrationWriterFactory> InWriter)
{
	checkf(InWriter.IsValid(), TEXT("Writer MUST not be NULL"));

	UpdateFormatRegistry(InFormats, InWriter.Get(), CalibrationWritersPerFormat);
	CalibrationWriters.Add(MoveTemp(InWriter));
}

TUniquePtr<IAudioReader> FMediaRWManager::CreateAudioReaderByFormat(const FString& InFormat, int32 InIndex)
{
	FString Key = GetFormatKey(InFormat, InIndex);
	IAudioReaderFactory** Factory = AudioReadersPerFormat.Find(Key);

	if (Factory)
	{
		return (*Factory)->CreateAudioReader();
	}

	return nullptr;
}

TUniquePtr<IVideoReader> FMediaRWManager::CreateVideoReaderByFormat(const FString& InFormat, int32 InIndex)
{
	FString Key = GetFormatKey(InFormat, InIndex);

	IVideoReaderFactory** Factory = VideoReadersPerFormat.Find(Key);

	if (Factory)
	{
		return (*Factory)->CreateVideoReader();
	}

	return nullptr;
}

TUniquePtr<ICalibrationReader> FMediaRWManager::CreateCalibrationReaderByFormat(const FString& InFormat, int32 InIndex)
{
	FString Key = GetFormatKey(InFormat, InIndex);

	ICalibrationReaderFactory** Factory = CalibrationReadersPerFormat.Find(Key);

	if (Factory)
	{
		return (*Factory)->CreateCalibrationReader();
	}

	return nullptr;
}

TValueOrError<TUniquePtr<IAudioReader>, FText> FMediaRWManager::CreateAudioReader(const FString& InPath, int32 InIndex)
{
	FString Format = FPaths::GetExtension(InPath);

	TUniquePtr<IAudioReader> Reader = CreateAudioReaderByFormat(Format);

	if (!Reader)
	{
		return MakeError(LOCTEXT("CreateAudioReader_ReaderIsntRegistered", "Audio reader for specified format isn't registered"));
	}

	TOptional<FText> Error = Reader->Open(InPath);

	if (Error.IsSet())
	{
		return MakeError(Error.GetValue());
	}

	return MakeValue(MoveTemp(Reader));
}

TValueOrError<TUniquePtr<IVideoReader>, FText> FMediaRWManager::CreateVideoReader(const FString& InPath, int32 InIndex)
{
	FString Format = FPaths::GetExtension(InPath);

	TUniquePtr<IVideoReader> Reader = CreateVideoReaderByFormat(Format);

	if (!Reader)
	{
		return MakeError(LOCTEXT("CreateVideoReader_ReaderIsntRegistered", "Video reader for specified format isn't registered"));
	}

	TOptional<FText> Error = Reader->Open(InPath);

	if (Error.IsSet())
	{
		return MakeError(Error.GetValue());
	}

	return MakeValue(MoveTemp(Reader));
}

TValueOrError<TUniquePtr<ICalibrationReader>, FText> FMediaRWManager::CreateCalibrationReader(const FString& InPath, int32 InIndex)
{
	FString Format = FPaths::GetExtension(InPath);

	TUniquePtr<ICalibrationReader> Reader = CreateCalibrationReaderByFormat(Format);

	if (!Reader)
	{
		return MakeError(LOCTEXT("CreateCalibrationReader_ReaderIsntRegistered", "Calibration reader for specified format isn't registered"));
	}

	TOptional<FText> Error = Reader->Open(InPath);

	if (Error.IsSet())
	{
		return MakeError(Error.GetValue());
	}

	return MakeValue(MoveTemp(Reader));
}

TUniquePtr<IAudioWriter> FMediaRWManager::CreateAudioWriterByFormat(const FString& InFormat, int32 InIndex)
{
	FString Key = GetFormatKey(InFormat, InIndex);

	IAudioWriterFactory** Factory = AudioWritersPerFormat.Find(Key);

	if (Factory)
	{
		return (*Factory)->CreateAudioWriter();
	}

	return nullptr;
}

TUniquePtr<IImageWriter> FMediaRWManager::CreateImageWriterByFormat(const FString& InFormat, int32 InIndex)
{
	FString Key = GetFormatKey(InFormat, InIndex);

	IImageWriterFactory** Factory = ImageWritersPerFormat.Find(Key);

	if (Factory)
	{
		return (*Factory)->CreateImageWriter();
	}

	return nullptr;
}

TUniquePtr<ICalibrationWriter> FMediaRWManager::CreateCalibrationWriterByFormat(const FString& InFormat, int32 InIndex)
{
	FString Key = GetFormatKey(InFormat, InIndex);

	ICalibrationWriterFactory** Factory = CalibrationWritersPerFormat.Find(Key);

	if (Factory)
	{
		return (*Factory)->CreateCalibrationWriter();
	}

	return nullptr;
}

TValueOrError<TUniquePtr<IAudioWriter>, FText> FMediaRWManager::CreateAudioWriter(const FString& InDirectory, const FString& InFileName, const FString& InFormat, int32 InIndex)
{
	TUniquePtr<IAudioWriter> Writer = CreateAudioWriterByFormat(InFormat);

	if (!Writer)
	{
		return MakeError(LOCTEXT("CreateAudioWriter_WriterIsntRegistered", "Audio writer for specified format isn't registered"));
	}

	TOptional<FText> Error = Writer->Open(InDirectory, InFileName, InFormat);

	if (Error.IsSet())
	{
		return MakeError(Error.GetValue());
	}

	return MakeValue(MoveTemp(Writer));
}

TValueOrError<TUniquePtr<IImageWriter>, FText> FMediaRWManager::CreateImageWriter(const FString& InDirectory, const FString& InFileName, const FString& InFormat, int32 InIndex)
{
	TUniquePtr<IImageWriter> Writer = CreateImageWriterByFormat(InFormat);

	if (!Writer)
	{
		return MakeError(LOCTEXT("CreateImageWriter_WriterIsntRegistered", "Image writer for specified format isn't registered"));
	}

	TOptional<FText> Error = Writer->Open(InDirectory, InFileName, InFormat);

	if (Error.IsSet())
	{
		return MakeError(Error.GetValue());
	}

	return MakeValue(MoveTemp(Writer));
}

TValueOrError<TUniquePtr<ICalibrationWriter>, FText> FMediaRWManager::CreateCalibrationWriter(const FString& InDirectory, const FString& InFileName, const FString& InFormat, int32 InIndex)
{
	TUniquePtr<ICalibrationWriter> Writer = CreateCalibrationWriterByFormat(InFormat);

	if (!Writer)
	{
		return MakeError(LOCTEXT("CreateCalibrationWriter_WriterIsntRegistered", "Calibration writer for specified format isn't registered"));
	}

	TOptional<FText> Error = Writer->Open(InDirectory, InFileName, InFormat);

	if (Error.IsSet())
	{
		return MakeError(Error.GetValue());
	}

	return MakeValue(MoveTemp(Writer));
}

#undef LOCTEXT_NAMESPACE