// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaSample.h"

namespace UE::CaptureManager
{

int32 ConvertBitsPerSample(EMediaAudioSampleFormat InSampleFormat)
{
	switch (InSampleFormat)
	{
		case EMediaAudioSampleFormat::Double:
			return 64;
		case EMediaAudioSampleFormat::Float:
		case EMediaAudioSampleFormat::Int32:
			return 32;
		case EMediaAudioSampleFormat::Int8:
			return 8;
		case EMediaAudioSampleFormat::Int16:
			return 16;
		default:
			return 16;
	}
}

int32 ConvertSampleRate(ESampleRate InSampleRate)
{
	switch (InSampleRate)
	{
		case UE::CaptureManager::ESampleRate::SR_8000Hz:
			return 8000;
		case UE::CaptureManager::ESampleRate::SR_16000Hz:
			return 16000;
		case UE::CaptureManager::ESampleRate::SR_48000Hz:
			return 48000;
		case UE::CaptureManager::ESampleRate::SR_88200Hz:
			return 88200;
		case UE::CaptureManager::ESampleRate::SR_96000Hz:
			return 96000;
		case UE::CaptureManager::ESampleRate::SR_192000Hz:
			return 192000;
		case UE::CaptureManager::ESampleRate::SR_44100Hz:
		default:
			return 44100;
	}
}

ESampleRate ConvertSampleRate(int32 InSampleRate)
{
	switch (InSampleRate)
	{
		case 8000:
			return UE::CaptureManager::ESampleRate::SR_8000Hz;
		case 16000:
			return UE::CaptureManager::ESampleRate::SR_16000Hz;
		case 48000:
			return UE::CaptureManager::ESampleRate::SR_48000Hz;
		case 88200:
			return UE::CaptureManager::ESampleRate::SR_88200Hz;
		case 96000:
			return UE::CaptureManager::ESampleRate::SR_96000Hz;
		case 192000:
			return UE::CaptureManager::ESampleRate::SR_192000Hz;
		default:
			return UE::CaptureManager::ESampleRate::SR_44100Hz;
	}
}

uint32 GetNumberOfChannels(EMediaTexturePixelFormat InPixelFormat)
{
	switch (InPixelFormat)
	{
		case UE::CaptureManager::EMediaTexturePixelFormat::U8_RGBA:
		case UE::CaptureManager::EMediaTexturePixelFormat::U8_BGRA:
		case UE::CaptureManager::EMediaTexturePixelFormat::F_Mono:
			return 4;
		case UE::CaptureManager::EMediaTexturePixelFormat::U8_RGB:
		case UE::CaptureManager::EMediaTexturePixelFormat::U8_BGR:
			return 3;
		case UE::CaptureManager::EMediaTexturePixelFormat::U8_YUY2:
		case UE::CaptureManager::EMediaTexturePixelFormat::U16_Mono:
			return 2;
		case UE::CaptureManager::EMediaTexturePixelFormat::U8_I444:
		case UE::CaptureManager::EMediaTexturePixelFormat::U8_I420:
		case UE::CaptureManager::EMediaTexturePixelFormat::U8_NV12:
		case UE::CaptureManager::EMediaTexturePixelFormat::U8_Mono:
		default:
			return 1;
	}
}

int32 FCoordinateSystem::GetIndex(FDirection InDirection)
{
	if (InDirection == FDirection::Front || InDirection == FDirection::Back)
	{
		return 0;
	}
	else if (InDirection == FDirection::Right || InDirection == FDirection::Left)
	{
		return 1;
	}
	else
	{
		return 2;
	}
}

int32 FCoordinateSystem::GetSign(FDirection InDirection)
{
	if (InDirection == FDirection::Front || InDirection == FDirection::Right || InDirection == FDirection::Up)
	{
		return 1;
	}
	else
	{
		return -1;
	}
}

FCoordinateSystem::FCoordinateSystem()
	: MatDescription(FMatrix::Identity)
{
}

FCoordinateSystem::FCoordinateSystem(FMatrix InMatDescription)
	: MatDescription(MoveTemp(InMatDescription))
{
	MatDescription.SetOrigin(FVector::ZeroVector);
	MatDescription.M[3][3] = 1.0;

	// Check if the matrix is orthogonal
	check((MatDescription * MatDescription.GetTransposed()).Equals(FMatrix::Identity, 0.0));
}

FCoordinateSystem::FCoordinateSystem(FDirection InXDirection, 
									 FDirection InYDirection, 
									 FDirection InZDirection)
	: MatDescription(EForceInit::ForceInitToZero)
{
	MatDescription.M[0][GetIndex(InXDirection)] = GetSign(InXDirection);
	MatDescription.M[1][GetIndex(InYDirection)] = GetSign(InYDirection);
	MatDescription.M[2][GetIndex(InZDirection)] = GetSign(InZDirection);
	MatDescription.M[3][3] = 1.0;

	// Check if the matrix is orthogonal
	check((MatDescription * MatDescription.GetTransposed()).Equals(FMatrix::Identity, 0.0));
}

const FMatrix& FCoordinateSystem::GetMatDescription() const
{
	return MatDescription;
}

const FCoordinateSystem UnrealCS;
const FCoordinateSystem OpenCvCS = FCoordinateSystem(FCoordinateSystem::Right, 
													 FCoordinateSystem::Down, 
													 FCoordinateSystem::Front);

FVector ConvertToCoordinateSystem(const FVector& InVector,
								  const FCoordinateSystem& InInputCoordinateSystem,
								  const FCoordinateSystem& InOutputCoordinateSystem)
{
	FMatrix InputMatrix = InInputCoordinateSystem.GetMatDescription();
	FMatrix OutputMatrix = InOutputCoordinateSystem.GetMatDescription();

	// Since we are working with the orthogonal matrices, we can assume that inverse is the same as transpose
	FVector4 Result = InputMatrix.GetTransposed().TransformVector(InVector);
	return OutputMatrix.TransformVector(Result);
}

FTransform ConvertToCoordinateSystem(const FTransform& InTransform,
									 const FCoordinateSystem& InInputCoordinateSystem,
									 const FCoordinateSystem& InOutputCoordinateSystem)
{
	FMatrix InputMatrix = InInputCoordinateSystem.GetMatDescription();
	FMatrix OutputMatrix = InOutputCoordinateSystem.GetMatDescription();

	// Since we are working with the orthogonal matrices, we can assume that inverse is the same as transpose
	FMatrix Result = OutputMatrix * InputMatrix.GetTransposed() * InTransform.ToMatrixWithScale() * InputMatrix * OutputMatrix.GetTransposed();

	FTransform Transform;

	Transform.SetFromMatrix(Result);

	return Transform;
}

}