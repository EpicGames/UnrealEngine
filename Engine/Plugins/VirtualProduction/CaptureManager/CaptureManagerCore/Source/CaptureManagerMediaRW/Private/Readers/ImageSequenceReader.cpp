// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageSequenceReader.h"

#include "MediaRWManager.h"

#include "Misc/FileHelper.h"
#include "IImageWrapperModule.h"
#include "Modules/ModuleManager.h"

#include "TakeDirectoryUtils.h"

#include "Internationalization/Regex.h"

#define LOCTEXT_NAMESPACE "ImageSequenceReader"

namespace UE::CaptureManager::Private
{

static UE::CaptureManager::EMediaTexturePixelFormat ConvertFormat(ERGBFormat InFormat, int32 InBitDepth)
{
	switch (InFormat)
	{
		case ERGBFormat::Gray:
			if (InBitDepth == 8)
			{
				return UE::CaptureManager::EMediaTexturePixelFormat::U8_Mono;
			}
			else if (InBitDepth == 16)
			{
				return UE::CaptureManager::EMediaTexturePixelFormat::U16_Mono;
			}
			else
			{
				return UE::CaptureManager::EMediaTexturePixelFormat::Undefined;
			}
		case ERGBFormat::GrayF:
			return UE::CaptureManager::EMediaTexturePixelFormat::F_Mono;
		case ERGBFormat::RGBA:
			return UE::CaptureManager::EMediaTexturePixelFormat::U8_RGBA;
		case ERGBFormat::BGRA:
			return UE::CaptureManager::EMediaTexturePixelFormat::U8_BGRA;
		default:
			return UE::CaptureManager::EMediaTexturePixelFormat::Undefined;
	}
}

}

void FImageSequenceReaderHelper::RegisterReaders(FMediaRWManager& InManager)
{
	TArray<FString> SupportedFormats = { TEXT("image_sequence") };
	InManager.RegisterVideoReader(SupportedFormats, MakeUnique<FImageSequenceReaderFactory>());
}

TUniquePtr<IVideoReader> FImageSequenceReaderFactory::CreateVideoReader()
{
	return MakeUnique<FImageSequenceReader>();
}

FImageSequenceReader::FImageSequenceReader() = default;
FImageSequenceReader::~FImageSequenceReader() = default;

TOptional<FText> FImageSequenceReader::Open(const FString& InDirectoryPath)
{
	if (!FPaths::DirectoryExists(InDirectoryPath))
	{
		FText Error = FText::Format(
			LOCTEXT("ImageSequenceFolderMissing", "Failed to read image sequence directory: {0}"),
			FText::FromString(InDirectoryPath));
		return Error;
	}

	using namespace UE::CaptureData;

	ImagePaths = GetImageSequenceFilesFromPath(InDirectoryPath);

	if (ImagePaths.IsEmpty())
	{
		FText Error = FText::Format(
			LOCTEXT("ImageSequenceEmptyDirectory", "No images found in the directory: {0}"),
			FText::FromString(InDirectoryPath));
		return Error;
	}

	check(ImagePaths.IsValidIndex(0));
	GetImageDimensionsFromPath(ImagePaths[0], Dimensions);

	return {};
}

TOptional<FText> FImageSequenceReader::Close()
{
	ImagePaths.Empty();

	return {};
}

TValueOrError<TUniquePtr<UE::CaptureManager::FMediaTextureSample>, FText> FImageSequenceReader::Next()
{
	const int32 CurrentFrame = CurrentFrameNumber;

	if (!ImagePaths.IsValidIndex(CurrentFrame))
	{
		return MakeValue(nullptr);
	}

	const FString& ImagePath = ImagePaths[CurrentFrame];

	TArray<uint8> RawFileData;
	if (!FFileHelper::LoadFileToArray(RawFileData, *ImagePath))
	{
		FText Error = FText::Format(LOCTEXT("ImageSequenceNextFrameReadFailed", "Failed to read frame {0} from {1}"),
									FText::AsNumber(CurrentFrame), FText::FromString(ImagePath));
		return MakeError(MoveTemp(Error));
	}

	TUniquePtr<UE::CaptureManager::FMediaTextureSample> Sample = 
		MakeUnique<UE::CaptureManager::FMediaTextureSample>();

	Sample->Dimensions = Dimensions;
	Sample->Time = FTimespan::FromSeconds(CurrentFrame);

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
	EImageFormat ImageFormat = ImageWrapperModule.DetectImageFormat(RawFileData.GetData(), RawFileData.Num());
	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(ImageFormat);

	if (!ImageWrapper.IsValid() || !ImageWrapper->SetCompressed(RawFileData.GetData(), RawFileData.Num()))
	{
		FText Error = FText::Format(LOCTEXT("ImageSequenceNextDataToImageFailed", "Unsupported image format {0} for {1}"),
									FText::FromString(FPaths::GetExtension(ImagePath, false)), FText::FromString(ImagePath));

		return MakeError(MoveTemp(Error));
	}

	// Emptying the first buffer before obtaining a new one.
	RawFileData.Empty();

	Sample->CurrentFormat = UE::CaptureManager::Private::ConvertFormat(ImageWrapper->GetFormat(), ImageWrapper->GetBitDepth());
	Sample->Stride = Dimensions.X * UE::CaptureManager::GetNumberOfChannels(Sample->CurrentFormat);

	TArray64<uint8> Buffer;
	if (!ImageWrapper->GetRaw(Buffer))
	{
		FText Error = FText::Format(LOCTEXT("ImageSequenceNextDecodeFailed", "Failed to decode the image {0}"),
									FText::FromString(ImagePath));

		return MakeError(MoveTemp(Error));
	}

	Sample->Buffer = MoveTemp(Buffer);

	++CurrentFrameNumber;

	return MakeValue(MoveTemp(Sample));
}

FTimespan FImageSequenceReader::GetDuration() const
{
	return FTimespan::FromSeconds(ImagePaths.Num());
}

FIntPoint FImageSequenceReader::GetDimensions() const
{
	return Dimensions;
}

FFrameRate FImageSequenceReader::GetFrameRate() const
{
	return FFrameRate(1, 1);
}

#undef LOCTEXT_NAMESPACE