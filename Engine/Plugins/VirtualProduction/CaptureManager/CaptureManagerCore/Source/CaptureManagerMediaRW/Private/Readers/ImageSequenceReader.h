// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaReader.h"
#include "IMediaRWFactory.h"

#include "GenericPlatform/GenericPlatformFile.h"

class FImageSequenceReaderHelper
{
public:

	static void RegisterReaders(class FMediaRWManager& InManager);
};

class FImageSequenceReaderFactory final : public IVideoReaderFactory
{
public:

	virtual TUniquePtr<IVideoReader> CreateVideoReader() override;
};

class FImageSequenceReader final : public IVideoReader
{
public:

	FImageSequenceReader();
	virtual ~FImageSequenceReader() override;

	virtual TOptional<FText> Open(const FString& InDirectoryPath) override;
	virtual TOptional<FText> Close() override;

	virtual TValueOrError<TUniquePtr<UE::CaptureManager::FMediaTextureSample>, FText> Next() override;

	// Duration and Frame Rate will be set to number of frames and 1 frame per second
	virtual FTimespan GetDuration() const override;
	virtual FIntPoint GetDimensions() const override;
	virtual FFrameRate GetFrameRate() const override;

private:

	TArray<FString> ImagePaths;
	FIntPoint Dimensions;
	std::atomic_int CurrentFrameNumber = 0;
};