// Copyright Epic Games, Inc. All Rights Reserved.

#include "MHADepthVideoReader.h"

#include "MediaRWManager.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/JsonSerializer.h"

#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"

#include "Utils/ParseTakeUtils.h"

#include <oodle2.h>

#define LOCTEXT_NAMESPACE "MHADepthVideoReader"

#pragma pack(push, 1)
struct FFrameHeader
{
	enum class EFrameType : uint8
	{
		TakeMetadata = 0,
		VideoMetadata = 1,
		DepthMetadata = 2,
		AudioMetadata = 3,
		VideoData = 4,
		DepthData = 5,
		AudioData = 6
	};

	EFrameType FrameType;
	char TimeCode[15];
	int64 TimeValue;
	int32 TimeScale;
	uint32 PayloadLength;
};
#pragma pack(pop)

void FMHADepthVideoReaderHelpers::RegisterReaders(FMediaRWManager& InManager)
{
	TArray<FString> SupportedFormats = { TEXT("mha_depth") };
	InManager.RegisterVideoReader(SupportedFormats, MakeUnique<FMHADepthVideoReaderFactory>());
}

TUniquePtr<IVideoReader> FMHADepthVideoReaderFactory::CreateVideoReader()
{
	return MakeUnique<FMHADepthVideoReader>();
}

TSharedPtr<FJsonObject> ParseDepthTakeMetadata(TArray<uint8> InMetadata)
{
	TSharedPtr<FJsonObject> OutObject;

	FMemoryReader Reader(InMetadata);

	TSharedRef<TJsonReader<UTF8CHAR>> JsonReader = TJsonReaderFactory<UTF8CHAR>::Create(&Reader);
	if (FJsonSerializer::Deserialize(JsonReader, OutObject))
	{
		return OutObject;
	}

	return nullptr;
}

static EMediaOrientation ParseOrientation(int32 InOrientation)
{
	switch (InOrientation)
	{
		case 1: // Portrait
			return EMediaOrientation::Original;
		case 2:	// PortraitUpsideDown
			return EMediaOrientation::CW180;
		case 3: // landscapeLeft
			return EMediaOrientation::CW90;
		case 4: // LandscapeRight
		default:
			return EMediaOrientation::CW270;
	}
}

inline int16 Combine(uint8 InLeft, uint8 InRight)
{
	return (((int16) InLeft << 8) + InRight);
}

inline int16 ZigzagDecode(int16 InValue)
{
	return (InValue >> 1) ^ (-(InValue & 1));
}

FMHADepthVideoReader::FMHADepthVideoReader() = default;
FMHADepthVideoReader::~FMHADepthVideoReader() = default;

TOptional<FText> FMHADepthVideoReader::Open(const FString& InFileName)
{
	check(FPaths::GetExtension(InFileName) == TEXT("bin"));

	ReadHandle.Reset(FPlatformFileManager::Get().GetPlatformFile().OpenRead(*InFileName));
	if (!ReadHandle)
	{
		FText Error = FText::Format(LOCTEXT("DepthFileOpenFailed", "Failed to open the depth file: {0}."), FText::FromString(InFileName));
		return Error;
	}

	TArray<uint8> Metadata;

	bool bMetadataFound = false;

	while (!bMetadataFound)
	{
		FFrameHeader FrameHeader;

		if (!ReadHandle->Read(reinterpret_cast<uint8_t*>(&FrameHeader), sizeof(FrameHeader)))
		{
			break;
		}

		switch (FrameHeader.FrameType)
		{
			case FFrameHeader::EFrameType::DepthMetadata:
				Metadata.SetNum(FrameHeader.PayloadLength);
				if (!ReadHandle->Read(Metadata.GetData(), FrameHeader.PayloadLength))
				{
					FText Error = LOCTEXT("DepthFileOpenDepthDataReadFailed", "Failed to read the depth metadata");
					return Error;
				}
				bMetadataFound = true;
				break;
			default:
				if (!ReadHandle->Seek(ReadHandle->Tell() + FrameHeader.PayloadLength))
				{
					FText Error = LOCTEXT("DepthFileOpenDepthOtherReadFailed", "Failed to read the depth data");
					return Error;
				}
		}
	}

	if (Metadata.IsEmpty())
	{
		FText Error = LOCTEXT("DepthFileOpenDepthMetadataEmptyFailed", "Failed to read the depth metadata");
		return Error;
	}

	TSharedPtr<FJsonObject> MetadataJson = ParseDepthTakeMetadata(Metadata);
	if (MetadataJson)
	{
		int32 IntOrientation = 4;
		MetadataJson->TryGetNumberField(TEXT("Orientation"), IntOrientation);
		Orientation = ParseOrientation(IntOrientation);

		const TSharedPtr<FJsonObject>* DepthDimensions;
		if (MetadataJson->TryGetObjectField(TEXT("DepthDimensions"), DepthDimensions))
		{
			DepthDimensions->ToSharedRef()->TryGetNumberField(TEXT("Width"), Dimensions.X);
			DepthDimensions->ToSharedRef()->TryGetNumberField(TEXT("Height"), Dimensions.Y);
		}

		float FloatFrameRate = 0.f;
		MetadataJson->TryGetNumberField(TEXT("DepthFrameRate"), FloatFrameRate);

		if (!FMath::IsNearlyZero(FloatFrameRate))
		{
			FrameRate = UE::CaptureManager::ParseFrameRate(FloatFrameRate);
		}
	}

	// Return the seek pointer to the start
	ReadHandle->Seek(0);

	return {};
}

TOptional<FText> FMHADepthVideoReader::Close()
{
	ReadHandle = nullptr;

	return {};
}

TValueOrError<TUniquePtr<UE::CaptureManager::FMediaTextureSample>, FText> FMHADepthVideoReader::Next()
{
	TUniquePtr<UE::CaptureManager::FMediaTextureSample> Sample = MakeUnique<UE::CaptureManager::FMediaTextureSample>();

	TArray<uint8> CompressedBuffer;

	bool bDataFound = false;
	while (!bDataFound)
	{
		FFrameHeader FrameHeader;

		// End of stream
		if (!ReadHandle->Read(reinterpret_cast<uint8_t*>(&FrameHeader), sizeof(FrameHeader)))
		{
			return MakeValue(nullptr);
		}

		switch (FrameHeader.FrameType)
		{
			case FFrameHeader::EFrameType::DepthData:
				CompressedBuffer.SetNum(FrameHeader.PayloadLength);
				if (!ReadHandle->Read(CompressedBuffer.GetData(), FrameHeader.PayloadLength))
				{
					FText Error = LOCTEXT("DepthFileNextDepthDataReadFailed", "Failed to read the depth data");
					return MakeError(MoveTemp(Error));
				}
				bDataFound = true;
				break;
			default:
				if (!ReadHandle->Seek(ReadHandle->Tell() + FrameHeader.PayloadLength))
				{
					FText Error = LOCTEXT("DepthFileNextDepthOtherReadFailed", "Failed to read the depth data");
					return MakeError(MoveTemp(Error));
				}
				continue;
		}
	}

	if (CompressedBuffer.IsEmpty())
	{
		FText Error = LOCTEXT("DepthFileNextDepthCompressedBufferEmptyReadFailed", "Failed to read the depth data");
		return MakeError(MoveTemp(Error));
	}

	Sample->Buffer.SetNumZeroed(Dimensions.X * Dimensions.Y * sizeof(int16)); // Depth data is int16

	//	Decompress the depth data.
	if (OodleLZ_Decompress(CompressedBuffer.GetData(), CompressedBuffer.Num(), Sample->Buffer.GetData(), Sample->Buffer.Num()) != Sample->Buffer.Num())
	{
		FText Error = LOCTEXT("DepthDataNextCorrupted", "Corrupted depth data detected");
		return MakeError(MoveTemp(Error));
	}

	//	Process the zigzag decoding and un-differentiate the depth values
	int16 PreviousValue = 0;

	for (int32 Index = 0; Index < Sample->Buffer.Num(); Index += 2)
	{
		int16 Value = *reinterpret_cast<const uint16*>(Sample->Buffer.GetData() + Index);

		const int16 ThisValue = PreviousValue + ZigzagDecode(Value);
		PreviousValue = ThisValue;

		FMemory::Memcpy(Sample->Buffer.GetData() + Index, &ThisValue, sizeof(int16));
	}

	Sample->Dimensions = Dimensions;
	Sample->CurrentFormat = UE::CaptureManager::EMediaTexturePixelFormat::U16_Mono;
	Sample->Orientation = Orientation;

	return MakeValue(MoveTemp(Sample));
}

FTimespan FMHADepthVideoReader::GetDuration() const
{
	// Unknown
	return FTimespan();
}

FIntPoint FMHADepthVideoReader::GetDimensions() const
{
	return Dimensions;
}

FFrameRate FMHADepthVideoReader::GetFrameRate() const
{
	return FrameRate;
}

#undef LOCTEXT_NAMESPACE