// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaReader.h"
#include "IMediaRWFactory.h"

#include "GenericPlatform/GenericPlatformFile.h"

class FMHADepthVideoReaderHelpers
{
public:

	static void RegisterReaders(class FMediaRWManager& InManager);
};

class FMHADepthVideoReaderFactory final : public IVideoReaderFactory
{
public:

	virtual TUniquePtr<IVideoReader> CreateVideoReader() override;
};

class FMHADepthVideoReader final : public IVideoReader
{
public:

	FMHADepthVideoReader();
	virtual ~FMHADepthVideoReader() override;

	virtual TOptional<FText> Open(const FString& InFileName) override;
	virtual TOptional<FText> Close() override;

	virtual TValueOrError<TUniquePtr<UE::CaptureManager::FMediaTextureSample>, FText> Next() override;

	virtual FTimespan GetDuration() const override;
	virtual FIntPoint GetDimensions() const override;
	virtual FFrameRate GetFrameRate() const override;

private:

	TUniquePtr<IFileHandle> ReadHandle;
	FIntPoint Dimensions;
	FFrameRate FrameRate = FFrameRate(30'000, 1'000);
	EMediaOrientation Orientation;
};