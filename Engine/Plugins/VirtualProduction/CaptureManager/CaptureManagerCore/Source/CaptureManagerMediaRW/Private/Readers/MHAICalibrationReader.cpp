// Copyright Epic Games, Inc. All Rights Reserved.

#include "MHAICalibrationReader.h"

#include "MediaRWManager.h"

#include "Serialization/JsonSerializer.h"

#include "Misc/FileHelper.h"

#define LOCTEXT_NAMESPACE "MHAICalibrationReader"

DEFINE_LOG_CATEGORY_STATIC(LogMHAICalibrationReader, Log, All);

#define MHAICAL_CHECK_AND_RETURN_MESSAGE(Result, Message)                              \
	if (!(Result))                                                                     \
	{                                                                                  \
		FText ErrorMessage = FText::Format(FText::FromString(TEXT("{0}")), Message);   \
		UE_LOGF(LogMHAICalibrationReader, Error, "%ls", *ErrorMessage.ToString()); \
		return ErrorMessage;                                                           \
	}

#define MHAICAL_CHECK_AND_RETURN_ERROR(Result, Message)                                \
	if (!(Result))                                                                     \
	{                                                                                  \
		FText ErrorMessage = FText::Format(FText::FromString(TEXT("{0}")), Message);   \
		UE_LOGF(LogMHAICalibrationReader, Error, "%ls", *ErrorMessage.ToString()); \
		return MakeError(MoveTemp(ErrorMessage));                                      \
	}

void FMHAICalibrationReaderHelpers::RegisterReaders(FMediaRWManager& InManager)
{
	TArray<FString> SupportedFormats = { TEXT("mhaical") };
	InManager.RegisterCalibrationReader(SupportedFormats, MakeUnique<FMHAICalibrationReaderFactory>());
}

TUniquePtr<ICalibrationReader> FMHAICalibrationReaderFactory::CreateCalibrationReader()
{
	return MakeUnique<FMHAICalibrationReader>();
}

TOptional<FText> FMHAICalibrationReader::Open(const FString& InFileName)
{
	MHAICAL_CHECK_AND_RETURN_MESSAGE(FPaths::GetExtension(InFileName) == TEXT("mhaical"), LOCTEXT("MHAICalibrationReader_InvalidExtension", "Provided file must have .mhaical extension"));

	FString JsonContent;
	MHAICAL_CHECK_AND_RETURN_MESSAGE(FFileHelper::LoadFileToString(JsonContent, *InFileName), LOCTEXT("MHAICalibrationReader_LoadFailed", "Failed to load the provided file"));

	TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<>::Create(JsonContent);
	MHAICAL_CHECK_AND_RETURN_MESSAGE(FJsonSerializer::Deserialize(JsonReader, JsonObject), LOCTEXT("MHAICalibrationReader_DeserializeFailed", "Failed to deserialize the file into json"));

	return {};
}

TOptional<FText> FMHAICalibrationReader::Close()
{
	JsonObject = nullptr;

	return {};
}

TValueOrError<TUniquePtr<UE::CaptureManager::FMediaCalibrationSample>, FText> FMHAICalibrationReader::Next()
{
	const TSharedPtr<FJsonObject>* Dimensions;
	
	if (CurrentSampleType == ESampleType::EndOfStream)
	{
		// End of stream
		return MakeValue(nullptr);
	}

	TUniquePtr<UE::CaptureManager::FMediaCalibrationSample> CalibrationSample = MakeUnique<UE::CaptureManager::FMediaCalibrationSample>();

	if (CurrentSampleType == ESampleType::Depth)
	{
		CalibrationSample->CameraId = TEXT("Depth");
		CalibrationSample->CameraType = UE::CaptureManager::FMediaCalibrationSample::Depth;

		MHAICAL_CHECK_AND_RETURN_ERROR(
			JsonObject->TryGetObjectField(TEXT("DepthDimensions"), Dimensions),
			LOCTEXT("MHAICalibrationReader_NoResolution", "Failed to obtain resolution"));
	}
	else 
	{
		CalibrationSample->CameraId = TEXT("Video");
		CalibrationSample->CameraType = UE::CaptureManager::FMediaCalibrationSample::Video;

		// ESampleType::Video
		MHAICAL_CHECK_AND_RETURN_ERROR(
			JsonObject->TryGetObjectField(TEXT("VideoDimensions"), Dimensions),
			LOCTEXT("MHAICalibrationReader_NoResolution", "Failed to obtain resolution"));
	}

	(*Dimensions)->TryGetNumberField(TEXT("Width"), CalibrationSample->Dimensions.X);
	(*Dimensions)->TryGetNumberField(TEXT("Height"), CalibrationSample->Dimensions.Y);

	int32 Orientation = 4;
	JsonObject->TryGetNumberField(TEXT("Orientation"), Orientation);
	CalibrationSample->Orientation = ParseOrientation(Orientation);

	const TSharedPtr<FJsonObject>* RefDimensions;

	MHAICAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetObjectField(TEXT("IntrinsicMatrixReferenceDimensions"), RefDimensions), 
		LOCTEXT("MHAICalibrationReader_NoRefDimnsions", "Failed to obtain reference dimension"));

	FIntPoint IntrinsicMatrixReferenceDimensions;
	(*RefDimensions)->TryGetNumberField(TEXT("Width"), IntrinsicMatrixReferenceDimensions.X);
	(*RefDimensions)->TryGetNumberField(TEXT("Height"), IntrinsicMatrixReferenceDimensions.Y);

	const TSharedPtr<FJsonObject>* LensDistorionCenter;
	MHAICAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetObjectField(TEXT("LensDistortionCenter"), LensDistorionCenter),
		LOCTEXT("MHAICalibrationReader_NoLensDistortionCenter", "Failed to obtain lens distortion center"));

	(*LensDistorionCenter)->TryGetNumberField(TEXT("X"), CalibrationSample->PrincipalPoint.X);
	(*LensDistorionCenter)->TryGetNumberField(TEXT("Y"), CalibrationSample->PrincipalPoint.Y);

	const TArray<TSharedPtr<FJsonValue>>* IntrinsicMatrixJson;
	MHAICAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetArrayField(TEXT("IntrinsicMatrix"), IntrinsicMatrixJson), 
		LOCTEXT("MHAICalibrationReader_NoIntrinsics", "Failed to obtain camera intrinsics"));

	const TArray<TSharedPtr<FJsonValue>>* LensDistortionTable;
	MHAICAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetArrayField(TEXT("LensDistortionLookupTable"), LensDistortionTable),
		LOCTEXT("MHAICalibrationReader_NoLensDistortionTable", "Failed to obtain lens distortion table"));

	const TArray<TSharedPtr<FJsonValue>>* InverseLensDistortionTable;
	MHAICAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetArrayField(TEXT("InverseLensDistortionLookupTable"), InverseLensDistortionTable),
		LOCTEXT("MHAICalibrationReader_NoInverseLensDistortionTable", "Failed to obtain inverse lens distortion table"));

	check(LensDistortionTable->Num() == InverseLensDistortionTable->Num());

	// Depth Camera doesn't have distortion model
	if (CurrentSampleType != ESampleType::Depth)
	{
		UE::CaptureManager::FIphoneDistortionModel DistortionModel;

		for (int32 Index = 0; Index < LensDistortionTable->Num(); ++Index)
		{
			const TSharedPtr<FJsonValue>& LensDistortionValue = (*LensDistortionTable)[Index];
			const TSharedPtr<FJsonValue>& InverseLensDistortionValue = (*InverseLensDistortionTable)[Index];

			DistortionModel.LensDistortionTable.Add(LensDistortionValue->AsNumber());
			DistortionModel.InverseLensDistortionTable.Add(InverseLensDistortionValue->AsNumber());
		}

		CalibrationSample->DistortionModel.Set<UE::CaptureManager::FIphoneDistortionModel>(MoveTemp(DistortionModel));
	}

	/*
		Intrinsics Matrix:
			[fx, 0., cx]
			[0., fy, cy]
			[0., 0., 1.]

		Focal Length (fx, fy)
		Optical Center (cx, cy)
	*/

	int32 XIndex = 0;
	int32 YIndex = 0;
	FMatrix IntrinsicMatrix(EForceInit::ForceInitToZero);

	for (const TSharedPtr<FJsonValue>& Value : *IntrinsicMatrixJson)
	{
		IntrinsicMatrix.M[XIndex][YIndex++] = Value->AsNumber();

		if (YIndex == 3)
		{
			YIndex = 0;
			++XIndex;
		}
	}

	// Normalize FocalLength
	CalibrationSample->FocalLength.X = IntrinsicMatrix.M[0][0] / IntrinsicMatrixReferenceDimensions.X;
	CalibrationSample->FocalLength.Y = IntrinsicMatrix.M[1][1] / IntrinsicMatrixReferenceDimensions.Y;

	float ScaleX = float(CalibrationSample->Dimensions.X) / float(IntrinsicMatrixReferenceDimensions.X);
	float ScaleY = float(CalibrationSample->Dimensions.Y) / float(IntrinsicMatrixReferenceDimensions.Y);
	
	// Normalize OpticalCenter
	FVector2D PrincipalPoint;
	PrincipalPoint.X = ((IntrinsicMatrix.M[0][2] + 0.5f) * ScaleX - 0.5f) / CalibrationSample->Dimensions.X;
	PrincipalPoint.Y = ((IntrinsicMatrix.M[1][2] + 0.5f) * ScaleY - 0.5f) / CalibrationSample->Dimensions.Y;

	CalibrationSample->PrincipalPoint = PrincipalPoint;
	CalibrationSample->InputCoordinateSystem = UE::CaptureManager::UnrealCS;

	CalibrationSample->Transform.SetFromMatrix(FMatrix::Identity);

	SwitchSampleType();

	return MakeValue(MoveTemp(CalibrationSample));
}

EMediaOrientation FMHAICalibrationReader::ParseOrientation(int32 InOrientation)
{
	// 1: Portrait, 2: Landscape, 3: Portrait, 4: Landscape (as per iPhone documentation)
	switch (InOrientation)
	{
		case 1: // Portrait
			return EMediaOrientation::Original;
		case 2:	// PortraitUpsideDown
			return EMediaOrientation::CW180;
		case 3: // LandscapeLeft
			return EMediaOrientation::CW90;
		case 4: // LandscapeRight
		default:
			return EMediaOrientation::CW270;
	}
}

void FMHAICalibrationReader::SwitchSampleType()
{
	if (CurrentSampleType == ESampleType::Depth)
	{
		CurrentSampleType = ESampleType::Video;
	}
	else
	{
		CurrentSampleType = ESampleType::EndOfStream;
	}
}

#undef LOCTEXT_NAMESPACE