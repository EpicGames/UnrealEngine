// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaReader.h"
#include "IMediaRWFactory.h"

#include "Dom/JsonObject.h"

class FMHAICalibrationReaderHelpers
{
public:

	static void RegisterReaders(class FMediaRWManager& InManager);
};

class FMHAICalibrationReaderFactory : public ICalibrationReaderFactory
{
public:

	virtual TUniquePtr<ICalibrationReader> CreateCalibrationReader() override;
};

class FMHAICalibrationReader : public ICalibrationReader
{
public:

	virtual TOptional<FText> Open(const FString& InFileName) override;
	virtual TOptional<FText> Close() override;

	virtual TValueOrError<TUniquePtr<UE::CaptureManager::FMediaCalibrationSample>, FText> Next() override;

private:
	
	enum ESampleType
	{
		Depth = 0,
		Video,
		EndOfStream
	};

	EMediaOrientation ParseOrientation(int32 InOrientation);
	void SwitchSampleType();

	TSharedPtr<FJsonObject> JsonObject;
	ESampleType CurrentSampleType = ESampleType::Depth;
};