// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenCVCalibrationReader.h"

#include "MediaRWManager.h"

#include "Serialization/JsonSerializer.h"

#include "Misc/FileHelper.h"

#define LOCTEXT_NAMESPACE "OpenCvCalibrationReader"

DEFINE_LOG_CATEGORY_STATIC(LogOpenCvCalibrationReader, Log, All);

#define OPENCVCAL_CHECK_AND_RETURN_MESSAGE(Result, Message)                              \
	if (!(Result))                                                                       \
	{                                                                                    \
		FText ErrorMessage = FText::Format(FText::FromString(TEXT("{0}")), Message);     \
		UE_LOGF(LogOpenCvCalibrationReader, Error, "%ls", *ErrorMessage.ToString()); \
		return ErrorMessage;                                                             \
	}

#define OPENCVCAL_CHECK_AND_RETURN_ERROR(Result, Message)                                \
	if (!(Result))                                                                       \
	{                                                                                    \
		FText ErrorMessage = FText::Format(FText::FromString(TEXT("{0}")), Message);     \
		UE_LOGF(LogOpenCvCalibrationReader, Error, "%ls", *ErrorMessage.ToString()); \
		return MakeError(MoveTemp(ErrorMessage));                                        \
	}

void FOpenCvCalibrationReaderHelpers::RegisterReaders(FMediaRWManager& InManager)
{
	const TArray<FString> SupportedFormats = { TEXT("opencv") };
	InManager.RegisterCalibrationReader(SupportedFormats, MakeUnique<FOpenCvCalibrationReaderFactory>());
}

TUniquePtr<ICalibrationReader> FOpenCvCalibrationReaderFactory::CreateCalibrationReader()
{
	return MakeUnique<FOpenCvCalibrationReader>();
}

TOptional<FText> FOpenCvCalibrationReader::Open(const FString& InFileName)
{
	OPENCVCAL_CHECK_AND_RETURN_MESSAGE(FPaths::GetExtension(InFileName) == TEXT("json"), LOCTEXT("OpenCvCalibrationReader_InvalidExtension", "Provided file must have .json extension"));

	FString JsonContent;
	OPENCVCAL_CHECK_AND_RETURN_MESSAGE(FFileHelper::LoadFileToString(JsonContent, *InFileName), LOCTEXT("OpenCvCalibrationReader_LoadFailed", "Failed to load the provided file"));

	TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<>::Create(JsonContent);
	OPENCVCAL_CHECK_AND_RETURN_MESSAGE(FJsonSerializer::Deserialize(JsonReader, JsonArray), LOCTEXT("OpenCvCalibrationReader_DeserializeFailed", "Failed to deserialize the file into json"));

	ArrayIndex = 0;

	return {};
}

TOptional<FText> FOpenCvCalibrationReader::Close()
{
	JsonArray.Empty();

	return {};
}

TValueOrError<TUniquePtr<UE::CaptureManager::FMediaCalibrationSample>, FText> FOpenCvCalibrationReader::Next()
{
	if (!JsonArray.IsValidIndex(ArrayIndex))
	{
		// End of stream
		return MakeValue(nullptr);
	}

	TSharedPtr<FJsonObject> JsonObject = JsonArray[ArrayIndex]->AsObject();

	const TSharedPtr<FJsonObject>* Metadata;

	OPENCVCAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetObjectField(TEXT("Metadata"), Metadata),
		LOCTEXT("OpenCVCalibrationReader_NoMetadata", "Failed to obtain metadata"));

	FString CameraId;
	bool ContainsCamera = (*Metadata)->TryGetStringField(TEXT("camera"), CameraId);

	if (!ContainsCamera)
	{
		// Ignoring the first element as it doesn't carry the calibration information
		++ArrayIndex;
		return Next();
	}

	TUniquePtr<UE::CaptureManager::FMediaCalibrationSample> CalibrationSample = MakeUnique<UE::CaptureManager::FMediaCalibrationSample>();

	FString CameraType;
	(*Metadata)->TryGetStringField(TEXT("type"), CameraType);

	CalibrationSample->CameraId = MoveTemp(CameraId);
	
	if (CalibrationSample->CameraId.Equals(TEXT("depth"), ESearchCase::Type::IgnoreCase) ||
		CameraType.Equals(TEXT("depth"), ESearchCase::Type::IgnoreCase))
	{
		CalibrationSample->CameraType = UE::CaptureManager::FMediaCalibrationSample::Depth;
	}
	else
	{
		CalibrationSample->CameraType = UE::CaptureManager::FMediaCalibrationSample::Video;
	}

	OPENCVCAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetNumberField(TEXT("image_size_x"), CalibrationSample->Dimensions.X),
		LOCTEXT("OpenCVCalibrationReader_NoImageWidth", "Failed to obtain the width"));
	
	OPENCVCAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetNumberField(TEXT("image_size_y"), CalibrationSample->Dimensions.Y),
		LOCTEXT("OpenCVCalibrationReader_NoImageHeight", "Failed to obtain the height"));

	FVector2D FocalLengthNotNormalized;
	OPENCVCAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetNumberField(TEXT("fx"), FocalLengthNotNormalized.X),
		LOCTEXT("OpenCVCalibrationReader_NoFocalWidth", "Failed to obtain the focal width"));

	OPENCVCAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetNumberField(TEXT("fy"), FocalLengthNotNormalized.Y),
		LOCTEXT("OpenCVCalibrationReader_NoFocalHeight", "Failed to obtain the focal height"));

	CalibrationSample->FocalLength.X = FocalLengthNotNormalized.X / CalibrationSample->Dimensions.X;
	CalibrationSample->FocalLength.Y = FocalLengthNotNormalized.Y / CalibrationSample->Dimensions.Y;

	FVector2D PrincipalPointNotNormalized;
	OPENCVCAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetNumberField(TEXT("cx"), PrincipalPointNotNormalized.X),
		LOCTEXT("OpenCVCalibrationReader_NoPrincipalPointX", "Failed to obtain the principal point (X)"));

	OPENCVCAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetNumberField(TEXT("cy"), PrincipalPointNotNormalized.Y),
		LOCTEXT("OpenCVCalibrationReader_NoPrincipalPointY", "Failed to obtain the principal point (Y)"));

	CalibrationSample->PrincipalPoint.X = PrincipalPointNotNormalized.X / CalibrationSample->Dimensions.X;
	CalibrationSample->PrincipalPoint.Y = PrincipalPointNotNormalized.Y / CalibrationSample->Dimensions.Y;

	UE::CaptureManager::FOpenCVDistortionModel DistortionModel;
	OPENCVCAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetNumberField(TEXT("k1"), DistortionModel.Radial.K1),
		LOCTEXT("OpenCVCalibrationReader_NoRadialDistortionK1", "Failed to obtain the radial distortion coeffiecient (k1)"));

	OPENCVCAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetNumberField(TEXT("k2"), DistortionModel.Radial.K2),
		LOCTEXT("OpenCVCalibrationReader_NoRadialDistortionK2", "Failed to obtain the radial distortion coeffiecient (k2)"));

	OPENCVCAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetNumberField(TEXT("p1"), DistortionModel.Tangential.P1),
		LOCTEXT("OpenCVCalibrationReader_NoTangentialDistortionP1", "Failed to obtain the tangential distortion coeffiecient (p1)"));

	OPENCVCAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetNumberField(TEXT("p2"), DistortionModel.Tangential.P2),
		LOCTEXT("OpenCVCalibrationReader_NoTangentialDistortionP2", "Failed to obtain the tangential distortion coeffiecient (p2)"));

	OPENCVCAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetNumberField(TEXT("k3"), DistortionModel.Radial.K3),
		LOCTEXT("OpenCVCalibrationReader_NoRadialDistortionK3", "Failed to obtain the radial distortion coeffiecient (k3)"));

	CalibrationSample->DistortionModel.Set<UE::CaptureManager::FOpenCVDistortionModel>(MoveTemp(DistortionModel));

	const TArray<TSharedPtr<FJsonValue>>* TransformArrayPtr;
	OPENCVCAL_CHECK_AND_RETURN_ERROR(
		JsonObject->TryGetArrayField(TEXT("transform"), TransformArrayPtr),
		LOCTEXT("OpenCVCalibrationReader_NoTransform", "Failed to obtain the transform matrix"));

	const TArray<TSharedPtr<FJsonValue>>& TransformArray = *TransformArrayPtr;

	FMatrix TransformMatrix;
	for (int32 Row = 0; Row < 4; ++Row)
	{
		for (int32 Column = 0; Column < 4; ++Column)
		{
			double Elem = TransformArray[Row * 4 + Column]->AsNumber();
			TransformMatrix.M[Column][Row] = Elem;
		}
	}

	CalibrationSample->Transform.SetFromMatrix(TransformMatrix);

	CalibrationSample->Orientation = EMediaOrientation::Original;
	CalibrationSample->InputCoordinateSystem = UE::CaptureManager::OpenCvCS;

	++ArrayIndex;

	return MakeValue(MoveTemp(CalibrationSample));
}

#undef LOCTEXT_NAMESPACE