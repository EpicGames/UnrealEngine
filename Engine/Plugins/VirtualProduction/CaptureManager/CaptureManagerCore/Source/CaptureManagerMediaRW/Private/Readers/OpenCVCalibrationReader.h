// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaReader.h"
#include "IMediaRWFactory.h"

#include "Dom/JsonObject.h"

class FOpenCvCalibrationReaderHelpers
{
public:

	static void RegisterReaders(class FMediaRWManager& InManager);
};

class FOpenCvCalibrationReaderFactory : public ICalibrationReaderFactory
{
public:

	virtual TUniquePtr<ICalibrationReader> CreateCalibrationReader() override;
};

class FOpenCvCalibrationReader : public ICalibrationReader
{
public:

	virtual TOptional<FText> Open(const FString& InFileName) override;
	virtual TOptional<FText> Close() override;

	virtual TValueOrError<TUniquePtr<UE::CaptureManager::FMediaCalibrationSample>, FText> Next() override;

private:

	TArray<TSharedPtr<FJsonValue>> JsonArray;
	std::atomic_int ArrayIndex = 0;
};