// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsMediaReader.h"

#include "MediaRWManager.h"

#if PLATFORM_WINDOWS && !UE_SERVER

#include "Utils/WindowsRWHelpers.h"
#include "Utils/WindowsJpegDecoder.h"

#define LOCTEXT_NAMESPACE "WindowsReader"

DEFINE_LOG_CATEGORY_STATIC(LogWindowsReader, Log, All);

#define WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, Message)                        \
	if (FAILED(Result))                                                              \
	{                                                                                \
		FText ErrorMessage = FWindowsRWHelpers::CreateErrorMessage(Result, Message); \
		UE_LOGF(LogWindowsReader, Error, "%ls", *ErrorMessage.ToString());  \
		return ErrorMessage;                                                         \
	}

#define WINAR_CHECK_AND_RETURN_ERROR(Result, Message)                                \
	if (FAILED(Result))                                                              \
	{                                                                                \
		FText ErrorMessage = FWindowsRWHelpers::CreateErrorMessage(Result, Message); \
		UE_LOGF(LogWindowsReader, Error, "%ls", *ErrorMessage.ToString());  \
		return MakeError(MoveTemp(ErrorMessage));                                    \
	}

EXTERN_GUID(UE_MF_SOURCE_READER_PASSTHROUGH_MODE, 0x043FF126, 0xFE2C, 0x4708, 0xA0, 0x9B, 0xDA, 0x2A, 0xB4, 0x35, 0xCE, 0xD9);

TUniquePtr<IAudioReader> FWindowsReadersFactory::CreateAudioReader()
{
	return MakeUnique<FWindowsAudioReader>();
}

TUniquePtr<IVideoReader> FWindowsReadersFactory::CreateVideoReader()
{
	return MakeUnique<FWindowsVideoReader>();
}

FWindowsAudioReader::FWindowsAudioReader() = default;
FWindowsAudioReader::~FWindowsAudioReader() = default;

TOptional<FText> FWindowsAudioReader::Open(const FString& InFileName)
{
	TComPtr<IMFMediaType> AudioMediaTypeIn;
	TComPtr<IMFMediaType> AudioMediaTypeOut;

	HRESULT Result = ::MFCreateSourceReaderFromURL(*InFileName, nullptr, &AudioReader);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, FText::Format(LOCTEXT("OpenAudio_FailedToOpenAudioFile", "Cannot open the audio file {0}"),
														 FText::FromString(InFileName)));

	PROPVARIANT DurationVar;
	int64 DurationTicks;	// 10,000,000 == 1.0e7 units per second. Each unit corresponds to 100 nanoseconds.

	Result = AudioReader->GetPresentationAttribute(MF_SOURCE_READER_MEDIASOURCE, MF_PD_DURATION, &DurationVar);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenAudio_FailedToRetrieveDuration", "Failed to retrieve the audio file duration"));

	Result = ::PropVariantToInt64(DurationVar, &DurationTicks);
	PropVariantClear(&DurationVar);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenAudio_FailedToRetrieveDurationValue", "Failed to retrieve the duration value"));

	Duration = FTimespan(DurationTicks);

	Result = AudioReader->SetStreamSelection(MF_SOURCE_READER_ALL_STREAMS, false);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenAudio_FailedToDeselectStreams", "Failed to deselect all streams"));

	Result = AudioReader->SetStreamSelection(MF_SOURCE_READER_FIRST_AUDIO_STREAM, true);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenAudio_FailedToSelectStream", "Failed to select the audio stream"));

	Result = ::MFCreateMediaType(&AudioMediaTypeIn);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenAudio_FailedToCreateMediaType", "Failed to create the media type"));

	Result = AudioMediaTypeIn->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Audio);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenAudio_FailedToConfigureMediaType", "Failed to configure the media type"));

	Result = AudioMediaTypeIn->SetGUID(MF_MT_SUBTYPE, MFAudioFormat_PCM);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenAudio_FailedToConfigureSubMediaType", "Failed to configure the media subtype"));

	Result = AudioMediaTypeIn->SetUINT32(MF_MT_AUDIO_BITS_PER_SAMPLE, 16);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenAudio_FailedToConfigureBitsPerSample", "Failed to configure audio bits per sample"));

	Result = AudioReader->SetCurrentMediaType(MF_SOURCE_READER_FIRST_AUDIO_STREAM, nullptr, AudioMediaTypeIn);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenAudio_FailedToSetMediaType", "Failed to set the media type to the reader"));

	Result = AudioReader->GetCurrentMediaType(MF_SOURCE_READER_FIRST_AUDIO_STREAM, &AudioMediaTypeOut);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenAudio_FailedToGetMediaType", "Failed to get the media type from the reader"));

	WAVEFORMATEX* WaveFormatOut;
	uint32 WaveFormatSize;

	Result = ::MFCreateWaveFormatExFromMFMediaType(AudioMediaTypeOut, &WaveFormatOut, &WaveFormatSize);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenAudio_FailedToConvertToAudio", "Failed to convert to audio from video"));

	ON_SCOPE_EXIT
	{
		::CoTaskMemFree(WaveFormatOut);
	};

	Channels = WaveFormatOut->nChannels;
	SampleRate = UE::CaptureManager::ConvertSampleRate(WaveFormatOut->nSamplesPerSec);
	Format = EMediaAudioSampleFormat::Int16;

	return {};
}

TOptional<FText> FWindowsAudioReader::Close()
{
	AudioReader->Flush(MF_SOURCE_READER_FIRST_AUDIO_STREAM);

	AudioReader.Reset();

	return {};
}

TValueOrError<TUniquePtr<UE::CaptureManager::FMediaAudioSample>, FText> FWindowsAudioReader::Next()
{
	TComPtr<IMFSample> MFSample;
	TComPtr<IMFMediaBuffer> Buffer;
	DWORD Flags;
	LONGLONG TimeStamp;
	LONGLONG WinDuration;

	HRESULT Result = AudioReader->ReadSample(MF_SOURCE_READER_FIRST_AUDIO_STREAM, 0, nullptr, &Flags, &TimeStamp, &MFSample);
	WINAR_CHECK_AND_RETURN_ERROR(Result, LOCTEXT("NextAudio_FailedToObtainSample", "Failed to read the next sample"));

	if (Flags & MF_SOURCE_READERF_ERROR)
	{
		FText ErrorMessage = FText::Format(LOCTEXT("NextAudio_FailedToObtainSampleFlags", "Failed to read the next sample: Flags({0})"), 
										   FText::FromString(FString::Printf(TEXT("%d"), Flags)));
		UE_LOGF(LogWindowsReader, Error, "%ls", *ErrorMessage.ToString());
		
		return MakeError(ErrorMessage);
	}

	if (Flags & MF_SOURCE_READERF_ENDOFSTREAM)
	{
		//	Finished reading the stream
		return MakeValue(nullptr);
	}

	Result = MFSample->GetSampleTime(&TimeStamp);
	WINAR_CHECK_AND_RETURN_ERROR(Result, LOCTEXT("NextAudio_FailedToReadSampleTime", "Failed to read the audio sample time"));

	TUniquePtr<UE::CaptureManager::FMediaAudioSample> CurrentSample = MakeUnique<UE::CaptureManager::FMediaAudioSample>();
	CurrentSample->Time = FTimespan(TimeStamp);
	CurrentSample->Channels = Channels;
	CurrentSample->SampleRate = SampleRate;
	CurrentSample->SampleFormat = Format;

	Result = MFSample->GetSampleDuration(&WinDuration);
	WINAR_CHECK_AND_RETURN_ERROR(Result, LOCTEXT("NextAudio_FailedToReadSampleDuration", "Failed to read the sample duration"));

	CurrentSample->Duration = FTimespan(WinDuration);

	Result = MFSample->ConvertToContiguousBuffer(&Buffer);
	WINAR_CHECK_AND_RETURN_ERROR(Result, LOCTEXT("NextAudio_FailedToReadSampleInfo", "Failed to retrieve audio sample information"));

	uint8* AudioData;
	unsigned long AudioDataSize = 0;

	Result = Buffer->Lock(&AudioData, nullptr, &AudioDataSize);

	ON_SCOPE_EXIT
	{
		Buffer->Unlock();
	};

	if (FAILED(Result))
	{
		FText ErrorMessage = 
			FWindowsRWHelpers::CreateErrorMessage(Result, LOCTEXT("NextAudio_FailedToReadSampleData", "Failed to retrieve audio sample data"));
		return MakeError(ErrorMessage);
	}

	CurrentSample->Frames = AudioDataSize / (CurrentSample->Channels * sizeof(int16));
	CurrentSample->Buffer.SetNum(AudioDataSize);
	FMemory::Memcpy(CurrentSample->Buffer.GetData(), AudioData, AudioDataSize);

	return MakeValue(MoveTemp(CurrentSample));
}

FTimespan FWindowsAudioReader::GetDuration() const
{
	return Duration;
}

EMediaAudioSampleFormat FWindowsAudioReader::GetSampleFormat() const
{
	return Format;
}

UE::CaptureManager::ESampleRate FWindowsAudioReader::GetSampleRate() const
{
	return SampleRate;
}

uint32 FWindowsAudioReader::GetNumChannels() const
{
	return Channels;
}

FWindowsVideoReader::FWindowsVideoReader() = default;
FWindowsVideoReader::~FWindowsVideoReader() = default;

TOptional<FText> FWindowsVideoReader::Open(const FString& InFileName)
{
	TComPtr<IMFAttributes> Attributes;
	HRESULT Result = MFCreateAttributes(&Attributes, 1);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenVideo_FailedToCreateAttributes", "Error while creating attributes"));

	Result = Attributes->SetUINT32(MF_SOURCE_READER_ENABLE_VIDEO_PROCESSING, 1); // Set to true
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenVideo_FailedToEnableVideoProcessing", "Error while enabling video processing"));

	Result = Attributes->SetUINT32(MF_SOURCE_READER_ENABLE_TRANSCODE_ONLY_TRANSFORMS, 1); // Set to true
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenVideo_FailedToEnableTranscodeFunc", "Error while enabling transcoding only functionality"));

	Result = Attributes->SetUINT32(UE_MF_SOURCE_READER_PASSTHROUGH_MODE, 1); // Set to true
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenVideo_FailedToEnablePassthroughMode", "Error while enabling passthrough mode"));

	Result = ::MFCreateSourceReaderFromURL(*InFileName, Attributes, &VideoReader);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, FText::Format(LOCTEXT("OpenVideo_FailedToOpenVideoFile", "Cannot open the video file {0}"),
														 FText::FromString(InFileName)));

	PROPVARIANT DurationVar;
	int64 DurationTicks;	// 10,000,000 == 1.0e7 units per second. Each unit corresponds to 100 nanoseconds.

	Result = VideoReader->GetPresentationAttribute(MF_SOURCE_READER_MEDIASOURCE, MF_PD_DURATION, &DurationVar);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenVideo_FailedToRetrieveDuration", "Failed to retrieve the video file duration"));

	Result = ::PropVariantToInt64(DurationVar, &DurationTicks);
	PropVariantClear(&DurationVar);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenVideo_FailedToRetrieveDurationValue", "Failed to retrieve the duration value"));

	Duration = FTimespan(DurationTicks);
	Result = VideoReader->SetStreamSelection(MF_SOURCE_READER_ALL_STREAMS, false);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenVideo_FailedToDeselectStreams", "Failed to deselect all streams"));

	Result = VideoReader->SetStreamSelection(MF_SOURCE_READER_FIRST_VIDEO_STREAM, true);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenVideo_FailedToSelectStream", "Failed to select the video stream"));

	TComPtr<IMFMediaType> VideoMediaType;

	Result = VideoReader->GetCurrentMediaType(MF_SOURCE_READER_FIRST_VIDEO_STREAM, &VideoMediaType);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenVideo_FailedToGetMediaType", "Failed to get the media type from the reader"));

	UINT32 VideoWidth = 0;
	UINT32 VideoHeight = 0;

	Result = ::MFGetAttributeSize(VideoMediaType, MF_MT_FRAME_SIZE, &VideoWidth, &VideoHeight);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenVideo_FailedToObtainDimensions", "Failed to obtain the video resolution"));

	UINT32 FrameRateNumerator = 0;
	UINT32 FrameRateDenominator = 0;

	Result = ::MFGetAttributeRatio(VideoMediaType, MF_MT_FRAME_RATE, &FrameRateNumerator, &FrameRateDenominator);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenVideo_FailedToObtainFrameRate", "Failed to obtain the video frame rate"));

	Result = VideoMediaType->GetGUID(MF_MT_SUBTYPE, &InputVideoSubType);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenVideo_FailedToObtainMediaSubtype", "Failed to obtain the video subtype"));

	GUID NewSubType;

	// We support only what Windows Media Foundation supports
	if (InputVideoSubType == MFVideoFormat_H264)
	{
		// Other supported PixelFormats are I420 (IYUV), NV12, YUY2, YV12
		NewSubType = MFVideoFormat_I420;
		PixelFormat = UE::CaptureManager::EMediaTexturePixelFormat::U8_I420;
	}
	else if (InputVideoSubType == MFVideoFormat_H265 ||
			 InputVideoSubType == MFVideoFormat_HEVC)
	{
		// Other supported PixelFormats are P010
		NewSubType = MFVideoFormat_NV12;
		PixelFormat = UE::CaptureManager::EMediaTexturePixelFormat::U8_NV12;
	}
	else if (InputVideoSubType == MFVideoFormat_MJPG)
	{
		NewSubType = InputVideoSubType;
	}
	else
	{
		WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(-1, LOCTEXT("OpenVideo_UnsupportedFormatPassed", "Unsupported format detected"));
	}

	// Decode to a new type
	Result = VideoMediaType->SetGUID(MF_MT_SUBTYPE, NewSubType);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenVideo_FailedToSetMediaSubType", "Failed to set the media sub type to the reader"));

	Result = VideoReader->SetCurrentMediaType(MF_SOURCE_READER_FIRST_VIDEO_STREAM, nullptr, VideoMediaType);
	WINAR_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("OpenVideo_FailedToSetMediaType", "Failed to set the media type to the reader"));

	Dimensions = FIntPoint(VideoWidth, VideoHeight);
	FrameRate = FFrameRate(FrameRateNumerator, FrameRateDenominator);

	return {};
}

TOptional<FText> FWindowsVideoReader::Close()
{
	VideoReader->Flush(MF_SOURCE_READER_FIRST_VIDEO_STREAM);

	VideoReader.Reset();

	return {};
}

TValueOrError<TUniquePtr<UE::CaptureManager::FMediaTextureSample>, FText> FWindowsVideoReader::Next()
{
	TComPtr<IMFSample> MFSample;
	TComPtr<IMFMediaBuffer> Buffer;
	DWORD Flags;
	LONGLONG TimeStamp;
	LONGLONG WinDuration;

	HRESULT Result = VideoReader->ReadSample(MF_SOURCE_READER_FIRST_VIDEO_STREAM, 0, nullptr, &Flags, &TimeStamp, &MFSample);
	WINAR_CHECK_AND_RETURN_ERROR(Result, LOCTEXT("NextVideo_FailedToObtainSample", "Failed to read the next sample"));

	if (Flags & MF_SOURCE_READERF_ERROR)
	{
		FText ErrorMessage = FText::Format(LOCTEXT("NextVideo_FailedToObtainSampleFlags", "Failed to read the next sample: Flags({0})"),
										   FText::FromString(FString::Printf(TEXT("%d"), Flags)));
		UE_LOGF(LogWindowsReader, Error, "%ls", *ErrorMessage.ToString());

		return MakeError(ErrorMessage);
	}

	if (Flags & MF_SOURCE_READERF_ENDOFSTREAM)
	{
		//	Finished reading the stream
		return MakeValue(nullptr);
	}

	Result = MFSample->GetSampleTime(&TimeStamp);
	WINAR_CHECK_AND_RETURN_ERROR(Result, LOCTEXT("NextVideo_FailedToReadSampleTime", "Failed to read the video sample time"));

	TUniquePtr<UE::CaptureManager::FMediaTextureSample> CurrentSample = MakeUnique<UE::CaptureManager::FMediaTextureSample>();
	CurrentSample->Time = FTimespan(TimeStamp);
	CurrentSample->Stride = Dimensions.X * UE::CaptureManager::GetNumberOfChannels(PixelFormat);
	CurrentSample->Dimensions = Dimensions;
	CurrentSample->DesiredFormat = UE::CaptureManager::EMediaTexturePixelFormat::Undefined;

	Result = MFSample->GetSampleDuration(&WinDuration);
	WINAR_CHECK_AND_RETURN_ERROR(Result, LOCTEXT("NextVideo_FailedToReadSampleDuration", "Failed to read the sample duration"));

	CurrentSample->Duration = FTimespan(WinDuration);

	Result = MFSample->ConvertToContiguousBuffer(&Buffer);
	WINAR_CHECK_AND_RETURN_ERROR(Result, LOCTEXT("NextVideo_FailedToReadSampleInfo", "Failed to retrieve video sample information"));

	DWORD BufferSize = 0;
	DWORD BufferMaxSize = 0;
	uint8* BitmapData = nullptr;

	Result = Buffer->Lock(&BitmapData, &BufferMaxSize, &BufferSize);

	ON_SCOPE_EXIT
	{
		Buffer->Unlock();
	};

	if (FAILED(Result))
	{
		FText ErrorMessage =
			FWindowsRWHelpers::CreateErrorMessage(Result, LOCTEXT("NextVideo_FailedToReadSampleData", "Failed to retrieve video sample data"));
		return MakeError(ErrorMessage);
	}

	if (InputVideoSubType == MFVideoFormat_MJPG)
	{
		TValueOrError<FWindowsJpegDecoder, FText> CreateDecoderError = FWindowsJpegDecoder::CreateJpegDecoder();

		if (CreateDecoderError.HasError())
		{
			return MakeError(CreateDecoderError.StealError());
		}

		FWindowsJpegDecoder Decoder = CreateDecoderError.StealValue();

		TOptional<FText> DecodeError = Decoder.Decode(BitmapData, BufferSize, CurrentSample->Buffer, CurrentSample->CurrentFormat);
		if (DecodeError.IsSet())
		{
			return MakeError(MoveTemp(DecodeError.GetValue()));
		}

		CurrentSample->Stride = Dimensions.X * UE::CaptureManager::GetNumberOfChannels(CurrentSample->CurrentFormat);
	}
	else
	{
		UINT32 DefaultStride = 0;
		TComPtr<IMFMediaType> OutType;
		if (SUCCEEDED(VideoReader->GetCurrentMediaType(MF_SOURCE_READER_FIRST_VIDEO_STREAM, &OutType)) &&
			SUCCEEDED(OutType->GetUINT32(MF_MT_DEFAULT_STRIDE, &DefaultStride)) && 
			DefaultStride != 0)
		{
			CurrentSample->Stride = FMath::Abs(static_cast<int32>(DefaultStride)); // In Bytes
		}

		CurrentSample->CurrentFormat = PixelFormat;
		CurrentSample->Buffer.SetNum(BufferSize);
		FMemory::Memcpy(CurrentSample->Buffer.GetData(), BitmapData, BufferSize);
	}

	return MakeValue(MoveTemp(CurrentSample));
}

FTimespan FWindowsVideoReader::GetDuration() const
{
	return Duration;
}

FIntPoint FWindowsVideoReader::GetDimensions() const
{
	return Dimensions;
}

FFrameRate FWindowsVideoReader::GetFrameRate() const
{
	return FrameRate;
}

#undef LOCTEXT_NAMESPACE

#endif // PLATFORM_WINDOWS && !UE_SERVER