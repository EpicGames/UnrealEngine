// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"
#include "HAL/Platform.h"

#if PLATFORM_WINDOWS && !UE_SERVER

#include "Windows/WindowsHWrapper.h"
#include "Windows/AllowWindowsPlatformTypes.h"

THIRD_PARTY_INCLUDES_START

#include <windows.h>
#include <mfapi.h>
#include <mfidl.h>
#include <mfreadwrite.h>
#include <propvarutil.h>
#include "Microsoft/COMPointer.h"

THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformTypes.h"

#include "IMediaReader.h"
#include "IMediaRWFactory.h"

class FWindowsReadersFactory final : 
	public IAudioReaderFactory, 
	public IVideoReaderFactory
{
public:

	virtual TUniquePtr<IAudioReader> CreateAudioReader() override;
	virtual TUniquePtr<IVideoReader> CreateVideoReader() override;
};

class FWindowsAudioReader final : public IAudioReader
{
public:

	FWindowsAudioReader();
	virtual ~FWindowsAudioReader() override;

	virtual TOptional<FText> Open(const FString& InFileName) override;
	virtual TOptional<FText> Close() override;

	virtual TValueOrError<TUniquePtr<UE::CaptureManager::FMediaAudioSample>, FText> Next() override;

	virtual FTimespan GetDuration() const override;
	virtual EMediaAudioSampleFormat GetSampleFormat() const override;
	virtual UE::CaptureManager::ESampleRate GetSampleRate() const override;
	virtual uint32 GetNumChannels() const override;

private:

	TComPtr<IMFSourceReader> AudioReader;

	FTimespan Duration;
	EMediaAudioSampleFormat Format;
	UE::CaptureManager::ESampleRate SampleRate;
	uint32 Channels;
};

class FWindowsVideoReader final : public IVideoReader
{
public:

	FWindowsVideoReader();
	virtual ~FWindowsVideoReader() override;

	virtual TOptional<FText> Open(const FString& InFileName) override;
	virtual TOptional<FText> Close() override;

	virtual TValueOrError<TUniquePtr<UE::CaptureManager::FMediaTextureSample>, FText> Next() override;

	virtual FTimespan GetDuration() const override;
	virtual FIntPoint GetDimensions() const override;
	virtual FFrameRate GetFrameRate() const override;

private:

	TComPtr<IMFSourceReader> VideoReader;

	FTimespan Duration;
	FIntPoint Dimensions;
	FFrameRate FrameRate;
	UE::CaptureManager::EMediaTexturePixelFormat PixelFormat;
	GUID InputVideoSubType;
};

#endif // PLATFORM_WINDOWS && !UE_SERVER
