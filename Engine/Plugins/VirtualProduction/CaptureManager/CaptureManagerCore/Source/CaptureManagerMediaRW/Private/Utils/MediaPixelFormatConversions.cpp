// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/MediaPixelFormatConversions.h"

#include "Async/ParallelFor.h"

namespace UE::CaptureManager
{

namespace Private
{
template<typename T>
static T ScalePixel(T InPixel, TRange<T> InInputRange, TRange<T> InOutputRange)
{
	const float Factor = static_cast<float>(InOutputRange.GetUpperBoundValue()) / (InInputRange.GetUpperBoundValue() - InInputRange.GetLowerBoundValue());
	const uint64 NewPixelValue = Factor * (InPixel - InInputRange.GetLowerBoundValue()) + 0.5;
	return FMath::Clamp(NewPixelValue, InOutputRange.GetLowerBoundValue(), InOutputRange.GetUpperBoundValue());
}

static inline void RGBfromYUV(uint8& R, uint8& G, uint8& B, uint8 Y, uint8 U, uint8 V)
{
	int64 BigY = Y - 16;
	int64 BigU = U - 128;
	int64 BigV = V - 128;

	int64 BigR = (298 * BigY + 409 * BigV + 128) >> 8;
	int64 BigG = (298 * BigY - 100 * BigU - 208 * BigV + 128) >> 8;
	int64 BigB = (298 * BigY + 516 * BigU + 128) >> 8;

	R = FMath::Clamp(BigR, 0, 255);
	G = FMath::Clamp(BigG, 0, 255);
	B = FMath::Clamp(BigB, 0, 255);
}

static inline FColor CreateColor(uint8 R, uint8 G, uint8 B, uint8 A)
{
	return FColor(R, G, B, A);
}

}

TArray<uint8> ConvertYUVToMono(const UE::CaptureManager::FMediaTextureSample* InSample, bool InScaleRange)
{
	using namespace UE::CaptureManager;

	TArray<uint8> NewBuffer;
	NewBuffer.SetNumUninitialized(InSample->Dimensions.X * InSample->Dimensions.Y);

	uint32 StrideY = InSample->Stride;
	int32 MonoStride = InSample->Dimensions.X;

	ParallelFor(InSample->Dimensions.Y, [&NewBuffer, StrideY, MonoStride, InScaleRange, InSample](int32 Y) mutable
	{
		for (int32 X = 0; X < InSample->Dimensions.X; ++X)
		{
			int32 YIndex = Y * StrideY + X;
			uint8 PixelValue = InSample->Buffer[YIndex];

			if (InScaleRange)
			{
				using FRange = TRange<uint8>;
				PixelValue = Private::ScalePixel(PixelValue,
												 FRange::Inclusive(16, 235),
												 FRange::Inclusive(MIN_uint8, MAX_uint8));
			}

			NewBuffer[Y * MonoStride + X] = PixelValue;
		}
	});

	return NewBuffer;
}

TArray<uint8> ConvertYUY2ToMono(const UE::CaptureManager::FMediaTextureSample* InSample, bool InScaleRange)
{
	using namespace UE::CaptureManager;

	TArray<uint8> NewBuffer;
	NewBuffer.SetNumUninitialized(InSample->Dimensions.X * InSample->Dimensions.Y);

	int32 YUY2Stride = InSample->Stride;
	int32 MonoStride = InSample->Dimensions.X;

	ParallelFor(InSample->Dimensions.Y, [&NewBuffer, YUY2Stride, MonoStride, InScaleRange, InSample](int32 Y) mutable
	{
		for (int32 X = 0; X < InSample->Dimensions.X; X += 2)
		{
			int32 YIndex = Y * YUY2Stride + X * 2;

			uint8_t Y0 = InSample->Buffer[YIndex];
			uint8_t Y1 = InSample->Buffer[YIndex + 2];

			if (InScaleRange)
			{
				using FRange = TRange<uint8>;
				Y0 = Private::ScalePixel(Y0,
										 FRange::Inclusive(16, 235),
										 FRange::Inclusive(MIN_uint8, MAX_uint8));
				Y1 = Private::ScalePixel(Y1,
										 FRange::Inclusive(16, 235),
										 FRange::Inclusive(MIN_uint8, MAX_uint8));

			}

			NewBuffer[Y * MonoStride + X] = Y0;
			NewBuffer[Y * MonoStride + X + 1] = Y1;
		}
	});

	return NewBuffer;
}

TArray<uint8> ConvertI420ToBGRA(const UE::CaptureManager::FMediaTextureSample* InSample)
{
	using namespace UE::CaptureManager;

	EMediaTexturePixelFormat OutputFormat = EMediaTexturePixelFormat::U8_BGRA;
	uint32 Channels = GetNumberOfChannels(OutputFormat);

	TArray<uint8> NewBuffer;
	NewBuffer.SetNumUninitialized(InSample->Dimensions.X * InSample->Dimensions.Y * Channels);

	uint32 StrideY = InSample->Stride;
	uint32 StrideUV = InSample->Stride / 2;

	int32 BGRAStride = InSample->Dimensions.X * Channels;

	int32 BufferHeight = (InSample->Buffer.Num() * 2) / (InSample->Stride * 3);

	uint32 YPlaneSize = StrideY * BufferHeight;
	uint32 UPlaneSize = StrideUV * (BufferHeight / 2);
	uint32 VPlaneSize = InSample->Buffer.Num() - (YPlaneSize + UPlaneSize);

	const TArrayView<const uint8> YPlane = MakeArrayView(InSample->Buffer.GetData(), YPlaneSize);
	const TArrayView<const uint8> UPlane = MakeArrayView(InSample->Buffer.GetData() + YPlaneSize, UPlaneSize);
	const TArrayView<const uint8> VPlane = MakeArrayView(InSample->Buffer.GetData() + YPlaneSize + UPlaneSize, VPlaneSize);

	ParallelFor(InSample->Dimensions.Y, [&NewBuffer, YPlane, UPlane, VPlane, StrideY, StrideUV, BGRAStride, InSample](int32 Y) mutable
	{
		for (int32 X = 0; X < InSample->Dimensions.X; ++X)
		{
			int32 YIndex = Y * StrideY + X;
			int32 UVIndex = (Y / 2) * StrideUV + (X / 2);

			uint8_t Y0 = YPlane[YIndex];
			uint8_t U = UPlane[UVIndex];
			uint8_t V = VPlane[UVIndex];

			uint8 R, G, B;
			Private::RGBfromYUV(R, G, B, Y0, U, V);

			NewBuffer[Y * BGRAStride + X * 4] = B;
			NewBuffer[Y * BGRAStride + X * 4 + 1] = G;
			NewBuffer[Y * BGRAStride + X * 4 + 2] = R;
			NewBuffer[Y * BGRAStride + X * 4 + 3] = 255;
		}
	});

	return NewBuffer;
}

TArray<uint8> ConvertNV12ToBGRA(const UE::CaptureManager::FMediaTextureSample* InSample)
{
	using namespace UE::CaptureManager;

	EMediaTexturePixelFormat OutputFormat = EMediaTexturePixelFormat::U8_BGRA;
	uint32 Channels = GetNumberOfChannels(OutputFormat);

	TArray<uint8> NewBuffer;
	NewBuffer.SetNumUninitialized(InSample->Dimensions.X * InSample->Dimensions.Y * Channels);

	uint32 StrideY = InSample->Stride;

	int32 BGRAStride = InSample->Dimensions.X * Channels;

	int32 BufferHeight = (InSample->Buffer.Num() * 2) / (InSample->Stride * 3);

	uint32 YPlaneSize = StrideY * BufferHeight;
	uint32 UVPlaneSize = InSample->Buffer.Num() - YPlaneSize;

	const TArrayView<const uint8> YPlane = MakeArrayView(InSample->Buffer.GetData(), YPlaneSize);
	const TArrayView<const uint8> UVPlane = MakeArrayView(InSample->Buffer.GetData() + YPlaneSize, UVPlaneSize);

	ParallelFor(InSample->Dimensions.Y, [&NewBuffer, YPlane, UVPlane, StrideY, BGRAStride, InSample, Channels](int32 Y) mutable
	{
		for (int32 X = 0; X < InSample->Dimensions.X; ++X)
		{
			int32 YIndex = Y * StrideY + X;
			int32 UVIndex = (Y / 2) * StrideY + (X & ~1);

			uint8_t Y0 = YPlane[YIndex];
			uint8_t U = UVPlane[UVIndex];
			uint8_t V = UVPlane[UVIndex + 1];

			uint8 R, G, B;
			Private::RGBfromYUV(R, G, B, Y0, U, V);

			NewBuffer[Y * BGRAStride + X * 4] = B;
			NewBuffer[Y * BGRAStride + X * 4 + 1] = G;
			NewBuffer[Y * BGRAStride + X * 4 + 2] = R;
			NewBuffer[Y * BGRAStride + X * 4 + 3] = 255;
		}
	});

	return NewBuffer;
}

TArray<uint8> ConvertYUY2ToBGRA(const UE::CaptureManager::FMediaTextureSample* InSample)
{
	using namespace UE::CaptureManager;

	EMediaTexturePixelFormat OutputFormat = EMediaTexturePixelFormat::U8_BGRA;

	uint32 Channels = GetNumberOfChannels(OutputFormat);

	TArray<uint8> NewBuffer;
	NewBuffer.SetNumUninitialized(InSample->Dimensions.X * InSample->Dimensions.Y * Channels);

	int32 YUY2Stride = InSample->Stride;
	int32 BGRAStride = InSample->Dimensions.X * Channels;

	ParallelFor(InSample->Dimensions.Y, [&NewBuffer, YUY2Stride, BGRAStride, InSample](int32 Y) mutable
	{
		for (int32 X = 0; X < InSample->Dimensions.X; X += 2)
		{
			int32 YIndex = Y * YUY2Stride + X * 2;

			uint8 Y0 = InSample->Buffer[YIndex];
			uint8 U = InSample->Buffer[YIndex + 1];
			uint8 Y1 = InSample->Buffer[YIndex + 2];
			uint8 V = InSample->Buffer[YIndex + 3];

			uint8 R0, G0, B0;
			Private::RGBfromYUV(R0, G0, B0, Y0, U, V);

			uint8 R1, G1, B1;
			Private::RGBfromYUV(R1, G1, B1, Y1, U, V);

			NewBuffer[Y * BGRAStride + X * 4] = B0;
			NewBuffer[Y * BGRAStride + X * 4 + 1] = G0;
			NewBuffer[Y * BGRAStride + X * 4 + 2] = R0;
			NewBuffer[Y * BGRAStride + X * 4 + 3] = 255;

			NewBuffer[Y * BGRAStride + (X + 1) * 4] = B1;
			NewBuffer[Y * BGRAStride + (X + 1) * 4 + 1] = G1;
			NewBuffer[Y * BGRAStride + (X + 1) * 4 + 2] = R1;
			NewBuffer[Y * BGRAStride + (X + 1) * 4 + 3] = 255;
		}
	});

	return NewBuffer;
}

TArray<FColor> UEConvertYUVToMono(const UE::CaptureManager::FMediaTextureSample* InSample, bool InScaleRange)
{
	using namespace UE::CaptureManager;

	TArray<FColor> NewBuffer;
	NewBuffer.SetNumUninitialized(InSample->Dimensions.X * InSample->Dimensions.Y);

	uint32 StrideY = InSample->Stride;
	int32 MonoStride = InSample->Dimensions.X;

	ParallelFor(InSample->Dimensions.Y, [&NewBuffer, StrideY, MonoStride, InScaleRange, InSample](int32 Y) mutable
	{
		for (int32 X = 0; X < InSample->Dimensions.X; ++X)
		{
			int32 YIndex = Y * StrideY + X;
			uint8 PixelValue = InSample->Buffer[YIndex];

			if (InScaleRange)
			{
				using FRange = TRange<uint8>;
				PixelValue = Private::ScalePixel(PixelValue,
												 FRange::Inclusive(16, 235),
												 FRange::Inclusive(MIN_uint8, MAX_uint8));
			}


			NewBuffer[Y * MonoStride + X] = 
				Private::CreateColor(PixelValue, PixelValue, PixelValue, InScaleRange ? 235 : 255);
		}
	});

	return NewBuffer;
}

TArray<FColor> UEConvertYUY2ToMono(const UE::CaptureManager::FMediaTextureSample* InSample, bool InScaleRange)
{
	using namespace UE::CaptureManager;

	TArray<FColor> NewBuffer;
	NewBuffer.SetNumUninitialized(InSample->Dimensions.X * InSample->Dimensions.Y);

	int32 YUY2Stride = InSample->Stride;
	int32 MonoStride = InSample->Dimensions.X;

	ParallelFor(InSample->Dimensions.Y, [&NewBuffer, YUY2Stride, MonoStride, InScaleRange, InSample](int32 Y) mutable
	{
		for (int32 X = 0; X < InSample->Dimensions.X; X += 2)
		{
			int32 YIndex = Y * YUY2Stride + X * 2;

			uint8_t Y0 = InSample->Buffer[YIndex];
			uint8_t Y1 = InSample->Buffer[YIndex + 2];

			if (InScaleRange)
			{
				using FRange = TRange<uint8>;
				Y0 = Private::ScalePixel(Y0,
										 FRange::Inclusive(16, 235),
										 FRange::Inclusive(MIN_uint8, MAX_uint8));
				Y1 = Private::ScalePixel(Y1,
										 FRange::Inclusive(16, 235),
										 FRange::Inclusive(MIN_uint8, MAX_uint8));

			}

			NewBuffer[Y * MonoStride + X] = 
				Private::CreateColor(Y0, Y0, Y0, InScaleRange ? 235 : 255);
			NewBuffer[Y * MonoStride + X + 1] = 
				Private::CreateColor(Y1, Y1, Y1, InScaleRange ? 235 : 255);
		}
	});

	return NewBuffer;
}

TArray<FColor> UEConvertI420ToBGRA(const UE::CaptureManager::FMediaTextureSample* InSample)
{
	using namespace UE::CaptureManager;

	EMediaTexturePixelFormat OutputFormat = EMediaTexturePixelFormat::U8_BGRA;

	TArray<FColor> NewBuffer;
	NewBuffer.SetNumUninitialized(InSample->Dimensions.X * InSample->Dimensions.Y);

	uint32 StrideY = InSample->Stride;
	uint32 StrideUV = InSample->Stride / 2;

	int32 BGRAStride = InSample->Dimensions.X;

	int32 BufferHeight = (InSample->Buffer.Num() * 2) / (InSample->Stride * 3);

	uint32 YPlaneSize = StrideY * BufferHeight;
	uint32 UPlaneSize = StrideUV * (BufferHeight / 2);
	uint32 VPlaneSize = InSample->Buffer.Num() - (YPlaneSize + UPlaneSize);

	const TArrayView<const uint8> YPlane = MakeArrayView(InSample->Buffer.GetData(), YPlaneSize);
	const TArrayView<const uint8> UPlane = MakeArrayView(InSample->Buffer.GetData() + YPlaneSize, UPlaneSize);
	const TArrayView<const uint8> VPlane = MakeArrayView(InSample->Buffer.GetData() + YPlaneSize + UPlaneSize, VPlaneSize);

	ParallelFor(InSample->Dimensions.Y, [&NewBuffer, YPlane, UPlane, VPlane, StrideY, StrideUV, BGRAStride, InSample](int32 Y) mutable
	{
		for (int32 X = 0; X < InSample->Dimensions.X; ++X)
		{
			int32 YIndex = Y * StrideY + X;
			int32 UVIndex = (Y / 2) * StrideUV + (X / 2);

			uint8_t Y0 = YPlane[YIndex];
			uint8_t U = UPlane[UVIndex];
			uint8_t V = VPlane[UVIndex];

			uint8 R, G, B;
			Private::RGBfromYUV(R, G, B, Y0, U, V);

			NewBuffer[Y * BGRAStride + X] = Private::CreateColor(R, G, B, 255);
		}
	});

	return NewBuffer;
}

TArray<FColor> UEConvertNV12ToBGRA(const UE::CaptureManager::FMediaTextureSample* InSample)
{
	using namespace UE::CaptureManager;

	EMediaTexturePixelFormat OutputFormat = EMediaTexturePixelFormat::U8_BGRA;

	TArray<FColor> NewBuffer;
	NewBuffer.SetNumUninitialized(InSample->Dimensions.X * InSample->Dimensions.Y);

	uint32 StrideY = InSample->Stride;

	int32 BGRAStride = InSample->Dimensions.X;

	int32 BufferHeight = (InSample->Buffer.Num() * 2) / (InSample->Stride * 3);

	uint32 YPlaneSize = StrideY * BufferHeight;
	uint32 UVPlaneSize = InSample->Buffer.Num() - YPlaneSize;

	const TArrayView<const uint8> YPlane = MakeArrayView(InSample->Buffer.GetData(), YPlaneSize);
	const TArrayView<const uint8> UVPlane = MakeArrayView(InSample->Buffer.GetData() + YPlaneSize, UVPlaneSize);

	ParallelFor(InSample->Dimensions.Y, [&NewBuffer, YPlane, UVPlane, StrideY, BGRAStride, InSample](int32 Y) mutable
	{
		for (int32 X = 0; X < InSample->Dimensions.X; ++X)
		{
			int32 YIndex = Y * StrideY + X;
			int32 UVIndex = (Y / 2) * StrideY + (X & ~1);

			uint8_t Y0 = YPlane[YIndex];
			uint8_t U = UVPlane[UVIndex];
			uint8_t V = UVPlane[UVIndex + 1];

			uint8 R, G, B;
			Private::RGBfromYUV(R, G, B, Y0, U, V);

			NewBuffer[Y * BGRAStride + X] = Private::CreateColor(R, G, B, 255);
		}
	});

	return NewBuffer;
}

TArray<FColor> UEConvertYUY2ToBGRA(const UE::CaptureManager::FMediaTextureSample* InSample)
{
	using namespace UE::CaptureManager;

	EMediaTexturePixelFormat OutputFormat = EMediaTexturePixelFormat::U8_BGRA;

	TArray<FColor> NewBuffer;
	NewBuffer.SetNumUninitialized(InSample->Dimensions.X * InSample->Dimensions.Y);

	int32 YUY2Stride = InSample->Stride;

	int32 BGRAStride = InSample->Dimensions.X;

	ParallelFor(InSample->Dimensions.Y, [&NewBuffer, YUY2Stride, BGRAStride, InSample](int32 Y) mutable
	{
		for (int32 X = 0; X < InSample->Dimensions.X; X += 2)
		{
			int32 YIndex = Y * YUY2Stride + X * 2;

			uint8 Y0 = InSample->Buffer[YIndex];
			uint8 U = InSample->Buffer[YIndex + 1];
			uint8 Y1 = InSample->Buffer[YIndex + 2];
			uint8 V = InSample->Buffer[YIndex + 3];

			uint8 R0, G0, B0;
			Private::RGBfromYUV(R0, G0, B0, Y0, U, V);

			uint8 R1, G1, B1;
			Private::RGBfromYUV(R1, G1, B1, Y1, U, V);

			NewBuffer[Y * BGRAStride + X] = Private::CreateColor(R0, G0, B0, 255);
			NewBuffer[Y * BGRAStride + (X + 1)] = Private::CreateColor(R1, G1, B1, 255);
		}
	});

	return NewBuffer;
}

}