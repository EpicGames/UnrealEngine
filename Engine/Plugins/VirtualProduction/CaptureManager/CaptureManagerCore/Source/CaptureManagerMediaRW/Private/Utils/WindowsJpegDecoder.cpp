// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsJpegDecoder.h"
#include "WindowsRWHelpers.h"

#if PLATFORM_WINDOWS && !UE_SERVER

#define LOCTEXT_NAMESPACE "WindowsJpegDecoder"

DEFINE_LOG_CATEGORY_STATIC(LogWindowsJpegDecoder, Log, All);

#define WINJD_CHECK_AND_RETURN_ERROR_MESSAGE(Result, Message)                        \
	if (FAILED(Result))                                                              \
	{                                                                                \
		FText ErrorMessage = FWindowsRWHelpers::CreateErrorMessage(Result, Message); \
		UE_LOGF(LogWindowsJpegDecoder, Error, "%ls", *ErrorMessage.ToString());  \
		return ErrorMessage;														 \
	}


TValueOrError<FWindowsJpegDecoder, FText> FWindowsJpegDecoder::CreateJpegDecoder()
{
	FWindowsJpegDecoder Decoder;

	TOptional<FText> InitializeResult = Decoder.Initialize();

	if (InitializeResult.IsSet())
	{
		return MakeError(MoveTemp(InitializeResult.GetValue()));
	}

	return MakeValue(MoveTemp(Decoder));
}

FWindowsJpegDecoder::FWindowsJpegDecoder() = default;

TOptional<FText> FWindowsJpegDecoder::Initialize()
{
	HRESULT Result = CoCreateInstance(CLSID_WICImagingFactory, nullptr, CLSCTX_INPROC_SERVER, IID_PPV_ARGS(&WindowsImagingFactory));
	WINJD_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Constructor_FailedToInitialize", "Failed to initialize Windows Imaging Component"));

	return {};
}

TOptional<FText> FWindowsJpegDecoder::Decode(uint8* InData, uint32 InSize, TArray<uint8>& OutImage, UE::CaptureManager::EMediaTexturePixelFormat& OutPixelFormat)
{
	TComPtr<IWICBitmapDecoder> Decoder;
	HRESULT Result = WindowsImagingFactory->CreateDecoder(GUID_ContainerFormatJpeg, nullptr, &Decoder);
	WINJD_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Decode_FailedToCreateDecoder", "Failed to create the JPEG decoder"));

	TComPtr<IWICStream> DecoderStream;
	Result = WindowsImagingFactory->CreateStream(&DecoderStream);
	WINJD_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Decode_FailedToCreateStream", "Failed to create the decoder stream"));

	Result = DecoderStream->InitializeFromMemory(InData, InSize);
	WINJD_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Decode_FailedToSetBuffer", "Failed to initialize the stream from buffer"));

	Result = Decoder->Initialize(DecoderStream.Get(), WICDecodeMetadataCacheOnLoad);
	WINJD_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Decode_FailedToSetStream", "Failed to initialize the decoder from stream"));

	uint32 FrameIndex = 0; // Only one frame is decoded
	TComPtr<IWICBitmapFrameDecode> DecodedFrame;
	Result = Decoder->GetFrame(FrameIndex, &DecodedFrame);
	WINJD_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Decode_FailedToDecode", "Failed to decode the jpeg frame"));

	uint32 Width = 0;
	uint32 Height = 0;

	DecodedFrame->GetSize(&Width, &Height);
	WICRect Lock = { 0, 0, Width, Height };

	WICPixelFormatGUID PixelFormat;
	Result = DecodedFrame->GetPixelFormat(&PixelFormat);
	WINJD_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Decode_FailedObtainPixelFormat", "Failed to obtain the pixel format"));

	OutPixelFormat = ConvertPixelFormat(PixelFormat);

	uint32 BytesPerPixel = GetNumberOfChannels(OutPixelFormat);
	uint32 Stride = (Width * BytesPerPixel);

	OutImage.SetNum(Stride * Height);

	Result = DecodedFrame->CopyPixels(&Lock, Stride, OutImage.Num(), OutImage.GetData());
	WINJD_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Decode_FailedCopyPixels", "Failed to obtain the pixels from the decoded image"));

	return {};
}

UE::CaptureManager::EMediaTexturePixelFormat FWindowsJpegDecoder::ConvertPixelFormat(WICPixelFormatGUID InPixelFormat)
{
	using namespace UE::CaptureManager;

	if (InPixelFormat == GUID_WICPixelFormat8bppGray)
	{
		return EMediaTexturePixelFormat::U8_Mono;
	}
	else if (InPixelFormat == GUID_WICPixelFormat16bppGray)
	{
		return EMediaTexturePixelFormat::U16_Mono;
	}
	else if (InPixelFormat == GUID_WICPixelFormat24bppBGR)
	{
		return EMediaTexturePixelFormat::U8_BGR;
	}
	else if (InPixelFormat == GUID_WICPixelFormat24bppRGB)
	{
		return EMediaTexturePixelFormat::U8_RGB;
	}
	else if (InPixelFormat == GUID_WICPixelFormat32bppBGR || InPixelFormat == GUID_WICPixelFormat32bppBGRA)
	{
		return EMediaTexturePixelFormat::U8_BGRA;
	}
	else if (InPixelFormat == GUID_WICPixelFormat32bppGrayFloat)
	{
		return EMediaTexturePixelFormat::F_Mono;
	}
	else if (InPixelFormat == GUID_WICPixelFormat32bppRGBA)
	{
		return EMediaTexturePixelFormat::U8_RGBA;
	}
	else
	{
		return EMediaTexturePixelFormat::Undefined;
	}
}

#undef LOCTEXT_NAMESPACE

#endif