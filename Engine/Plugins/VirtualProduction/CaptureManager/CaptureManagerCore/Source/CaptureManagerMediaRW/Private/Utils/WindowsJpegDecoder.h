// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"
#include "HAL/Platform.h"

#if PLATFORM_WINDOWS && !UE_SERVER

#include "Windows/WindowsHWrapper.h"
#include "Windows/AllowWindowsPlatformTypes.h"

THIRD_PARTY_INCLUDES_START

#include <windows.h>
#include <mfapi.h>
#include <mfidl.h>
#include <wincodec.h>
#include "Microsoft/COMPointer.h"

THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformTypes.h"

#include "MediaSample.h"

class FWindowsJpegDecoder
{
public:

	static TValueOrError<FWindowsJpegDecoder, FText> CreateJpegDecoder();

	TOptional<FText> Decode(uint8* InData, uint32 InSize, TArray<uint8>& OutImage, UE::CaptureManager::EMediaTexturePixelFormat& OutPixelFormat);

private:

	FWindowsJpegDecoder();
	TOptional<FText> Initialize();

	static UE::CaptureManager::EMediaTexturePixelFormat ConvertPixelFormat(WICPixelFormatGUID InPixelFormat);

	TComPtr<IWICImagingFactory> WindowsImagingFactory;
};

#endif