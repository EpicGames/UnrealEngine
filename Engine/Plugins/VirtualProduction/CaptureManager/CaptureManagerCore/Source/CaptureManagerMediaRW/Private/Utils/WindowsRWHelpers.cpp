// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsRWHelpers.h"

#include "MediaRWManager.h"

#if PLATFORM_WINDOWS && !UE_SERVER

DEFINE_LOG_CATEGORY_STATIC(LogWindowsRWHelper, Log, All);

bool FWindowsRWHelpers::bInitialized = false;
bool FWindowsRWHelpers::bOwnsComInit = false;

bool FWindowsRWHelpers::Init()
{
	if (bInitialized)
	{
		return true;
	}

	// The WMF DLLs are delay-loaded by this module (see CaptureManagerMediaRW.Build.cs).
	// If the DLL is completely absent, calling MFStartup would trigger a delay-load
	// failure (structured exception), so we probe first to avoid that. On some Windows
	// configurations (e.g. Server Core) the DLL is present but non-functional - that
	// case is handled below when MFStartup returns E_NOTIMPL.
	HMODULE MfplatHandle = LoadLibraryW(L"mfplat.dll");
	if (!MfplatHandle)
	{
		return false;
	}
	FreeLibrary(MfplatHandle);

	HRESULT Result = CoInitializeEx(nullptr, COINIT_APARTMENTTHREADED);

	// RPC_E_CHANGED_MODE means COM is already initialized on this thread with a
	// different threading model (typically COINIT_MULTITHREADED by the engine).
	// COM is still usable in that case, but we did not increment the ref count
	// and must not call CoUninitialize during shutdown.
	bOwnsComInit = (Result != RPC_E_CHANGED_MODE);

	if (!SUCCEEDED(Result) && Result != RPC_E_CHANGED_MODE)
	{
		FString Message = FormatWindowsMessage(Result);
		UE_LOGF(LogWindowsRWHelper, Error, "Failed to initialize Windows Media Foundation %ls", *Message);

		return false;
	}

	Result = MFStartup(MF_VERSION);

	if (!SUCCEEDED(Result))
	{
		if (bOwnsComInit)
		{
			CoUninitialize();
		}

		// E_NOTIMPL indicates WMF is not functional on this system (e.g. Server
		// Core where mfplat.dll is present but the runtime is stubbed out). This
		// is an expected condition, not an error worth reporting.
		if (Result == E_NOTIMPL)
		{
			UE_LOGF(LogWindowsRWHelper, Log, "Windows Media Foundation is not available on this system. Windows media readers/writers will be disabled.");
		}
		else
		{
			FString Message = FormatWindowsMessage(Result);
			UE_LOGF(LogWindowsRWHelper, Error, "Failed to start Windows Media Foundation %ls", *Message);
		}

		return false;
	}

	bInitialized = true;
	return true;
}

void FWindowsRWHelpers::Deinit()
{
	if (bInitialized)
	{
		MFShutdown();

		if (bOwnsComInit)
		{
			CoUninitialize();
		}

		bInitialized = false;
		bOwnsComInit = false;
	}
}

void FWindowsRWHelpers::RegisterReaders(FMediaRWManager& InManager)
{
	TArray<FString> SupportedExtensions = { TEXT("mov"), TEXT("mp4") };
	InManager.RegisterAudioReader(SupportedExtensions, MakeUnique<FWindowsReadersFactory>());
	InManager.RegisterVideoReader(SupportedExtensions, MakeUnique<FWindowsReadersFactory>());
}

void FWindowsRWHelpers::RegisterWriters(FMediaRWManager& InManager)
{
	TArray<FString> SupportedExtensions = { TEXT("png"), TEXT("jpg"), TEXT("jpeg") };
	InManager.RegisterImageWriter(SupportedExtensions, MakeUnique<FWindowsImageWriterFactory>());
}

FText FWindowsRWHelpers::CreateErrorMessage(HRESULT InResult, FText InMessage)
{
	FText WindowsErrorMessage = FText::FromString(FWindowsRWHelpers::FormatWindowsMessage(InResult));
	FText ErrorMessage = FText::Format(FText::FromString(TEXT("{0}: {1}")), MoveTemp(InMessage), WindowsErrorMessage);

	return ErrorMessage;
}

FString FWindowsRWHelpers::FormatWindowsMessage(HRESULT InResult)
{
	constexpr uint32 BufSize = 1024;
	WIDECHAR Buffer[BufSize];

	FormatMessage(FORMAT_MESSAGE_FROM_SYSTEM, nullptr, InResult, 0, Buffer, BufSize, nullptr);

	return Buffer;
}

#endif // PLATFORM_WINDOWS && !UE_SERVER