// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Readers/WindowsMediaReader.h"
#include "Writers/WindowsImageWriter.h"

#if PLATFORM_WINDOWS && !UE_SERVER

class FWindowsRWHelpers
{
public:

	static bool Init();
	static void Deinit();

	static void RegisterReaders(class FMediaRWManager& InManager);
	static void RegisterWriters(class FMediaRWManager& InManager);

	static FText CreateErrorMessage(HRESULT InResult, FText InMessage);

private:

	static FString FormatWindowsMessage(HRESULT InResult);

	static bool bInitialized;
	static bool bOwnsComInit;
};

#endif // PLATFORM_WINDOWS && !UE_SERVER