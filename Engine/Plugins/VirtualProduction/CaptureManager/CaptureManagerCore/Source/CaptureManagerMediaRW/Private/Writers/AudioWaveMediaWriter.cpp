// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioWaveMediaWriter.h"

#include "MediaRWManager.h"

#include "AudioWaveFormatParser.h"
#include "HAL/PlatformFileManager.h"

#define LOCTEXT_NAMESPACE "AudioWaveWriter"

DEFINE_LOG_CATEGORY_STATIC(LogAudioWaveWriter, Log, All);

#define AWW_CHECK_AND_RETURN_MESSAGE(Result, Message)                                 \
	if (!(Result))                                                                    \
	{                                                                                 \
		FText ErrorMessage = FText::Format(FText::FromString(TEXT("{0}")), Message);  \
		UE_LOGF(LogAudioWaveWriter, Error, "%ls", *ErrorMessage.ToString()); \
		return ErrorMessage;                                                          \
	}

void FAudioWaveWriterHelpers::RegisterWriters(FMediaRWManager& InManager)
{
	TArray<FString> SupportedFormats = { TEXT("wav") };
	InManager.RegisterAudioWriter(SupportedFormats, MakeUnique<FAudioWaveWriterFactory>());
}

TUniquePtr<IAudioWriter> FAudioWaveWriterFactory::CreateAudioWriter()
{
	return MakeUnique<FAudioWaveWriter>();
}

static constexpr uint32 WaveFileHeaderSize = 44;
static constexpr uint32 PCMFormatChunkSize = 16;

#define CHUNK_ID_RIFF			(0x46464952)	// "RIFF"
#define CHUNK_TYPE_WAVE			(0x45564157)	// "WAVE"
#define CHUNK_ID_FMT			(0x20746D66)	// "fmt "
#define CHUNK_ID_DATA			(0x61746164)	// "data"

FAudioWaveWriter::FAudioWaveWriter() = default;
FAudioWaveWriter::~FAudioWaveWriter() = default;

TOptional<FText> FAudioWaveWriter::Open(const FString& InDirectory, const FString& InFileName, const FString& InFormat)
{
	AWW_CHECK_AND_RETURN_MESSAGE(InFormat == TEXT("wav"), LOCTEXT("Open_UnsupportedFormat", "Unsupported audio format"));

	FString FullFilePath = FPaths::SetExtension(InDirectory / InFileName, InFormat);

	FileHandle.Reset(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*FullFilePath));
	AWW_CHECK_AND_RETURN_MESSAGE(FileHandle != nullptr, FText::Format(LOCTEXT("Open_FailedToCreateAudioFile", "Failed to create audio file {0}"), FText::FromString(FullFilePath)));

	bool Result = FileHandle->Seek(WaveFileHeaderSize);
	AWW_CHECK_AND_RETURN_MESSAGE(Result, LOCTEXT("Open_FailedToSeekSizeOfHeader", "Failed to seek to the size of the WAV header"));

	return {};
}

TOptional<FText> FAudioWaveWriter::Close()
{
	checkf(FileHandle, TEXT("File handle MUST not be null"));

	int32 BitsPerSampleNum = UE::CaptureManager::ConvertBitsPerSample(BitsPerSample);
	int32 SampleRateNum = UE::CaptureManager::ConvertSampleRate(SampleRate);

	uint32 UInt32Buffer;
	FFormatChunk PCMFormatChunk;

	bool Result = FileHandle->Seek(0); // Seek to the beginning of the file to write the header

	UInt32Buffer = CHUNK_ID_RIFF;
	Result &= FileHandle->Write(reinterpret_cast<const uint8*>(&UInt32Buffer), sizeof(uint32));

	UInt32Buffer = TotalDataBytesWritten + WaveFileHeaderSize - 8;
	Result &= FileHandle->Write(reinterpret_cast<const uint8*>(&UInt32Buffer), sizeof(uint32));

	UInt32Buffer = CHUNK_TYPE_WAVE;
	Result &= FileHandle->Write(reinterpret_cast<const uint8*>(&UInt32Buffer), sizeof(uint32));

	UInt32Buffer = CHUNK_ID_FMT;
	Result &= FileHandle->Write(reinterpret_cast<const uint8*>(&UInt32Buffer), sizeof(uint32));

	UInt32Buffer = PCMFormatChunkSize;
	Result &= FileHandle->Write(reinterpret_cast<const uint8*>(&UInt32Buffer), sizeof(uint32));

	int32 BlockSize = NumChannels * BitsPerSampleNum / 8;
	PCMFormatChunk.FormatTag = 1;	// Linear PCM
	PCMFormatChunk.NumChannels = NumChannels;
	PCMFormatChunk.SamplesPerSec = SampleRateNum;
	PCMFormatChunk.AverageBytesPerSec = SampleRateNum * BlockSize;
	PCMFormatChunk.BlockAlign = BlockSize;
	PCMFormatChunk.BitsPerSample = BitsPerSampleNum;

	Result &= FileHandle->Write(reinterpret_cast<const uint8*>(&PCMFormatChunk), PCMFormatChunkSize);

	UInt32Buffer = CHUNK_ID_DATA;
	Result &= FileHandle->Write(reinterpret_cast<const uint8*>(&UInt32Buffer), sizeof(uint32));

	UInt32Buffer = TotalDataBytesWritten;
	Result &= FileHandle->Write(reinterpret_cast<const uint8*>(&UInt32Buffer), sizeof(uint32));

	AWW_CHECK_AND_RETURN_MESSAGE(Result, LOCTEXT("Close_FailedToWriteHeader", "Failed to write the WAV header data"));

	FileHandle->Flush();

	FileHandle = nullptr;

	return {};
}

TOptional<FText> FAudioWaveWriter::Append(UE::CaptureManager::FMediaAudioSample* InSample)
{
	int32 BitsPerSampleNum = UE::CaptureManager::ConvertBitsPerSample(BitsPerSample);
	int32 SampleRateNum = UE::CaptureManager::ConvertSampleRate(SampleRate);

	int32 BlockSize = NumChannels * BitsPerSampleNum / 8;

	const int64 Time = InSample->Time.GetTicks();
	const int64 Duration = InSample->Duration.GetTicks();
	const uint8* Data = static_cast<const uint8*>(InSample->Buffer.GetData());
	const int32 Frames = InSample->Frames;
	const uint32 Size = Frames * BlockSize;
	uint32 BytesToSkip = 0;

	if (Time + Duration < 0)
	{
		return {};
	}

	if (TotalDataBytesWritten <= 0)
	{
		//	The audio samples can kick in before or after the first video frame arrives.
		//	Add zero paddings or skip some samples to match the start of the video.
		if (Time < 0)
		{
			BytesToSkip = FMath::RoundToInt(Time * SampleRateNum * -1.0e-7) * BlockSize;
		}
		else
		{
			const int32 SamplesToPad = FMath::RoundToInt(Time * SampleRateNum * 1e-7);
			TArray<uint8> ZeroBlock;
			ZeroBlock.SetNumZeroed(BlockSize);

			for (int32 i = 0; i < SamplesToPad; ++i)
			{
				FileHandle->Write(ZeroBlock.GetData(), ZeroBlock.Num());
				TotalDataBytesWritten += BlockSize;
			}
		}
	}

	if (BytesToSkip >= Size)
	{
		return {};
	}

	bool Result = FileHandle->Write(Data + BytesToSkip, Size - BytesToSkip);
	AWW_CHECK_AND_RETURN_MESSAGE(Result, LOCTEXT("Append_FailedToWriteData", "Failed to write the data to the file"));
	
	TotalDataBytesWritten += Size - BytesToSkip;

	return {};
}

#undef LOCTEXT_NAMESPACE