// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaSample.h"
#include "IMediaWriter.h"
#include "IMediaRWFactory.h"

class FAudioWaveWriterHelpers
{
public:

	static void RegisterWriters(class FMediaRWManager& InManager);
};

class FAudioWaveWriterFactory final : public IAudioWriterFactory
{
public:

	virtual TUniquePtr<IAudioWriter> CreateAudioWriter();
};

class FAudioWaveWriter final : public IAudioWriter
{
public:

	FAudioWaveWriter();
	virtual ~FAudioWaveWriter() override;

	virtual TOptional<FText> Open(const FString& InDirectory, const FString& InFileName, const FString& InFormat) override;
	virtual TOptional<FText> Close() override;

	virtual TOptional<FText> Append(UE::CaptureManager::FMediaAudioSample* InSample) override;

private:

	TUniquePtr<IFileHandle> FileHandle;
	uint64 TotalDataBytesWritten = 0;
};