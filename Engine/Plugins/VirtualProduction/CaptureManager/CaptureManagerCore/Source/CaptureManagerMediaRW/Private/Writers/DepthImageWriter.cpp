// Copyright Epic Games, Inc. All Rights Reserved.

#include "DepthImageWriter.h"

#include "MediaRWManager.h"

#include "Modules/ModuleManager.h"

#include "IImageWrapperModule.h"
#include "IImageWrapper.h"

#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"

#define LOCTEXT_NAMESPACE "DepthImageWriter"

void FDepthExrImageWriterHelpers::RegisterWriters(FMediaRWManager& InManager)
{
	TArray<FString> SupportedFormats = { TEXT("exr") };
	InManager.RegisterImageWriter(SupportedFormats, MakeUnique<FDepthExrImageWriterFactory>());
}

TUniquePtr<IImageWriter> FDepthExrImageWriterFactory::CreateImageWriter()
{
	return MakeUnique<FDepthExrImageWriter>();
}

static FIntPoint GetOutputSize(const FIntPoint InInputSize, const EMediaOrientation InOrientation)
{
	switch (InOrientation)
	{
		case EMediaOrientation::CW90:
		case EMediaOrientation::CW270:
			return FIntPoint(InInputSize.Y, InInputSize.X);
		case EMediaOrientation::Original:
		case EMediaOrientation::CW180:
		default:
			return InInputSize;
	}
}

FDepthExrImageWriter::FDepthExrImageWriter()
	: ImageWrapperModule(FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper")))
{
}

FDepthExrImageWriter::~FDepthExrImageWriter() = default;

TOptional<FText> FDepthExrImageWriter::Open(const FString& InDirectory, const FString& InFileName, const FString& InFormat)
{
	if (!FPlatformFileManager::Get().GetPlatformFile().CreateDirectory(*InDirectory))
	{
		FText Error = FText::Format(
			LOCTEXT("DirectoryCreationFailed", "Failed to create the directory: {0}."), FText::FromString(InDirectory));
		return Error;
	}

	Directory = InDirectory;
	FileName = InFileName;
	Format = InFormat;

	return {};
}

TOptional<FText> FDepthExrImageWriter::Close()
{
	return {};
}

TOptional<FText> FDepthExrImageWriter::Append(UE::CaptureManager::FMediaTextureSample* InSample)
{
	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::EXR);

	if (!ImageWrapper)
	{
		FText Error = LOCTEXT("DepthConverter_CreateError", "Failed to create the image wrapper.");
		return Error;
	}

	const FString ExrFilePath = FPaths::Combine(Directory, FString::Printf(TEXT("%s_%06d.exr"), *FileName, FrameNumber));

	FIntPoint OutputSize = GetOutputSize(InSample->Dimensions, InSample->Rotation);

	TArray<float> RotatedData = Transform(MoveTemp(InSample->Buffer), InSample->Dimensions, InSample->Rotation);

	if (!ImageWrapper->SetRaw(RotatedData.GetData(), RotatedData.Num() * sizeof(float), OutputSize.X, OutputSize.Y, ERGBFormat::GrayF, sizeof(float) * 8))
	{
		FText Error = FText::Format(LOCTEXT("DepthImageCreationFailed", "Failed to create the depth image: {0}."), FText::FromString(ExrFilePath));
		return Error;
	}

	EImageCompressionQuality Compression = EImageCompressionQuality::Default;

	const TArray64<uint8> ExrBuffer = ImageWrapper->GetCompressed((int32) Compression);

	if (!FFileHelper::SaveArrayToFile(ExrBuffer, *ExrFilePath))
	{
		FText Error = FText::Format(LOCTEXT("DepthImageSaveFailed", "Failed to save the depth image: {0}."), FText::FromString(ExrFilePath));
		return Error;
	}

	++FrameNumber;

	return {};
}

TArray<float> FDepthExrImageWriter::Transform(TArray<uint8> InDepthArray, FIntPoint InDimensions, EMediaOrientation InOrientation) const
{
	TArray<float> RotatedData;
	RotatedData.SetNumZeroed(InDimensions.X * InDimensions.Y);

	FIntPoint OutputSize = GetOutputSize(InDimensions, InOrientation);

	TArrayView<int16> ConvertedData(reinterpret_cast<int16*>(InDepthArray.GetData()), InDepthArray.Num() / 2);

	switch (InOrientation)
	{
		case EMediaOrientation::Original:
			for (int32 Y = 0; Y < OutputSize.Y; ++Y)
			{
				for (int32 X = 0; X < OutputSize.X; ++X)
				{
					int32 Index = Y * InDimensions.X + X;
					RotatedData[Y * OutputSize.X + X] = ConvertedData[Index] / TrueDepthResolutionPerCentimeter;
				}
			}
			break;
		case EMediaOrientation::CW90:
			for (int32 Y = 0; Y < OutputSize.Y; ++Y)
			{
				for (int32 X = 0; X < OutputSize.X; ++X)
				{
					int32 Index = (OutputSize.X - X - 1) * InDimensions.X + Y;
					RotatedData[Y * OutputSize.X + X] = ConvertedData[Index] / TrueDepthResolutionPerCentimeter;
				}
			}
			break;
		case EMediaOrientation::CW180:
			for (int32 Y = 0; Y < OutputSize.Y; ++Y)
			{
				for (int32 X = 0; X < OutputSize.X; ++X)
				{
					int32 Index = (InDimensions.Y - Y - 1) * InDimensions.X + (InDimensions.X - X - 1);
					RotatedData[Y * OutputSize.X + X] = ConvertedData[Index] / TrueDepthResolutionPerCentimeter;
				}
			}
			break;
		case EMediaOrientation::CW270:
			for (int32 Y = 0; Y < OutputSize.Y; ++Y)
			{
				for (int32 X = 0; X < OutputSize.X; ++X)
				{
					int32 Index = X * InDimensions.X + (OutputSize.Y - Y - 1);
					RotatedData[Y * OutputSize.X + X] = ConvertedData[Index] / TrueDepthResolutionPerCentimeter;
				}
			}
			break;
	}

	return RotatedData;
}

#undef LOCTEXT_NAMESPACE