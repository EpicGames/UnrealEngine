// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaWriter.h"
#include "IMediaRWFactory.h"

class IImageWrapper;
class IImageWrapperModule;

class FDepthExrImageWriterHelpers
{
public:

	static void RegisterWriters(class FMediaRWManager& InManager);
};

class FDepthExrImageWriterFactory final : public IImageWriterFactory
{
public:

	virtual TUniquePtr<IImageWriter> CreateImageWriter() override;
};

class FDepthExrImageWriter final : public IImageWriter
{
public:

	FDepthExrImageWriter();

	virtual ~FDepthExrImageWriter() override;

	virtual TOptional<FText> Open(const FString& InDirectory, const FString& InFileName, const FString& InFormat) override;
	virtual TOptional<FText> Close() override;

	virtual TOptional<FText> Append(UE::CaptureManager::FMediaTextureSample* InSample) override;

private:

	TArray<float> Transform(TArray<uint8> InDepthArray, FIntPoint InDimensions, EMediaOrientation InOrientation) const;

	static constexpr float TrueDepthResolutionPerCentimeter = 80.f;

	IImageWrapperModule& ImageWrapperModule;

	FString Directory;
	FString FileName;
	FString Format;

	uint32 FrameNumber = 0;
};