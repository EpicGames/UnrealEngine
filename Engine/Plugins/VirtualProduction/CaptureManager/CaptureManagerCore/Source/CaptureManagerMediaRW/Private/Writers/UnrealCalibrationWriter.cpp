// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealCalibrationWriter.h"

#include "MediaRWManager.h"

#include "Misc/FileHelper.h"

#define LOCTEXT_NAMESPACE "UnrealCalibrationWriter"

#define UNREAL_FORMAT_VERSION 1 // Version of the unreal calibration file
#define UNREAL_CALIBRATION_FORMAT_VERSION 1 // Version of the calibration model

namespace UE::CaptureManager::Private
{

static void WriteiPhoneDistortionModel(TSharedPtr<TJsonWriter<TCHAR>> InJsonWriter, const UE::CaptureManager::FIphoneDistortionModel& InDistortionModel)
{
	InJsonWriter->WriteObjectStart(TEXT("DistortionModel"));

	InJsonWriter->WriteValue(TEXT("Name"), TEXT("iphone"));

	InJsonWriter->WriteArrayStart(TEXT("LensDistortionLookupTable"));

	for (double Value : InDistortionModel.LensDistortionTable)
	{
		InJsonWriter->WriteValue(Value);
	}

	InJsonWriter->WriteArrayEnd(); // LensDistortionLookupTable

	InJsonWriter->WriteArrayStart(TEXT("InverseLensDistortionLookupTable"));

	for (double Value : InDistortionModel.InverseLensDistortionTable)
	{
		InJsonWriter->WriteValue(Value);
	}

	InJsonWriter->WriteArrayEnd(); // InverseLensDistortionLookupTable

	InJsonWriter->WriteObjectEnd();
}

static void WriteOpencvDistortionModel(TSharedPtr<TJsonWriter<TCHAR>> InJsonWriter, const UE::CaptureManager::FOpenCVDistortionModel& InDistortionModel)
{
	InJsonWriter->WriteObjectStart(TEXT("DistortionModel"));

	InJsonWriter->WriteValue(TEXT("Name"), TEXT("opencv"));

	InJsonWriter->WriteObjectStart(TEXT("Radial"));
	InJsonWriter->WriteValue(TEXT("K1"), InDistortionModel.Radial.K1);
	InJsonWriter->WriteValue(TEXT("K2"), InDistortionModel.Radial.K2);
	InJsonWriter->WriteValue(TEXT("K3"), InDistortionModel.Radial.K3);
	InJsonWriter->WriteObjectEnd();

	InJsonWriter->WriteObjectStart(TEXT("Tangential"));
	InJsonWriter->WriteValue(TEXT("P1"), InDistortionModel.Tangential.P1);
	InJsonWriter->WriteValue(TEXT("P2"), InDistortionModel.Tangential.P2);
	InJsonWriter->WriteObjectEnd();

	InJsonWriter->WriteObjectEnd();
}

}

void FUnrealCalibrationWriterHelpers::RegisterWriters(FMediaRWManager& InManager)
{
	const TArray<FString> SupportedFormats = { TEXT("unreal") };
	InManager.RegisterCalibrationWriter(SupportedFormats, MakeUnique<FUnrealCalibrationWriterFactory>());
}

TUniquePtr<ICalibrationWriter> FUnrealCalibrationWriterFactory::CreateCalibrationWriter()
{
	return MakeUnique<FUnrealCalibrationWriter>();
}

TOptional<FText> FUnrealCalibrationWriter::Open(const FString& InDirectory, const FString& InFileName, const FString& InFormat)
{
	DestinationFile = FPaths::SetExtension(InDirectory / InFileName, TEXT("json"));

	JsonWriter = TJsonWriterFactory<TCHAR>::Create(&JsonString);

	JsonWriter->WriteObjectStart();

	JsonWriter->WriteValue(TEXT("Version"), UNREAL_FORMAT_VERSION);

	JsonWriter->WriteArrayStart(TEXT("Calibrations"));

	return {};
}

TOptional<FText> FUnrealCalibrationWriter::Close()
{
	JsonWriter->WriteArrayEnd();
	JsonWriter->WriteObjectEnd();

	JsonWriter->Close();
	JsonWriter = nullptr;

	if (!FFileHelper::SaveStringToFile(JsonString, *DestinationFile))
	{
		return FText::Format(LOCTEXT("UnrealCalibrationWriter_Close", "Failed to serialize json file"), FText::FromString(DestinationFile));
	}

	return {};
}

TOptional<FText> FUnrealCalibrationWriter::Append(UE::CaptureManager::FMediaCalibrationSample* InSample)
{
	using namespace UE::CaptureManager;

	// Calibration sample
	JsonWriter->WriteObjectStart();
	
	// Version
	JsonWriter->WriteValue(TEXT("Version"), UNREAL_CALIBRATION_FORMAT_VERSION);

	// CameraId
	JsonWriter->WriteValue(TEXT("CameraId"), InSample->CameraId);
	JsonWriter->WriteValue(TEXT("CameraType"), static_cast<std::underlying_type_t<FMediaCalibrationSample::ECameraType>>(InSample->CameraType));

	// Dimensions
	JsonWriter->WriteObjectStart(TEXT("Dimensions"));
	JsonWriter->WriteValue(TEXT("Width"), InSample->Dimensions.X);
	JsonWriter->WriteValue(TEXT("Height"), InSample->Dimensions.Y);
	JsonWriter->WriteObjectEnd(); // Dimensions

	// Orientation
	JsonWriter->WriteValue(TEXT("Orientation"), static_cast<std::underlying_type_t<EMediaOrientation>>(InSample->Orientation));

	// Distortion Models
	if (InSample->DistortionModel.IsType<UE::CaptureManager::FIphoneDistortionModel>())
	{
		FIphoneDistortionModel DistortionModel = 
			InSample->DistortionModel.Get<FIphoneDistortionModel>();

		Private::WriteiPhoneDistortionModel(JsonWriter, DistortionModel);
	}
	else if (InSample->DistortionModel.IsType<UE::CaptureManager::FOpenCVDistortionModel>())
	{
		FOpenCVDistortionModel DistortionModel =
			InSample->DistortionModel.Get<FOpenCVDistortionModel>();

		Private::WriteOpencvDistortionModel(JsonWriter, DistortionModel);
	}
	
	// Focal length (fx, fy)
	JsonWriter->WriteArrayStart(TEXT("FocalLength"));
	JsonWriter->WriteValue(InSample->FocalLength.X);
	JsonWriter->WriteValue(InSample->FocalLength.Y);
	JsonWriter->WriteArrayEnd(); // FocalLength

	// Principal Point (cx, cy)
	JsonWriter->WriteArrayStart(TEXT("PrincipalPoint"));
	JsonWriter->WriteValue(InSample->PrincipalPoint.X);
	JsonWriter->WriteValue(InSample->PrincipalPoint.Y);
	JsonWriter->WriteArrayEnd(); // PrincipalPoint

	// Transform (a11, a12, a13, a14, a21, a22, ...)
	JsonWriter->WriteArrayStart(TEXT("Transform"));

	if (InSample->InputCoordinateSystem != UE::CaptureManager::UnrealCS)
	{
		InSample->Transform = UE::CaptureManager::ConvertToCoordinateSystem(InSample->Transform, InSample->InputCoordinateSystem, UE::CaptureManager::UnrealCS);
	}

	FMatrix Transform = InSample->Transform.ToMatrixNoScale();
	for (int32 Row = 0; Row < 4; ++Row)
	{
		for (int32 Column = 0; Column < 4; ++Column)
		{
			JsonWriter->WriteValue(Transform.M[Row][Column]);
		}
	}

	JsonWriter->WriteArrayEnd(); // Transform

	JsonWriter->WriteObjectEnd();

	return {};
}

#undef LOCTEXT_NAMESPACE