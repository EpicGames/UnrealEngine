// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaSample.h"
#include "IMediaWriter.h"
#include "IMediaRWFactory.h"

#include "Serialization/JsonWriter.h"

class FUnrealCalibrationWriterHelpers
{
public:

	static void RegisterWriters(class FMediaRWManager& InManager);
};

class FUnrealCalibrationWriterFactory final : public ICalibrationWriterFactory
{
public:

	virtual TUniquePtr<ICalibrationWriter> CreateCalibrationWriter() override;
};


class FUnrealCalibrationWriter final : public ICalibrationWriter
{
public:

	virtual TOptional<FText> Open(const FString& InDirectory, const FString& InFileName, const FString& InFormat) override;
	virtual TOptional<FText> Close() override;

	virtual TOptional<FText> Append(UE::CaptureManager::FMediaCalibrationSample* InSample) override;

private:

	FString DestinationFile;
	TSharedPtr<TJsonWriter<TCHAR>> JsonWriter;
	FString JsonString;
};