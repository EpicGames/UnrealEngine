// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsImageWriter.h"

#include "MediaRWManager.h"

#include "IImageWrapperModule.h"
#include "Modules/ModuleManager.h"
#include "Utils/MediaPixelFormatConversions.h"

#if PLATFORM_WINDOWS && !UE_SERVER

#include "Utils/WindowsRWHelpers.h"

#define LOCTEXT_NAMESPACE "WindowsImageReader"

DEFINE_LOG_CATEGORY_STATIC(LogWindowsImageWriter, Log, All);

static const GUID UnsupportedEncoder = GUID_NULL;

#define WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, Message)                        \
	if (FAILED(Result))                                                              \
	{                                                                                \
		FText ErrorMessage = FWindowsRWHelpers::CreateErrorMessage(Result, Message); \
		UE_LOGF(LogWindowsImageWriter, Error, "%ls", *ErrorMessage.ToString());  \
		return ErrorMessage;														 \
	}

TUniquePtr<IImageWriter> FWindowsImageWriterFactory::CreateImageWriter()
{
	return MakeUnique<FWindowsImageWriter>();
}

namespace UE::CaptureManager::Private
{

WICBitmapTransformOptions ConvertToWindowsTransformOptions(EMediaOrientation InRotation)
{
	switch (InRotation)
	{
		case EMediaOrientation::CW90:
			return WICBitmapTransformRotate90;
		case EMediaOrientation::CW180:
			return WICBitmapTransformRotate180;
		case EMediaOrientation::CW270:
			return WICBitmapTransformRotate270;
		case EMediaOrientation::Original:
		default:
			return WICBitmapTransformRotate0;
	}
}

FIntPoint GetNewDimensions(const FMediaTextureSample* InNewSample)
{
	switch (InNewSample->Rotation)
	{
		case EMediaOrientation::CW90:
		case EMediaOrientation::CW270:
			return FIntPoint(InNewSample->Dimensions.Y, InNewSample->Dimensions.X);
		case EMediaOrientation::Original:
		case EMediaOrientation::CW180:
		default:
			return InNewSample->Dimensions;
	}
}

WICPixelFormatGUID ConvertPixelFormat(EMediaTexturePixelFormat InPixelFormat)
{
	switch (InPixelFormat)
	{
		case EMediaTexturePixelFormat::U8_RGBA:
			return GUID_WICPixelFormat32bppRGBA;
		case EMediaTexturePixelFormat::U8_RGB:
			return GUID_WICPixelFormat24bppRGB;
		case EMediaTexturePixelFormat::U8_BGRA:
			return GUID_WICPixelFormat32bppBGRA;
		case EMediaTexturePixelFormat::U8_BGR:
			return GUID_WICPixelFormat24bppBGR;
		case EMediaTexturePixelFormat::U8_Mono:
			return GUID_WICPixelFormat8bppGray;
		case EMediaTexturePixelFormat::U16_Mono:
			return GUID_WICPixelFormat16bppGray;
		case EMediaTexturePixelFormat::F_Mono:
			return GUID_WICPixelFormat32bppGrayFloat;
		case EMediaTexturePixelFormat::U8_I420:
		case EMediaTexturePixelFormat::U8_YUY2:
		case EMediaTexturePixelFormat::U8_NV12:
		case EMediaTexturePixelFormat::U8_I444:
		default:
			return GUID_NULL;
	}
}

FString GetPixelFormatString(EMediaTexturePixelFormat InPixelFormat)
{
	switch (InPixelFormat)
	{
		case EMediaTexturePixelFormat::U8_RGBA:
			return TEXT("U8 RGBA");
		case EMediaTexturePixelFormat::U8_RGB:
			return TEXT("U8 RGBA");
		case EMediaTexturePixelFormat::U8_BGRA:
			return TEXT("U8 RGBA");
		case EMediaTexturePixelFormat::U8_BGR:
			return TEXT("U8 RGBA");
		case EMediaTexturePixelFormat::U8_Mono:
			return TEXT("U8 RGBA");
		case EMediaTexturePixelFormat::U16_Mono:
			return TEXT("U8 RGBA");
		case EMediaTexturePixelFormat::F_Mono:
			return TEXT("U8 RGBA");
		case EMediaTexturePixelFormat::U8_I420:
			return TEXT("U8 I420");
		case EMediaTexturePixelFormat::U8_YUY2:
			return TEXT("U8 YUY2");
		case EMediaTexturePixelFormat::U8_NV12:
			return TEXT("U8 NV12");
		case EMediaTexturePixelFormat::U8_I444:
			return TEXT("U8 I444");
		default:
			return TEXT("Undefined");
	}
}

class FConverter
{
public:

	static FConverter Create(TComPtr<IWICImagingFactory> InFactory,
							 TComPtr<IWICBitmap> InBitmap,
							 EMediaTexturePixelFormat InInputFormat,
							 EMediaTexturePixelFormat InOutputFormat)
	{
		FConverter Converter;

		Converter.PixelFormat = Private::ConvertPixelFormat(InInputFormat);
		Converter.Source = InBitmap;

		if (InInputFormat == InOutputFormat)
		{
			return Converter;
		}

		TComPtr<IWICFormatConverter> WICConverter;
		HRESULT Result = InFactory->CreateFormatConverter(&WICConverter);

		if (SUCCEEDED(Result))
		{
			WICPixelFormatGUID OutPixelFormat = Private::ConvertPixelFormat(InOutputFormat);
			if (OutPixelFormat != GUID_NULL)
			{
				BOOL bCanConvert = false;
				Result = WICConverter->CanConvert(Converter.PixelFormat, OutPixelFormat, &bCanConvert);
				if (SUCCEEDED(Result) && bCanConvert)
				{
					Result = WICConverter->Initialize(InBitmap, OutPixelFormat, WICBitmapDitherTypeNone, nullptr, 0.f, WICBitmapPaletteTypeCustom);
					if (SUCCEEDED(Result))
					{
						Converter.PixelFormat = OutPixelFormat;
						Converter.Source = WICConverter;
					}
					else
					{
						FString Message =
							FString::Format(TEXT("Failed to convert from ({0}) to the ({1}) pixel format"),
											{ Private::GetPixelFormatString(InInputFormat), Private::GetPixelFormatString(InOutputFormat) });
						UE_LOGF(LogWindowsImageWriter, Warning, "%ls", *Message);
					}
				}
				else
				{
					FString Message =
						FString::Format(TEXT("Unsupported pixel format conversion: {0} -> {1}"),
										{ Private::GetPixelFormatString(InInputFormat), Private::GetPixelFormatString(InOutputFormat) });

					UE_LOGF(LogWindowsImageWriter, Warning, "%ls", *Message);
				}
			}
			else
			{
				UE_LOGF(LogWindowsImageWriter, Warning, "Unsupported pixel format provided.");
			}
		}
		else
		{
			UE_LOGF(LogWindowsImageWriter, Warning, "Failed to create converter from WIC");
		}

		return Converter;
	}

	TComPtr<IWICBitmapSource> GetSource()
	{
		return Source;
	}

	WICPixelFormatGUID GetPixelFormat()
	{
		return PixelFormat;
	}

private:

	TComPtr<IWICBitmapSource> Source;
	WICPixelFormatGUID PixelFormat;
};

}

FWindowsImageWriter::FWindowsImageWriter() = default;
FWindowsImageWriter::~FWindowsImageWriter() = default;

TOptional<FText> FWindowsImageWriter::Open(const FString& InDirectory, const FString& InFileName, const FString& InFormat)
{
	HRESULT Result = CoCreateInstance(CLSID_WICImagingFactory, NULL, CLSCTX_INPROC_SERVER, IID_PPV_ARGS(&WindowsImagingFactory));
	WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Open_FailedToInitialize", "Failed to initialize Windows Imaging Component"));

	GUID EncoderGuid = GetEncoderGuidBasedOnFormat(InFormat);
	if (EncoderGuid == UnsupportedEncoder)
	{
		FText ErrorMessage = LOCTEXT("Open_UnsupportedEncoder", "Unsupported encoder provided");
		UE_LOGF(LogWindowsImageWriter, Error, "%ls", *ErrorMessage.ToString());

		return ErrorMessage;
	}
	
	Directory = InDirectory;
	FileName = InFileName;
	Format = InFormat;

	return {};
}

TOptional<FText> FWindowsImageWriter::Close()
{
	WindowsImagingFactory = nullptr;

	return {};
}

TOptional<FText> FWindowsImageWriter::Append(UE::CaptureManager::FMediaTextureSample* InSample)
{
	using namespace UE::CaptureManager;

	if (InSample->DesiredFormat == EMediaTexturePixelFormat::U8_Mono || 
		InSample->DesiredFormat == EMediaTexturePixelFormat::Undefined)
	{
		InSample->DesiredFormat = EMediaTexturePixelFormat::U8_Mono;
		if (InSample->CurrentFormat == UE::CaptureManager::EMediaTexturePixelFormat::U8_YUY2)
		{
			InSample->Buffer = ConvertYUY2ToMono(InSample, true);
			InSample->CurrentFormat = EMediaTexturePixelFormat::U8_Mono;
			InSample->Stride = InSample->Dimensions.X * GetNumberOfChannels(InSample->CurrentFormat);
		}
		else if (InSample->CurrentFormat == UE::CaptureManager::EMediaTexturePixelFormat::U8_I420 ||
				 InSample->CurrentFormat == UE::CaptureManager::EMediaTexturePixelFormat::U8_NV12)
		{
			InSample->Buffer = ConvertYUVToMono(InSample, true);
			InSample->CurrentFormat = EMediaTexturePixelFormat::U8_Mono;
			InSample->Stride = InSample->Dimensions.X * GetNumberOfChannels(InSample->CurrentFormat);
		}
		else
		{
			// Do nothing
		}
	}
	else if (InSample->DesiredFormat == EMediaTexturePixelFormat::U8_BGRA)
	{
		if (InSample->CurrentFormat == UE::CaptureManager::EMediaTexturePixelFormat::U8_I420)
		{
			InSample->Buffer = ConvertI420ToBGRA(InSample);
			InSample->CurrentFormat = EMediaTexturePixelFormat::U8_BGRA;
			InSample->Stride = InSample->Dimensions.X * GetNumberOfChannels(InSample->CurrentFormat);
		}
		else if (InSample->CurrentFormat == UE::CaptureManager::EMediaTexturePixelFormat::U8_NV12)
		{
			InSample->Buffer = ConvertNV12ToBGRA(InSample);
			InSample->CurrentFormat = EMediaTexturePixelFormat::U8_BGRA;
			InSample->Stride = InSample->Dimensions.X * GetNumberOfChannels(InSample->CurrentFormat);
		}
		else if (InSample->CurrentFormat == UE::CaptureManager::EMediaTexturePixelFormat::U8_YUY2)
		{
			InSample->Buffer = ConvertYUY2ToBGRA(InSample);
			InSample->CurrentFormat = EMediaTexturePixelFormat::U8_BGRA;
			InSample->Stride = InSample->Dimensions.X * GetNumberOfChannels(InSample->CurrentFormat);
		}
		else
		{
			// Do nothing
		}
	}

	WICPixelFormatGUID PixelFormat = Private::ConvertPixelFormat(InSample->CurrentFormat);
	if (PixelFormat == GUID_NULL)
	{
		FText ErrorMessage = 
			FText::Format(
				LOCTEXT("Append_UnsupportedPixelFormat", "Image has unsupported pixel format ({0})"), 
						FText::FromString(*Private::GetPixelFormatString(InSample->CurrentFormat)));
		UE_LOGF(LogWindowsImageWriter, Error, "%ls", *ErrorMessage.ToString());

		return ErrorMessage;
	}

	TComPtr<IWICBitmap> Bitmap;
	HRESULT Result = WindowsImagingFactory->CreateBitmapFromMemory(InSample->Dimensions.X,
																   InSample->Dimensions.Y,
																   PixelFormat,
																   InSample->Stride,
																   InSample->Buffer.Num(),
																   InSample->Buffer.GetData(),
																   &Bitmap);
	WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Append_FailedToCreateBitmap", "Failed to create bitmap from memory"));

	InSample->Buffer.Empty();

	Private::FConverter Converter = Private::FConverter::Create(WindowsImagingFactory, Bitmap, InSample->CurrentFormat, InSample->DesiredFormat);

	TComPtr<IWICBitmapSource> Source = Converter.GetSource();
	PixelFormat = Converter.GetPixelFormat();

	UE_LOGF(LogWindowsImageWriter, Verbose, "Output pixel format is %ls", *Private::GetPixelFormatString(InSample->CurrentFormat));
	
	TComPtr<IWICBitmapFlipRotator> Rotator = nullptr;

	GUID EncoderGuid = GetEncoderGuidBasedOnFormat(Format);
	if (InSample->Rotation != EMediaOrientation::Original && EncoderGuid != GUID_ContainerFormatJpeg)
	{
		Result = WindowsImagingFactory->CreateBitmapFlipRotator(&Rotator);
		WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Open_FailedToCreateRotator", "Failed to create Rotator object"));

		Result = Rotator->Initialize(Source, Private::ConvertToWindowsTransformOptions(InSample->Rotation));
		WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Append_FailedToInitializeRotator", "Failed to initialize rotator from memory"));

		Source = Rotator;
	}
	
	FString FrameFileName = CreateFrameFileName();
	FString Path = FPaths::SetExtension(Directory / FrameFileName, Format);

	TComPtr<IWICStream> Stream;
	Result = WindowsImagingFactory->CreateStream(&Stream);
	WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Append_FailedToCreateStream", "Failed to create stream object used for writing the frame"));

	Result = Stream->InitializeFromFilename(*Path, GENERIC_WRITE);
	WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, FText::Format(LOCTEXT("Append_FailedToInitializeStream", "Failed to initialize stream object based of the filename: {0}"),
														 FText::FromString(Path)));
	TComPtr<IWICBitmapEncoder> Encoder;

	Result = WindowsImagingFactory->CreateEncoder(EncoderGuid, nullptr, &Encoder);
	WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, FText::Format(LOCTEXT("Open_FailedToCreateEncoder", "Failed to create Encoder object for format {0}"),
														 FText::FromString(Format)));

	Result = Encoder->Initialize(Stream, WICBitmapEncoderNoCache);
	WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, FText::Format(LOCTEXT("Append_FailedToInitializeEncoder", "Failed to initialize encoder for format: {0}"), FText::FromString(Format)));

	IPropertyBag2* PropertyBag = nullptr;
	TComPtr<IWICBitmapFrameEncode> EncodedFrame;
	Result = Encoder->CreateNewFrame(&EncodedFrame, &PropertyBag);
	WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Append_FailedToCreateEncodedFrame", "Failed to create encoded frame"));

	if (EncoderGuid == GUID_ContainerFormatJpeg)
	{
		FString PropertyName = TEXT("BitmapTransform");
		PROPBAG2 Options = { 0 };
		Options.pstrName = PropertyName.GetCharArray().GetData();
		VARIANT Variant;
		VariantInit(&Variant);
		Variant.vt = VT_UI1;
		Variant.bVal = Private::ConvertToWindowsTransformOptions(InSample->Rotation);
		constexpr ULONG NumberOfProperties = 1;
		PropertyBag->Write(NumberOfProperties, &Options, &Variant);
	}

	Result = EncodedFrame->Initialize(PropertyBag);
	WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Append_FailedToInitializeEncodedFrame", "Failed to initialize encoded frame"));

	FIntPoint NewDimensions = Private::GetNewDimensions(InSample);
	Result = EncodedFrame->SetSize(NewDimensions.X, NewDimensions.Y);

	Result = EncodedFrame->SetPixelFormat(&PixelFormat);
	WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Append_FailedToSetPixelFormat", "Failed to configure pixel format on the encoded frame"));

	Result = EncodedFrame->WriteSource(Source, nullptr);
	WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Append_FailedToWriteData", "Failed to write the data to the encoded frame"));

	Result = EncodedFrame->Commit();
	WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Append_FailedToFinalizeConfiguring", "Failed to finalize configuring encoded frame"));

	Result = Encoder->Commit();
	WINIW_CHECK_AND_RETURN_ERROR_MESSAGE(Result, LOCTEXT("Append_FailedToFinalize", "Failed to finalize encoding"));

	++FrameNumber;

	return {};
}

GUID FWindowsImageWriter::GetEncoderGuidBasedOnFormat(const FString& InFormat)
{
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
	EImageFormat ImageFormat = ImageWrapperModule.GetImageFormatFromExtension(*InFormat);

	if (ImageFormat == EImageFormat::JPEG)
	{
		return GUID_ContainerFormatJpeg;
	}
	else if (ImageFormat == EImageFormat::PNG)
	{
		return GUID_ContainerFormatPng;
	}

	return UnsupportedEncoder;
}

FString FWindowsImageWriter::CreateFrameFileName()
{
	FString FrameIndexStr = FString::Printf(TEXT("%06d"), FrameNumber);

	return FString::Format(TEXT("{0}_{1}"), { FileName, FrameIndexStr });
}

#undef LOCTEXT_NAMESPACE

#endif // PLATFORM_WINDOWS && !UE_SERVER
