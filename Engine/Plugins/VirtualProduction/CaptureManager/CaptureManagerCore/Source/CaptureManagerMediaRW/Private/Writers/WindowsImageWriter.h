// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"
#include "HAL/Platform.h"

#if PLATFORM_WINDOWS && !UE_SERVER

#include "Windows/WindowsHWrapper.h"
#include "Windows/AllowWindowsPlatformTypes.h"

THIRD_PARTY_INCLUDES_START

#include <windows.h>
#include <mfapi.h>
#include <mfidl.h>
#include <wincodec.h>
#include "Microsoft/COMPointer.h"

THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformTypes.h"

#include "IMediaWriter.h"
#include "IMediaRWFactory.h"

class FWindowsImageWriterFactory final : public IImageWriterFactory
{
public:

	virtual TUniquePtr<IImageWriter> CreateImageWriter() override;
};

class FWindowsImageWriter final : public IImageWriter
{
public:

	FWindowsImageWriter();
	virtual ~FWindowsImageWriter() override;

	virtual TOptional<FText> Open(const FString& InDirectory, const FString& InFileName, const FString& InFormat) override;
	virtual TOptional<FText> Close() override;

	virtual TOptional<FText> Append(UE::CaptureManager::FMediaTextureSample* InSample) override;

private:

	GUID GetEncoderGuidBasedOnFormat(const FString& InFormat);
	FString CreateFrameFileName();

	TComPtr<IWICImagingFactory> WindowsImagingFactory;

	FString Directory;
	FString FileName;
	FString Format;

	uint32 FrameNumber = 0;
};

#endif // PLATFORM_WINDOWS && !UE_SERVER