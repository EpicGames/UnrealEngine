// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

#include "MediaRWManager.h"

#define UE_API CAPTUREMANAGERMEDIARW_API

class FCaptureManagerMediaRWModule : public IModuleInterface
{
public:

	UE_API void StartupModule();
	UE_API void ShutdownModule();

	UE_API FMediaRWManager& Get();

private:

	TUniquePtr<FMediaRWManager> MediaRWManager;
};

#undef UE_API