// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaReader.h"
#include "IMediaWriter.h"

class IAudioReaderFactory
{
public:

	virtual ~IAudioReaderFactory() = default;

	virtual TUniquePtr<IAudioReader> CreateAudioReader() = 0;
};

class IVideoReaderFactory
{
public:

	virtual ~IVideoReaderFactory() = default;

	virtual TUniquePtr<IVideoReader> CreateVideoReader() = 0;
};

class ICalibrationReaderFactory
{
public:

	virtual ~ICalibrationReaderFactory() = default;

	virtual TUniquePtr<ICalibrationReader> CreateCalibrationReader() = 0;
};

class IAudioWriterFactory
{
public:

	virtual ~IAudioWriterFactory() = default;

	virtual TUniquePtr<IAudioWriter> CreateAudioWriter() = 0;
};

class IImageWriterFactory
{
public:

	virtual ~IImageWriterFactory() = default;

	virtual TUniquePtr<IImageWriter> CreateImageWriter() = 0;
};

class ICalibrationWriterFactory
{
public:

	virtual ~ICalibrationWriterFactory() = default;

	virtual TUniquePtr<ICalibrationWriter> CreateCalibrationWriter() = 0;
};