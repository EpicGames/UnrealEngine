// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaSample.h"

class IAudioReader
{
public:

	virtual ~IAudioReader() = default;

	virtual TOptional<FText> Open(const FString& InFileName) = 0;
	virtual TOptional<FText> Close() = 0;

	virtual TValueOrError<TUniquePtr<UE::CaptureManager::FMediaAudioSample>, FText> Next() = 0;

	virtual FTimespan GetDuration() const = 0;
	virtual EMediaAudioSampleFormat GetSampleFormat() const = 0;
	virtual UE::CaptureManager::ESampleRate GetSampleRate() const = 0;
	virtual uint32 GetNumChannels() const = 0;
};

class IVideoReader
{
public:

	virtual ~IVideoReader() = default;

	virtual TOptional<FText> Open(const FString& InFileName) = 0;
	virtual TOptional<FText> Close() = 0;

	virtual TValueOrError<TUniquePtr<UE::CaptureManager::FMediaTextureSample>, FText> Next() = 0;

	virtual FTimespan GetDuration() const = 0;
	virtual FIntPoint GetDimensions() const = 0;
	virtual FFrameRate GetFrameRate() const = 0;
};

class ICalibrationReader
{
public:

	virtual ~ICalibrationReader() = default;

	virtual TOptional<FText> Open(const FString& InFileName) = 0;
	virtual TOptional<FText> Close() = 0;

	virtual TValueOrError<TUniquePtr<UE::CaptureManager::FMediaCalibrationSample>, FText> Next() = 0;
};