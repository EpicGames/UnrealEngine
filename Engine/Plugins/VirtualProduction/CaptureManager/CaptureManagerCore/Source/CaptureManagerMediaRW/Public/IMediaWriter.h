// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaSample.h"

class IAudioWriter
{
public:

	virtual ~IAudioWriter() = default;

	// FileName is the name of file without the extensions while the Format represents an extension without the dot
	virtual TOptional<FText> Open(const FString& InDirectory, const FString& InFileName, const FString& InFormat) = 0;
	virtual TOptional<FText> Close() = 0;

	virtual TOptional<FText> Append(UE::CaptureManager::FMediaAudioSample* Sample) = 0;

	void Configure(UE::CaptureManager::ESampleRate InSampleRate, int32 InNumChannels, EMediaAudioSampleFormat InBitsPerSample)
	{
		SampleRate = InSampleRate;
		NumChannels = InNumChannels;
		BitsPerSample = InBitsPerSample;
	}

protected:

	UE::CaptureManager::ESampleRate SampleRate = UE::CaptureManager::ESampleRate::SR_44100Hz;
	int32 NumChannels = 1;
	EMediaAudioSampleFormat BitsPerSample = EMediaAudioSampleFormat::Int16;
};

class IImageWriter
{
public:

	virtual ~IImageWriter() = default;

	// FileName is the name of file without the extensions while the Format represents an extension without the dot
	virtual TOptional<FText> Open(const FString& InDirectory, const FString& InFileName, const FString& InFormat) = 0;
	virtual TOptional<FText> Close() = 0;

	virtual TOptional<FText> Append(UE::CaptureManager::FMediaTextureSample* InSample) = 0;
};

class ICalibrationWriter
{
public:

	virtual ~ICalibrationWriter() = default;

	// FileName is the name of file without the extensions while the Format represents an extension without the dot
	virtual TOptional<FText> Open(const FString& InDirectory, const FString& InFileName, const FString& InFormat) = 0;
	virtual TOptional<FText> Close() = 0;

	virtual TOptional<FText> Append(UE::CaptureManager::FMediaCalibrationSample* InSample) = 0;
};