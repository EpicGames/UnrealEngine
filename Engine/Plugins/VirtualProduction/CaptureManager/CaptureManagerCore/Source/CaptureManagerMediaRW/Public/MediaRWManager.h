// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaRWFactory.h"

#include "Containers/Map.h"

#define UE_API CAPTUREMANAGERMEDIARW_API

class FMediaRWManager
{
public:

	UE_API FMediaRWManager();

	FMediaRWManager(const FMediaRWManager& InCopy) = delete;
	FMediaRWManager& operator=(const FMediaRWManager& InCopy) = delete;

	UE_API void RegisterAudioReader(const TArray<FString>& InFormats, TUniquePtr<IAudioReaderFactory> InReader);
	UE_API void RegisterVideoReader(const TArray<FString>& InFormats, TUniquePtr<IVideoReaderFactory> InReader);
	UE_API void RegisterCalibrationReader(const TArray<FString>& InFormats, TUniquePtr<ICalibrationReaderFactory> InReader);

	UE_API void RegisterAudioWriter(const TArray<FString>& InFormats, TUniquePtr<IAudioWriterFactory> InWriter);
	UE_API void RegisterImageWriter(const TArray<FString>& InFormats, TUniquePtr<IImageWriterFactory> InWriter);
	UE_API void RegisterCalibrationWriter(const TArray<FString>& InFormats, TUniquePtr<ICalibrationWriterFactory> InWriter);

	UE_API TUniquePtr<IAudioReader> CreateAudioReaderByFormat(const FString& InFormat, int32 InIndex = 0);
	UE_API TUniquePtr<IVideoReader> CreateVideoReaderByFormat(const FString& InFormat, int32 InIndex = 0);
	UE_API TUniquePtr<ICalibrationReader> CreateCalibrationReaderByFormat(const FString& InFormat, int32 InIndex = 0);

	UE_API TValueOrError<TUniquePtr<IAudioReader>, FText> CreateAudioReader(const FString& InPath, int32 InIndex = 0);
	UE_API TValueOrError<TUniquePtr<IVideoReader>, FText> CreateVideoReader(const FString& InPath, int32 InIndex = 0);
	UE_API TValueOrError<TUniquePtr<ICalibrationReader>, FText> CreateCalibrationReader(const FString& InPath, int32 InIndex = 0);

	UE_API TUniquePtr<IAudioWriter> CreateAudioWriterByFormat(const FString& InFormat, int32 InIndex = 0);
	UE_API TUniquePtr<IImageWriter> CreateImageWriterByFormat(const FString& InFormat, int32 InIndex = 0);
	UE_API TUniquePtr<ICalibrationWriter> CreateCalibrationWriterByFormat(const FString& InFormat, int32 InIndex = 0);

	UE_API TValueOrError<TUniquePtr<IAudioWriter>, FText> CreateAudioWriter(const FString& InDirectory, const FString& InFileName, const FString& InFormat, int32 InIndex = 0);
	UE_API TValueOrError<TUniquePtr<IImageWriter>, FText> CreateImageWriter(const FString& InDirectory, const FString& InFileName, const FString& InFormat, int32 InIndex = 0);
	UE_API TValueOrError<TUniquePtr<ICalibrationWriter>, FText> CreateCalibrationWriter(const FString& InDirectory, const FString& InFileName, const FString& InFormat, int32 InIndex = 0);

private:

	TMap<FString, IAudioReaderFactory*> AudioReadersPerFormat;
	TArray<TUniquePtr<IAudioReaderFactory>> AudioReaders;

	TMap<FString, IVideoReaderFactory*> VideoReadersPerFormat;
	TArray<TUniquePtr<IVideoReaderFactory>> VideoReaders;

	TMap<FString, ICalibrationReaderFactory*> CalibrationReadersPerFormat;
	TArray<TUniquePtr<ICalibrationReaderFactory>> CalibrationReaders;

	TMap<FString, IAudioWriterFactory*> AudioWritersPerFormat;
	TArray<TUniquePtr<IAudioWriterFactory>> AudioWriters;

	TMap<FString, IImageWriterFactory*> ImageWritersPerFormat;
	TArray<TUniquePtr<IImageWriterFactory>> ImageWriters;

	TMap<FString, ICalibrationWriterFactory*> CalibrationWritersPerFormat;
	TArray<TUniquePtr<ICalibrationWriterFactory>> CalibrationWriters;
};

#undef UE_API