// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaAudioSample.h"
#include "IMediaTextureSample.h"

#define UE_API CAPTUREMANAGERMEDIARW_API

namespace UE::CaptureManager
{

enum class ESampleRate
{
	SR_8000Hz = 0,
	SR_16000Hz,
	SR_44100Hz,
	SR_48000Hz,
	SR_88200Hz,
	SR_96000Hz,
	SR_192000Hz
};

struct FMediaAudioSample
{
	TArray<uint8> Buffer;
	uint32 Channels;
	FTimespan Duration;
	EMediaAudioSampleFormat SampleFormat;
	uint32 Frames;
	ESampleRate SampleRate;
	FTimespan Time;
};

UE_API int32 ConvertBitsPerSample(EMediaAudioSampleFormat InSampleFormat);
UE_API int32 ConvertSampleRate(ESampleRate InSampleRate);

UE_API ESampleRate ConvertSampleRate(int32 InSampleRate);

enum class EMediaTexturePixelFormat
{
	Undefined = -1,
	U8_RGB = 0,
	U8_BGR,
	U8_RGBA,
	U8_BGRA,
	U8_I444,
	U8_I420,
	U8_YUY2,
	U8_NV12,
	U8_Mono,
	U16_Mono,
	F_Mono,

	Default = U8_RGB
};

struct FMediaTextureSample
{
	TArray<uint8> Buffer;
	int32 Stride = -1;
	FIntPoint Dimensions;
	FTimespan Duration;
	EMediaTexturePixelFormat CurrentFormat;
	EMediaTexturePixelFormat DesiredFormat;
	FTimespan Time;
	EMediaOrientation Orientation;
	EMediaOrientation Rotation;
};

UE_API uint32 GetNumberOfChannels(EMediaTexturePixelFormat InPixelFormat);

struct FOpenCVDistortionModel
{
	struct FRadial
	{
		double K1 = 0.0;
		double K2 = 0.0;
		double K3 = 0.0;
	};

	struct FTangential
	{
		double P1 = 0.0;
		double P2 = 0.0;
	};

	FRadial Radial;
	FTangential Tangential;
};

struct FIphoneDistortionModel
{
	TArray<double> LensDistortionTable;
	TArray<double> InverseLensDistortionTable;
};

class FCoordinateSystem
{
public:

	enum FDirection
	{
		Front = 0,
		Back,
		Right,
		Left,
		Up,
		Down
	};

	UE_API FCoordinateSystem();
	UE_API FCoordinateSystem(FMatrix InMatDescription);
	UE_API FCoordinateSystem(FDirection InXDirection, FDirection InYDirection, FDirection InZDirection);

	UE_API const FMatrix& GetMatDescription() const;

private:

	static int32 GetIndex(FDirection InDirection);
	static int32 GetSign(FDirection InDirection);

	FMatrix MatDescription;
};

inline bool operator==(const FCoordinateSystem& InLeft, const FCoordinateSystem& InRight)
{
	return InLeft.GetMatDescription() == InRight.GetMatDescription();
}

extern UE_API const FCoordinateSystem UnrealCS;
extern UE_API const FCoordinateSystem OpenCvCS;

struct FMediaCalibrationSample
{
	enum ECameraType
	{
		Video = 0,
		Depth,

		Unknown
	};

	FString CameraId;
	ECameraType CameraType = ECameraType::Unknown;

	FVector2D FocalLength;
	FVector2D PrincipalPoint;
	FTransform Transform;

	FIntPoint Dimensions;
	EMediaOrientation Orientation;

	TVariant<FEmptyVariantState, FIphoneDistortionModel, FOpenCVDistortionModel> DistortionModel;
	
	FCoordinateSystem InputCoordinateSystem;
};

UE_API FVector ConvertToCoordinateSystem(const FVector& InVector, 
															const FCoordinateSystem& InInputCoordinateSystem, 
															const FCoordinateSystem& InOutputCoordinateSystem);

UE_API FTransform ConvertToCoordinateSystem(const FTransform& InTransform,
															   const FCoordinateSystem& InInputCoordinateSystem,
															   const FCoordinateSystem& InOutputCoordinateSystem);


}

#undef UE_API
