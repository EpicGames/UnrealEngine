// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaSample.h"

namespace UE::CaptureManager
{

CAPTUREMANAGERMEDIARW_API TArray<uint8> ConvertYUVToMono(const UE::CaptureManager::FMediaTextureSample* InSample, bool InScaleRange); // Any YUV to Mono
CAPTUREMANAGERMEDIARW_API TArray<uint8> ConvertYUY2ToMono(const UE::CaptureManager::FMediaTextureSample* InSample, bool InScaleRange);
CAPTUREMANAGERMEDIARW_API TArray<uint8> ConvertI420ToBGRA(const UE::CaptureManager::FMediaTextureSample* InSample);
CAPTUREMANAGERMEDIARW_API TArray<uint8> ConvertNV12ToBGRA(const UE::CaptureManager::FMediaTextureSample* InSample);
CAPTUREMANAGERMEDIARW_API TArray<uint8> ConvertYUY2ToBGRA(const UE::CaptureManager::FMediaTextureSample* InSample);

CAPTUREMANAGERMEDIARW_API TArray<FColor> UEConvertYUVToMono(const UE::CaptureManager::FMediaTextureSample* InSample, bool InScaleRange); // Any YUV to Mono
CAPTUREMANAGERMEDIARW_API TArray<FColor> UEConvertYUY2ToMono(const UE::CaptureManager::FMediaTextureSample* InSample, bool InScaleRange);
CAPTUREMANAGERMEDIARW_API TArray<FColor> UEConvertI420ToBGRA(const UE::CaptureManager::FMediaTextureSample* InSample);
CAPTUREMANAGERMEDIARW_API TArray<FColor> UEConvertNV12ToBGRA(const UE::CaptureManager::FMediaTextureSample* InSample);
CAPTUREMANAGERMEDIARW_API TArray<FColor> UEConvertYUY2ToBGRA(const UE::CaptureManager::FMediaTextureSample* InSample);

}