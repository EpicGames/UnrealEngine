// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CaptureManagerPipeline : ModuleRules
{
	public CaptureManagerPipeline(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[]
		{
			"CaptureManagerTakeMetadata"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"ControlFlows",
			"Core",
			"CoreUObject",
			"ImageWrapper",
			"CaptureUtils",
			"Projects"
		});
	}
}
