// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerPipeline.h"

#include "Async/Monitor.h"
#include "ControlFlowConcurrency.h"
#include "Logging/LogMacros.h"
#include "Templates/ValueOrError.h"
#include "Internationalization/Internationalization.h"

#include "ControlFlow.h"

#define LOCTEXT_NAMESPACE "CaptureManagerPipeline"

DEFINE_LOG_CATEGORY_STATIC(LogCaptureManagerPipeline, Log, All);

class FCaptureManagerPipelineImpl
{
public:

	FCaptureManagerPipelineImpl(const FString& InName)
		: MainControlFlow(MakeShared<FControlFlow>(TEXT("CaptureManagerPipeline")))
	{
	}

	FSimpleMulticastDelegate& OnFlowCancel()
	{
		return MainControlFlow->OnFlowCancel();
	}

	FConcurrentFlowsDefiner& QueueConcurrentFlows(const FString& InName)
	{
		return MainControlFlow->QueueConcurrentFlows(InName);
	}

	FControlFlowWaitDelegate& QueueWait(const FString& InName)
	{
		return MainControlFlow->QueueWait(InName);
	}

	void ExecuteFlow()
	{
		MainControlFlow->ExecuteFlow();
	}

	bool IsRunning() const
	{
		return MainControlFlow->IsRunning();
	}

	void CancelFlow()
	{
		MainControlFlow->CancelFlow();
	}

	void Reset()
	{
		MainControlFlow->Reset();
	}

private:

	TSharedPtr<FControlFlow> MainControlFlow;
};

FCaptureManagerPipeline::FCaptureManagerPipeline(EPipelineExecutionPolicy InExecutionPolicy)
	: Impl(MakePimpl<FCaptureManagerPipelineImpl>(TEXT("CaptureManagerPipeline")))
	, ExecutionPolicy(InExecutionPolicy)
{
}

FCaptureManagerPipeline::~FCaptureManagerPipeline() = default;

FGuid FCaptureManagerPipeline::AddGenericNode(TSharedPtr<FCaptureManagerPipelineNode> InNode)
{
	return AddParallelPipelineNode(MoveTemp(InNode));
}

FGuid FCaptureManagerPipeline::AddConvertVideoNode(TSharedPtr<FConvertVideoNode> InNode)
{
	return AddParallelPipelineNode(MoveTemp(InNode));
}

FGuid FCaptureManagerPipeline::AddConvertAudioNode(TSharedPtr<FConvertAudioNode> InNode)
{
	return AddParallelPipelineNode(MoveTemp(InNode));
}

FGuid FCaptureManagerPipeline::AddConvertDepthNode(TSharedPtr<FConvertDepthNode> InNode)
{
	return AddParallelPipelineNode(MoveTemp(InNode));
}

FGuid FCaptureManagerPipeline::AddConvertCalibrationNode(TSharedPtr<FConvertCalibrationNode> InNode)
{
	return AddParallelPipelineNode(MoveTemp(InNode));
}

FGuid FCaptureManagerPipeline::AddSyncedNode(TSharedPtr<FCaptureManagerPipelineNode> InNode)
{
	FGuid UniqueId = FGuid::NewGuid();

	SyncNodes.Lock()->Add(UniqueId, MoveTemp(InNode));

	return UniqueId;
}

FGuid FCaptureManagerPipeline::AddParallelPipelineNode(TSharedPtr<FCaptureManagerPipelineNode> InNode)
{
	FGuid UniqueId = FGuid::NewGuid();

	ParallelNodes.Lock()->Add(UniqueId, MoveTemp(InNode));

	return UniqueId;
}

FCaptureManagerPipeline::FResult FCaptureManagerPipeline::Run()
{
	using namespace UE::CaptureManager;

	Impl->Reset();

	TMonitor<FResult> ThreadSafeResult;

	FSimpleMulticastDelegate::FDelegate CanceledDelegate = FSimpleMulticastDelegate::FDelegate::CreateLambda([this, &ThreadSafeResult]()
	{	
		FText Message = LOCTEXT("Run_Canceled", "The pipeline has been canceled");
		int32 Code = -10;

		TArray<FGuid> ParellelNodeIds, SyncNodeIds;
		{
			ParallelNodes.Lock()->GetKeys(ParellelNodeIds);
			SyncNodes.Lock()->GetKeys(SyncNodeIds);
		}

		for (const FGuid& UniqueId : ParellelNodeIds)
		{
			ThreadSafeResult.Lock()->Add(UniqueId, MakeError(MoveTemp(Message), Code));
		}

		for (const FGuid& UniqueId : SyncNodeIds)
		{
			ThreadSafeResult.Lock()->Add(UniqueId, MakeError(MoveTemp(Message), Code));
		}

		UE_LOGF(LogCaptureManagerPipeline, Warning, "Pipeline canceled");
	});

	Impl->OnFlowCancel().Add(MoveTemp(CanceledDelegate));
	Impl->QueueConcurrentFlows(TEXT("MainNode")).BindLambda([this, &ThreadSafeResult](TSharedRef<FConcurrentControlFlows> InConcurrentFlow) mutable
	{
		EConcurrentExecution Execution =
			ExecutionPolicy == EPipelineExecutionPolicy::Synchronous ? EConcurrentExecution::Default : EConcurrentExecution::Parallel;

		InConcurrentFlow->SetExecution(Execution);

		int32 Index = 0;

		TArray<FGuid> NodeIds;
		TMonitor<FNodeMap>::FHelper ParallelNodesHelper = ParallelNodes.Lock(); 
		ParallelNodesHelper->GetKeys(NodeIds);

		for (const FGuid& UniqueId : NodeIds)
		{
			TSharedPtr<FCaptureManagerPipelineNode> Node = ParallelNodesHelper->FindChecked(UniqueId);

			FString Name = Node->GetName();
			InConcurrentFlow->AddOrGetFlow(Index, Name)
				.QueueWait(Name).BindLambda([this, Index, Id = UniqueId, ExeNode = MoveTemp(Node), &ThreadSafeResult](FControlFlowNodeRef InSubFlow) mutable
			{
				FCaptureManagerPipelineNode::FResult Result = ExeNode->Execute();
				ThreadSafeResult.Lock()->Add(MoveTemp(Id), Result);

				ParallelNodes.Lock()->Remove(Id);

				if (InSubFlow->HasCancelBeenRequested())
				{
					return;
				}

				if (Result.HasValue())
				{
					InSubFlow->ContinueFlow();
				}
				else
				{
					InSubFlow->CancelFlow();
				}
			});

			++Index;
		}
	});

	{
		TMonitor<FNodeMap>::FHelper SyncNodesHelper = SyncNodes.Lock();
		TArray<FGuid> SyncedNodeIds;
		SyncNodesHelper->GetKeys(SyncedNodeIds);

		for (const FGuid& UniqueId : SyncedNodeIds)
		{
			TSharedPtr<FCaptureManagerPipelineNode> Node = SyncNodesHelper->FindChecked(UniqueId);
			FString Name = Node->GetName();

			Impl->QueueWait(Name).BindLambda([this, Id = UniqueId, ExeNode = MoveTemp(Node), &ThreadSafeResult](FControlFlowNodeRef InSubFlow) mutable
			{
				FCaptureManagerPipelineNode::FResult Result = ExeNode->Execute();
				ThreadSafeResult.Lock()->Add(MoveTemp(Id), Result);

				SyncNodes.Lock()->Remove(Id);

				if (InSubFlow->HasCancelBeenRequested())
				{
					return;
				}

				if (Result.HasValue())
				{
					InSubFlow->ContinueFlow();
				}
				else
				{
					InSubFlow->CancelFlow();
				}
			});
		}
	}

	Impl->ExecuteFlow();

	UE_LOGF(LogCaptureManagerPipeline, Display, "Data conversion pipeline completed");

	return ThreadSafeResult.Claim();
}

void FCaptureManagerPipeline::Cancel()
{
	if (Impl->IsRunning())
	{
		for (const TPair<FGuid, TSharedPtr<FCaptureManagerPipelineNode>>& NodePair : *ParallelNodes.Lock())
		{
			NodePair.Value->Cancel();
		}

		for (const TPair<FGuid, TSharedPtr<FCaptureManagerPipelineNode>>& NodePair : *SyncNodes.Lock())
		{
			NodePair.Value->Cancel();
		}

		Impl->CancelFlow();
	}
}

#undef LOCTEXT_NAMESPACE