// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

using FCaptureManagerPipelineModule = IModuleInterface;

IMPLEMENT_MODULE(FCaptureManagerPipelineModule, CaptureManagerPipeline);
