// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerPipelineNode.h"

#include "ControlFlow.h"

#include "Templates/Function.h"

class FCaptureManagerPipelineNode::FCaptureManagerPipelineNodeImpl
{
private:

	DECLARE_DELEGATE_RetVal(FResult, FStep);

public:

	FCaptureManagerPipelineNodeImpl(const FString& InName, FCaptureManagerPipelineNode* InOwner)
		: ControlFlow(MakeShared<FControlFlow>(*InName))
		, Owner(InOwner)
	{
	}

	FString GetName() const
	{
		return ControlFlow->GetDebugName();
	}

	FCaptureManagerPipelineNode::FResult Execute()
	{
		FResult Result = MakeValue();

		ControlFlow->QueueWait(TEXT("Prepare")).BindRaw(this, &FCaptureManagerPipelineNodeImpl::PreparePrivate, TRetainedRef<FResult>(Result));
		ControlFlow->QueueWait(TEXT("Run")).BindRaw(this, &FCaptureManagerPipelineNodeImpl::RunPrivate, TRetainedRef<FResult>(Result));
		ControlFlow->QueueWait(TEXT("Validate")).BindRaw(this, &FCaptureManagerPipelineNodeImpl::ValidatePrivate, TRetainedRef<FResult>(Result));

		ControlFlow->ExecuteFlow();

		return Result;
	}

	void Cancel()
	{
		if (ControlFlow->IsRunning())
		{
			ControlFlow->CancelFlow();
		}
	}

private:

	void PreparePrivate(FControlFlowNodeRef InFlowHandle, TRetainedRef<FResult> InResult)
	{
		ExecuteStepPrivate(FStep::CreateRaw(Owner, &FCaptureManagerPipelineNode::Prepare), MoveTemp(InFlowHandle), MoveTemp(InResult));
	}

	void RunPrivate(FControlFlowNodeRef InFlowHandle, TRetainedRef<FResult> InResult)
	{
		ExecuteStepPrivate(FStep::CreateRaw(Owner, &FCaptureManagerPipelineNode::Run), MoveTemp(InFlowHandle), MoveTemp(InResult));
	}

	void ValidatePrivate(FControlFlowNodeRef InFlowHandle, TRetainedRef<FResult> InResult)
	{
		ExecuteStepPrivate(FStep::CreateRaw(Owner, &FCaptureManagerPipelineNode::Validate), MoveTemp(InFlowHandle), MoveTemp(InResult));
	}

	void ExecuteStepPrivate(FStep InStep, FControlFlowNodeRef InFlowHandle, TRetainedRef<FResult> InResult)
	{
		FResult& Result = InResult.Get();
		Result = InStep.Execute();
		
		if (InFlowHandle->HasCancelBeenRequested())
		{
			return;
		}

		if (Result.HasValue())
		{
			InFlowHandle->ContinueFlow();
		}
		else
		{
			InFlowHandle->CancelFlow();
		}
	}

	TSharedPtr<FControlFlow> ControlFlow;
	FCaptureManagerPipelineNode* Owner;
};

FCaptureManagerPipelineError::FCaptureManagerPipelineError(FText InMessage, int32 InCode)
	: Message(MoveTemp(InMessage))
	, Code(InCode)
{
}

const FText& FCaptureManagerPipelineError::GetMessage() const
{
	return Message;
}

int32 FCaptureManagerPipelineError::GetCode() const
{
	return Code;
}

FCaptureManagerPipelineNode::FCaptureManagerPipelineNode(const FString& InName)
	: Impl(MakePimpl<FCaptureManagerPipelineNodeImpl>(InName, this))
{
}

FCaptureManagerPipelineNode::~FCaptureManagerPipelineNode() = default;

FString FCaptureManagerPipelineNode::GetName() const
{
	return Impl->GetName();
}

FCaptureManagerPipelineNode::FResult FCaptureManagerPipelineNode::Execute()
{
	return Impl->Execute();
}

void FCaptureManagerPipelineNode::Cancel()
{
	Impl->Cancel();
}