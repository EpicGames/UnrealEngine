// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/ConvertAudioNode.h"

#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "Misc/Paths.h"

#include "Modules/ModuleManager.h"

#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "ConvertAudioNode"

static const FString AudioDirectory = TEXT("Audio");
static const TArray<FString> SupportedFormats =
{
	TEXT("wav")
};

FConvertAudioNode::FConvertAudioNode(const FTakeMetadata::FAudio& InAudio,
									 const FString& InOutputDirectory)
	: FCaptureManagerPipelineNode(TEXT("ConvertAudioNode"))
	, Audio(InAudio)
{
	OutputDirectory = InOutputDirectory / AudioDirectory;
}

FConvertAudioNode::~FConvertAudioNode() = default;

FConvertAudioNode::FResult FConvertAudioNode::Prepare()
{
	IFileManager& FileManager = IFileManager::Get();

	if (!FPaths::DirectoryExists(*OutputDirectory))
	{
		bool bCreated = FileManager.MakeDirectory(*OutputDirectory, true);
		if (!bCreated)
		{
			FText Message = FText::Format(LOCTEXT("ConvertAudioNode_Prepare_DirectoryMissing", "Failed to create the base directory {0}"), FText::FromString(OutputDirectory));
			return MakeError(MoveTemp(Message));
		}
	}

	FString AudioPath = GetAudioDirectory();
	FileManager.MakeDirectory(*AudioPath, false);

	return MakeValue();
}

FConvertAudioNode::FResult FConvertAudioNode::Validate()
{
	FString AudioPath = GetAudioDirectory();
	FConvertAudioNode::FResult Result = CheckForAudioFile(AudioPath);

	if (Result.HasError())
	{
		return Result;
	}

	return MakeValue();
}

FString FConvertAudioNode::GetAudioDirectory() const
{
	return OutputDirectory / Audio.Name;
}

FConvertAudioNode::FResult FConvertAudioNode::CheckForAudioFile(const FString& InAudioDirectory)
{
	IFileManager& FileManager = IFileManager::Get();

	if (!FPaths::DirectoryExists(InAudioDirectory))
	{
		FText Message = LOCTEXT("ConvertAudioNode_Validate_DirectoryMissing", "The output directory is missing");
		return MakeError(MoveTemp(Message));
	}

	FText ResultMessage;
	bool bDirectoryIsEmpty = true;
	bool bFilesAreValid = FileManager.IterateDirectory(*InAudioDirectory, [&ResultMessage, &bDirectoryIsEmpty](const TCHAR* InFileName, bool bIsDirectory)
	{
		bDirectoryIsEmpty = false;

		if (bIsDirectory)
		{
			ResultMessage = FText::Format(
				LOCTEXT("ConvertAudioNode_Validate_UnexpectedDirectory", "Unexpected directory found: {0}"), FText::FromString(FPaths::GetPathLeaf(InFileName)));

			return false;
		}

		FString Extension = FPaths::GetExtension(InFileName);

		if (!SupportedFormats.Contains(Extension))
		{
			FString SupportedFormatsString = FString::Join(SupportedFormats, TEXT(","));

			ResultMessage = FText::Format(
				LOCTEXT("ConvertAudioNode_Validate_InvalidFormat", "Unsupported audio file format: {0}, expected {1}"),
				FText::FromString(Extension),
				FText::FromString(SupportedFormatsString));

			return false;
		}
		
		return true;
	});

	if (bDirectoryIsEmpty)
	{
		FText Message = FText::Format(LOCTEXT("ConvertVideoNode_Validate_EmptyDirectory", "Folder is empty: {0}"), FText::FromString(InAudioDirectory));
		return MakeError(MoveTemp(Message));
	}

	if (!bFilesAreValid)
	{
		return MakeError(MoveTemp(ResultMessage));
	}

	return MakeValue();
}

#undef LOCTEXT_NAMESPACE