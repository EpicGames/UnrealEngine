// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/ConvertCalibrationNode.h"

#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "Misc/Paths.h"

#include "Internationalization/Text.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "ConvertCalibrationNode"

static const FString CalibrationDirectory = TEXT("Calibration");

FConvertCalibrationNode::FConvertCalibrationNode(const FTakeMetadata::FCalibration& InCalibration,
												 const FString& InOutputDirectory)
	: FCaptureManagerPipelineNode(TEXT("ConvertCalibrationNode"))
	, Calibration(InCalibration)
{
	OutputDirectory = InOutputDirectory / CalibrationDirectory;
}

FConvertCalibrationNode::FResult FConvertCalibrationNode::Prepare()
{
	IFileManager& FileManager = IFileManager::Get();

	if (!FPaths::DirectoryExists(*OutputDirectory))
	{
		bool bCreated = FileManager.MakeDirectory(*OutputDirectory, true);
		if (!bCreated)
		{
			FText Message = FText::Format(LOCTEXT("ConvertCalibrationNode_Prepare_BaseDirectoryMissing", "Failed to create the base calibration directory {0}"), FText::FromString(OutputDirectory));
			return MakeError(MoveTemp(Message));
		}
	}

	FString CalibrationPath = GetCalibrationDirectory();

	if (!FPaths::DirectoryExists(*CalibrationPath))
	{
		bool bCreated = FileManager.MakeDirectory(*CalibrationPath, false);
		if (!bCreated)
		{
			FText Message = FText::Format(LOCTEXT("ConvertCalibrationNode_Prepare_DirectoryMissing", "Failed to create the calibration directory {0}"), FText::FromString(CalibrationPath));
			return MakeError(MoveTemp(Message));
		}
	}

	return MakeValue();
}

FConvertCalibrationNode::FResult FConvertCalibrationNode::Validate()
{
	FString CalibrationPath = GetCalibrationDirectory();

	if (!FPaths::DirectoryExists(*CalibrationPath))
	{
		FText Message = FText::Format(LOCTEXT("ConvertCalibrationNode_Validate_DirectoryMissing", "Calibration directory doesn't exist {0}"), FText::FromString(CalibrationPath));
		return MakeError(MoveTemp(Message));
	}

	auto ContainsFile = [](const FString& InDirectory)
	{
		bool bContainsFile = false;

		IFileManager::Get()
			.IterateDirectory(*InDirectory, 
							  [&bContainsFile](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			if (bIsDirectory)
			{
				return true;
			}

			bContainsFile = true;

			return false;
		});

		return bContainsFile;
	};

	if (!ContainsFile(*CalibrationPath))
	{
		FText Message = FText::Format(LOCTEXT("ConvertCalibrationNode_Validate_FileMissing", "Calibration file doesn't exist in directory {0}"), FText::FromString(CalibrationPath));
		return MakeError(MoveTemp(Message));
	}

	return MakeValue();
}

FString FConvertCalibrationNode::GetCalibrationDirectory() const
{
	return OutputDirectory / Calibration.Name;
}

#undef LOCTEXT_NAMESPACE