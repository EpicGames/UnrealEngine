// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/ConvertDepthNode.h"

#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "Misc/Paths.h"

#include "IImageWrapperModule.h"
#include "Modules/ModuleManager.h"

#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "ConvertDepthNode"

static const FString DepthDirectory = TEXT("Depth");

FConvertDepthNode::FConvertDepthNode(const FTakeMetadata::FVideo& InDepth,
									 const FString& InOutputDirectory)
	: FCaptureManagerPipelineNode(TEXT("ConvertDepthNode"))
	, Depth(InDepth)
{
	OutputDirectory = InOutputDirectory / DepthDirectory;
}

FConvertDepthNode::~FConvertDepthNode() = default;

FConvertDepthNode::FResult FConvertDepthNode::Prepare()
{
	IFileManager& FileManager = IFileManager::Get();

	if (!FPaths::DirectoryExists(*OutputDirectory))
	{
		bool bCreated = FileManager.MakeDirectory(*OutputDirectory, true);
		if (!bCreated)
		{
			FText Message = FText::Format(LOCTEXT("ConvertDepthNode_Prepare_DirectoryMissing", "Failed to create the base directory {0}"), FText::FromString(OutputDirectory));
			return MakeError(MoveTemp(Message));
		}
	}

	FString DepthPath = GetDepthDirectory();
	FileManager.MakeDirectory(*DepthPath, false);

	return MakeValue();
}

FConvertDepthNode::FResult FConvertDepthNode::Validate()
{
	FString DepthPath = GetDepthDirectory();
	FConvertDepthNode::FResult Result = CheckImagesForDepth(DepthPath);

	if (Result.HasError())
	{
		return Result;
	}

	return MakeValue();
}

FString FConvertDepthNode::GetDepthDirectory() const
{
	return OutputDirectory / Depth.Name;
}

FConvertDepthNode::FResult FConvertDepthNode::CheckImagesForDepth(const FString& InDepthPath)
{
	IFileManager& FileManager = IFileManager::Get();
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));

	if (!FPaths::DirectoryExists(InDepthPath))
	{
		FText Message = LOCTEXT("ConvertDepthNode_Validate_DirectoryMissing", "The output directory is missing");
		return MakeError(MoveTemp(Message));
	}

	bool bDirectoryIsEmpty = true;
	bool bFilesAreValid = FileManager.IterateDirectory(*InDepthPath, [&ImageWrapperModule, &bDirectoryIsEmpty](const TCHAR* InFileName, bool bIsDirectory)
	{
		bDirectoryIsEmpty = false;

		if (bIsDirectory)
		{
			return false;
		}

		EImageFormat Format = ImageWrapperModule.GetImageFormatFromExtension(*FPaths::GetExtension(InFileName));
		return Format == EImageFormat::EXR;
	});

	if (bDirectoryIsEmpty)
	{
		FText Message = FText::Format(LOCTEXT("ConvertDepthNode_Validate_EmptyDirectory", "Folder is empty: {0}"), FText::FromString(InDepthPath));
		return MakeError(MoveTemp(Message));
	}

	if (!bFilesAreValid)
	{
		FText Message = LOCTEXT("ConvertDepthNode_Validate_InvalidFormat", "The images are in an unsupported format (supported format is EXR)");
		return MakeError(MoveTemp(Message));
	}

	return MakeValue();
}

#undef LOCTEXT_NAMESPACE