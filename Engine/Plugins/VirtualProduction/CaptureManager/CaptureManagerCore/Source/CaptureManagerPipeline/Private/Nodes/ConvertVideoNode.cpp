// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/ConvertVideoNode.h"

#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "Misc/Paths.h"

#include "IImageWrapperModule.h"
#include "Modules/ModuleManager.h"

#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "ConvertVideoNode"

static const FString VideoDirectory = TEXT("Video");

FConvertVideoNode::FConvertVideoNode(const FTakeMetadata::FVideo& InVideo,
									 const FString& InOutputDirectory)
	: FCaptureManagerPipelineNode(TEXT("ConvertVideoNode"))
	, Video(InVideo)
{
	OutputDirectory = InOutputDirectory / VideoDirectory;
}

FConvertVideoNode::~FConvertVideoNode() = default;

FConvertVideoNode::FResult FConvertVideoNode::Prepare()
{
	IFileManager& FileManager = IFileManager::Get();

	if (!FPaths::DirectoryExists(*OutputDirectory))
	{
		bool bCreated = FileManager.MakeDirectory(*OutputDirectory, true);
		if (!bCreated)
		{
			FText Message = FText::Format(LOCTEXT("ConvertVideoNode_Prepare_DirectoryMissing", "Failed to create the base directory {0}"), FText::FromString(OutputDirectory));
			return MakeError(MoveTemp(Message));
		}
	}

	FString VideoPath = GetVideoDirectory();
	FileManager.MakeDirectory(*VideoPath, false);

	return MakeValue();
}

FConvertVideoNode::FResult FConvertVideoNode::Validate()
{
	FString VideoPath = GetVideoDirectory();
	FConvertVideoNode::FResult Result = CheckImagesForVideo(VideoPath);

	if (Result.HasError())
	{
		return Result;
	}
	
	return MakeValue();
}

FString FConvertVideoNode::GetVideoDirectory() const
{
	return OutputDirectory / Video.Name;
}

FConvertVideoNode::FResult FConvertVideoNode::CheckImagesForVideo(const FString& InVideoPath)
{
	IFileManager& FileManager = IFileManager::Get();
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));

	if (!FPaths::DirectoryExists(InVideoPath))
	{
		FText Message = LOCTEXT("ConvertVideoNode_Validate_DirectoryMissing", "The output directory is missing");
		return MakeError(MoveTemp(Message));
	}

	bool bDirectoryIsEmpty = true;
	bool bFilesAreValid = FileManager.IterateDirectory(*InVideoPath, [&ImageWrapperModule, &bDirectoryIsEmpty](const TCHAR* InFileName, bool bIsDirectory)
	{
		bDirectoryIsEmpty = false;

		if (bIsDirectory)
		{
			return false;
		}

		EImageFormat Format = ImageWrapperModule.GetImageFormatFromExtension(*FPaths::GetExtension(InFileName));
		return Format != EImageFormat::Invalid;
	});

	if (bDirectoryIsEmpty)
	{
		FText Message = FText::Format(LOCTEXT("ConvertVideoNode_Validate_EmptyDirectory", "Folder is empty: {0}"), FText::FromString(InVideoPath));
		return MakeError(MoveTemp(Message));
	}

	if (!bFilesAreValid)
	{
		FText Message = LOCTEXT("ConvertVideoNode_Validate_InvalidFormat", "The images are in an unsupported format (supported format is JPEG)");
		return MakeError(MoveTemp(Message));
	}

	return MakeValue();
}

#undef LOCTEXT_NAMESPACE