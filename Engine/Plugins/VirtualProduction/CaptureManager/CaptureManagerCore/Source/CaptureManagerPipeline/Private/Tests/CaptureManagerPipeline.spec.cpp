// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#include "CaptureManagerPipeline.h"
#include "CaptureManagerPipelineTestUtils.h"

#include "Async/Monitor.h"

#if WITH_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FTestCaptureManagerPipeline, TEXT("TestCaptureManagerPipeline"), EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)


END_DEFINE_SPEC(FTestCaptureManagerPipeline)

void FTestCaptureManagerPipeline::Define()
{
	Describe(TEXT("Workflow_SingleNode"), [this]
	{
		It(TEXT("Success"), [this]
		{
			TSharedPtr<FCaptureManagerPipeline> CaptureManagerPipeline
				= MakeShared<FCaptureManagerPipeline>(EPipelineExecutionPolicy::Asynchronous);

			CaptureManagerPipeline->AddGenericNode(MakeShared<FNodeTestSuccess>());

			FCaptureManagerPipeline::FResult Results = CaptureManagerPipeline->Run();

			TestFalse(TEXT("Results should NOT be empty"), Results.IsEmpty());

			FNodeTestSuccess::FResult Result = Results.CreateConstIterator()->Value;
			TestFalse(TEXT("Result should NOT have an error"), Result.HasError());
		});

		It(TEXT("PrepareFail"), [this]
		{
			TSharedPtr<FCaptureManagerPipeline> CaptureManagerPipeline 
				= MakeShared<FCaptureManagerPipeline>(EPipelineExecutionPolicy::Asynchronous);

			CaptureManagerPipeline->AddGenericNode(MakeShared<FNodeTestPrepareFailed>());

			FCaptureManagerPipeline::FResult Results = CaptureManagerPipeline->Run();

			TestFalse(TEXT("Results should NOT be empty"), Results.IsEmpty());

			FNodeTestPrepareFailed::FResult Result = Results.CreateConstIterator()->Value;
			TestTrue(TEXT("Result should have an error"), Result.HasError());
			TestEqual(TEXT("Error code should match"), Result.GetError().GetCode(), FNodeTestPrepareFailed::PrepareFailCode);
		});

		It(TEXT("RunFail"), [this]
		{
			TSharedPtr<FCaptureManagerPipeline> CaptureManagerPipeline
				= MakeShared<FCaptureManagerPipeline>(EPipelineExecutionPolicy::Asynchronous);

			CaptureManagerPipeline->AddGenericNode(MakeShared<FNodeTestRunFailed>());

			FCaptureManagerPipeline::FResult Results = CaptureManagerPipeline->Run();

			TestFalse(TEXT("Results should NOT be empty"), Results.IsEmpty());

			FNodeTestRunFailed::FResult Result = Results.CreateConstIterator()->Value;
			TestTrue(TEXT("Result should have an error"), Result.HasError());
			TestEqual(TEXT("Error code should match"), Result.GetError().GetCode(), FNodeTestRunFailed::RunFailCode);
		});

		It(TEXT("ValidateFail"), [this]
		{
			TSharedPtr<FCaptureManagerPipeline> CaptureManagerPipeline
				= MakeShared<FCaptureManagerPipeline>(EPipelineExecutionPolicy::Asynchronous);

			CaptureManagerPipeline->AddGenericNode(MakeShared<FNodeTestValidateFailed>());

			FCaptureManagerPipeline::FResult Results = CaptureManagerPipeline->Run();

			TestFalse(TEXT("Results should NOT be empty"), Results.IsEmpty());

			FNodeTestValidateFailed::FResult Result = Results.CreateConstIterator()->Value;
			TestTrue(TEXT("Result should have an error"), Result.HasError());
			TestEqual(TEXT("Error code should match"), Result.GetError().GetCode(), FNodeTestValidateFailed::ValidateFailCode);
		});
	});

	Describe(TEXT("Workflow_MultipleNodes"), [this]
	{
		It(TEXT("Success"), [this]
		{
			static constexpr int32 NumNodes = 50;

			TSharedPtr<FCaptureManagerPipeline> CaptureManagerPipeline
				= MakeShared<FCaptureManagerPipeline>(EPipelineExecutionPolicy::Asynchronous);

			for (int32 Index = 0; Index < NumNodes; ++Index)
			{
				CaptureManagerPipeline->AddGenericNode(MakeShared<FNodeTestSuccess>());
			}

			FCaptureManagerPipeline::FResult Results = CaptureManagerPipeline->Run();

			TestFalse(TEXT("Results should NOT be empty"), Results.IsEmpty());
			TestEqual(TEXT("Number of results should match number of nodes"), NumNodes, Results.Num());

			for (const TPair<FGuid, FNodeTestPrepareFailed::FResult>& ResultPair : Results)
			{
				FNodeTestSuccess::FResult Result = ResultPair.Value;
				TestFalse(TEXT("Result should NOT have an error"), Result.HasError());
			}
		});

		It(TEXT("Failures"), [this]
		{
			TSharedPtr<FCaptureManagerPipeline> CaptureManagerPipeline
				= MakeShared<FCaptureManagerPipeline>(EPipelineExecutionPolicy::Asynchronous);

			TSharedPtr<FNodeTestSuccess> SuccessNode = MakeShared<FNodeTestSuccess>();
			TSharedPtr<FNodeTestPrepareFailed> PrepareFailNode = MakeShared<FNodeTestPrepareFailed>();
			TSharedPtr<FNodeTestRunFailed> RunFailNode = MakeShared<FNodeTestRunFailed>();
			TSharedPtr<FNodeTestValidateFailed> ValidateFailNode = MakeShared<FNodeTestValidateFailed>();

			FGuid SuccessNodeId = CaptureManagerPipeline->AddGenericNode(MoveTemp(SuccessNode));
			FGuid PrepareNodeId = CaptureManagerPipeline->AddGenericNode(MoveTemp(PrepareFailNode));
			FGuid RunNodeId = CaptureManagerPipeline->AddGenericNode(MoveTemp(RunFailNode));
			FGuid ValidateNodeId = CaptureManagerPipeline->AddGenericNode(MoveTemp(ValidateFailNode));

			FCaptureManagerPipeline::FResult Results = CaptureManagerPipeline->Run();

			TestFalse(TEXT("Results should NOT be empty"), Results.IsEmpty());
			
			FNodeTestSuccess::FResult SuccessResult = Results[SuccessNodeId];
			TestFalse(TEXT("Result should NOT have an error"), SuccessResult.HasError());
			
			FNodeTestPrepareFailed::FResult PrepareResult = Results[PrepareNodeId];
			TestTrue(TEXT("Result should have an error"), PrepareResult.HasError());
			TestEqual(TEXT("Error code should match"), PrepareResult.GetError().GetCode(), FNodeTestPrepareFailed::PrepareFailCode);

			FNodeTestRunFailed::FResult RunResult = Results[RunNodeId];
			TestTrue(TEXT("Result should have an error"), RunResult.HasError());
			TestEqual(TEXT("Error code should match"), RunResult.GetError().GetCode(), FNodeTestRunFailed::RunFailCode);

			FNodeTestValidateFailed::FResult ValidateResult = Results[ValidateNodeId];
			TestTrue(TEXT("Result should have an error"), ValidateResult.HasError());
			TestEqual(TEXT("Error code should match"), ValidateResult.GetError().GetCode(), FNodeTestValidateFailed::ValidateFailCode);
		});
	});
}

#endif // WITH_AUTOMATION_TESTS

