// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#include "CaptureManagerPipeline.h"
#include "CaptureManagerPipelineNodeTestUtils.h"

#include "Interfaces/IPluginManager.h"

#if WITH_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FTestCaptureManagerPipelineNodes, TEXT("TestCaptureManagerPipelineNodes"), EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

FTakeMetadataParser TakeMetadataParser; 

TValueOrError<TArray<TSharedPtr<FTestVideoNode>>, FText> CreateTestVideoNodeSuccess();
TValueOrError<TArray<TSharedPtr<FTestVideoNode>>, FText> CreateTestVideoNodeFailure();
TValueOrError<TArray<TSharedPtr<FTestAudioNode>>, FText> CreateTestAudioNodeSuccess();
TValueOrError<TArray<TSharedPtr<FTestAudioNode>>, FText> CreateTestAudioNodeFailure();
TValueOrError<TArray<TSharedPtr<FTestDepthNode>>, FText> CreateTestDepthNodeSuccess();
TValueOrError<TArray<TSharedPtr<FTestDepthNode>>, FText> CreateTestDepthNodeFailure();


TValueOrError<TArray<TSharedPtr<FTestVideoNode>>, FText> CreateVideoTestNode(FString InTakeName);
TValueOrError<TArray<TSharedPtr<FTestAudioNode>>, FText> CreateAudioTestNode(FString InTakeName);
TValueOrError<TArray<TSharedPtr<FTestDepthNode>>, FText> CreateDepthTestNode(FString InTakeName);


END_DEFINE_SPEC(FTestCaptureManagerPipelineNodes)

TValueOrError<TArray<TSharedPtr<FTestVideoNode>>, FText> FTestCaptureManagerPipelineNodes::CreateTestVideoNodeSuccess()
{
	FString TakeName = TEXT("Take_1");
	return CreateVideoTestNode(MoveTemp(TakeName));
}

TValueOrError<TArray<TSharedPtr<FTestVideoNode>>, FText> FTestCaptureManagerPipelineNodes::CreateTestVideoNodeFailure()
{
	FString TakeName = TEXT("Take_2");
	return CreateVideoTestNode(MoveTemp(TakeName));
}

TValueOrError<TArray<TSharedPtr<FTestVideoNode>>, FText> FTestCaptureManagerPipelineNodes::CreateVideoTestNode(FString InTakeName)
{
	const FString ContentDir = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetContentDir();

	const FString BaseDir = ContentDir / TEXT("CaptureManagerPipeline");
	const FString TakeDir = BaseDir / InTakeName;
	const FString TakeJsonFile = TakeDir / TEXT("take.json");

	TValueOrError<FTakeMetadata, FTakeMetadataParserError> Result = TakeMetadataParser.Parse(TakeJsonFile);

	if (Result.HasError())
	{
		return MakeError(Result.GetError().Message);
	}

	FTakeMetadata TakeMetadata = Result.StealValue();

	TArray<TSharedPtr<FTestVideoNode>> TestVideoNodeArray;

	for (const FTakeMetadata::FVideo& Video : TakeMetadata.Video)
	{
		TestVideoNodeArray.Add(MakeShared<FTestVideoNode>(InTakeName, Video, TakeDir));
	}

	return MakeValue(MoveTemp(TestVideoNodeArray));
}

TValueOrError<TArray<TSharedPtr<FTestAudioNode>>, FText> FTestCaptureManagerPipelineNodes::CreateTestAudioNodeSuccess()
{
	FString TakeName = TEXT("Take_3");
	return CreateAudioTestNode(MoveTemp(TakeName));
}

TValueOrError<TArray<TSharedPtr<FTestAudioNode>>, FText> FTestCaptureManagerPipelineNodes::CreateTestAudioNodeFailure()
{
	FString TakeName = TEXT("Take_4");
	return CreateAudioTestNode(MoveTemp(TakeName));
}

TValueOrError<TArray<TSharedPtr<FTestAudioNode>>, FText> FTestCaptureManagerPipelineNodes::CreateAudioTestNode(FString InTakeName)
{
	const FString ContentDir = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetContentDir();

	const FString BaseDir = ContentDir / TEXT("CaptureManagerPipeline");
	const FString TakeDir = BaseDir / InTakeName;
	const FString TakeJsonFile = TakeDir / TEXT("take.json");

	TValueOrError<FTakeMetadata, FTakeMetadataParserError> Result = TakeMetadataParser.Parse(TakeJsonFile);

	if (Result.HasError())
	{
		return MakeError(Result.GetError().Message);
	}

	FTakeMetadata TakeMetadata = Result.StealValue();

	TArray<TSharedPtr<FTestAudioNode>> TestAudioNodeArray;
	for (const FTakeMetadata::FAudio& Audio : TakeMetadata.Audio)
	{
		TestAudioNodeArray.Add(MakeShared<FTestAudioNode>(InTakeName, Audio, TakeDir));
	}

	return MakeValue(MoveTemp(TestAudioNodeArray));
}

TValueOrError<TArray<TSharedPtr<FTestDepthNode>>, FText> FTestCaptureManagerPipelineNodes::CreateTestDepthNodeSuccess()
{
	FString TakeName = TEXT("Take_5");
	return CreateDepthTestNode(MoveTemp(TakeName));
}

TValueOrError<TArray<TSharedPtr<FTestDepthNode>>, FText> FTestCaptureManagerPipelineNodes::CreateTestDepthNodeFailure()
{
	FString TakeName = TEXT("Take_6");
	return CreateDepthTestNode(MoveTemp(TakeName));
}

TValueOrError<TArray<TSharedPtr<FTestDepthNode>>, FText> FTestCaptureManagerPipelineNodes::CreateDepthTestNode(FString InTakeName)
{
	const FString ContentDir = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetContentDir();

	const FString BaseDir = ContentDir / TEXT("CaptureManagerPipeline");
	const FString TakeDir = BaseDir / InTakeName;
	const FString TakeJsonFile = TakeDir / TEXT("take.json");

	TValueOrError<FTakeMetadata, FTakeMetadataParserError> Result = TakeMetadataParser.Parse(TakeJsonFile);

	if (Result.HasError())
	{
		return MakeError(Result.GetError().Message);
	}

	FTakeMetadata TakeMetadata = Result.StealValue();

	TArray<TSharedPtr<FTestDepthNode>> TestDepthNodeArray;
	for (const FTakeMetadata::FVideo& Depth : TakeMetadata.Depth)
	{
		TestDepthNodeArray.Add(MakeShared<FTestDepthNode>(InTakeName, Depth, TakeDir));
	}

	return MakeValue(MoveTemp(TestDepthNodeArray));
}

#define TEST_AND_RETURN(Function) if (!Function) { return; }

void FTestCaptureManagerPipelineNodes::Define()
{
	Describe(TEXT("Workflow_ConvertVideoNode"), [this]
	{
		It(TEXT("Success"), [this]
		{
			TValueOrError<TArray<TSharedPtr<FTestVideoNode>>, FText> NodesCreated = CreateTestVideoNodeSuccess();

			TEST_AND_RETURN(TestTrue(TEXT("Parsing of the take should succeed"), NodesCreated.IsValid()));

			TArray<TSharedPtr<FTestVideoNode>> Nodes = NodesCreated.StealValue();

			TSharedPtr<FCaptureManagerPipeline> CaptureManagerPipeline
				= MakeShared<FCaptureManagerPipeline>(EPipelineExecutionPolicy::Asynchronous);

			for (TSharedPtr<FTestVideoNode> Node : Nodes)
			{
				CaptureManagerPipeline->AddConvertVideoNode(MoveTemp(Node));
			}
			
			FCaptureManagerPipeline::FResult Results = CaptureManagerPipeline->Run();

			TestFalse(TEXT("Results should NOT be empty"), Results.IsEmpty());

			for (const TPair<FGuid, FTestVideoNode::FResult>& NodeResultPair : Results)
			{
				FTestVideoNode::FResult Result = NodeResultPair.Value;
				TestFalse(TEXT("Result should NOT have an error"), Result.HasError());
			}
		});

		It(TEXT("Validation_Failure"), [this]
		{
			TValueOrError<TArray<TSharedPtr<FTestVideoNode>>, FText> NodesCreated = CreateTestVideoNodeFailure();

			TEST_AND_RETURN(TestTrue(TEXT("Parsing of the take should succeed"), NodesCreated.IsValid()));

			TArray<TSharedPtr<FTestVideoNode>> Nodes = NodesCreated.StealValue();

			TSharedPtr<FCaptureManagerPipeline> CaptureManagerPipeline
				= MakeShared<FCaptureManagerPipeline>(EPipelineExecutionPolicy::Asynchronous);

			for (TSharedPtr<FTestVideoNode> Node : Nodes)
			{
				CaptureManagerPipeline->AddConvertVideoNode(MoveTemp(Node));
			}

			FCaptureManagerPipeline::FResult Results = CaptureManagerPipeline->Run();

			TestFalse(TEXT("Results should NOT be empty"), Results.IsEmpty());

			for (const TPair<FGuid, FTestVideoNode::FResult>& NodeResultPair : Results)
			{
				FTestVideoNode::FResult Result = NodeResultPair.Value;
				TestTrue(TEXT("Result should have an error"), Result.HasError());
			}
		});
	});

	Describe(TEXT("Workflow_ConvertAudioNode"), [this]
	{
		It(TEXT("Success"), [this]
		{
			TValueOrError<TArray<TSharedPtr<FTestAudioNode>>, FText> NodesCreated = CreateTestAudioNodeSuccess();

			TEST_AND_RETURN(TestTrue(TEXT("Parsing of the take should succeed"), NodesCreated.IsValid()));

			TArray<TSharedPtr<FTestAudioNode>> Nodes = NodesCreated.StealValue();

			TSharedPtr<FCaptureManagerPipeline> CaptureManagerPipeline
				= MakeShared<FCaptureManagerPipeline>(EPipelineExecutionPolicy::Asynchronous);

			for (TSharedPtr<FTestAudioNode> Node : Nodes)
			{
				CaptureManagerPipeline->AddConvertAudioNode(MoveTemp(Node));
			}

			FCaptureManagerPipeline::FResult Results = CaptureManagerPipeline->Run();
			TestFalse(TEXT("Results should NOT be empty"), Results.IsEmpty());

			for (const TPair<FGuid, FTestAudioNode::FResult>& NodeResultPair : Results)
			{
				FTestAudioNode::FResult Result = NodeResultPair.Value;
				TestFalse(TEXT("Result should NOT have an error"), Result.HasError());
			}
		});

		It(TEXT("Validation_Failure"), [this]
		{
			TValueOrError<TArray<TSharedPtr<FTestAudioNode>>, FText> NodesCreated = CreateTestAudioNodeFailure();

			TEST_AND_RETURN(TestTrue(TEXT("Parsing of the take should succeed"), NodesCreated.IsValid()));

			TArray<TSharedPtr<FTestAudioNode>> Nodes = NodesCreated.StealValue();

			TSharedPtr<FCaptureManagerPipeline> CaptureManagerPipeline
				= MakeShared<FCaptureManagerPipeline>(EPipelineExecutionPolicy::Asynchronous);

			for (TSharedPtr<FTestAudioNode> Node : Nodes)
			{
				CaptureManagerPipeline->AddConvertAudioNode(MoveTemp(Node));
			}

			FCaptureManagerPipeline::FResult Results = CaptureManagerPipeline->Run();
			TestFalse(TEXT("Results should NOT be empty"), Results.IsEmpty());

			for (const TPair<FGuid, FTestAudioNode::FResult>& NodeResultPair : Results)
			{
				FTestAudioNode::FResult Result = NodeResultPair.Value;
				TestTrue(TEXT("Result should have an error"), Result.HasError());
			}
		});
	});

	Describe(TEXT("Workflow_ConvertDepthNode"), [this]
	{
		It(TEXT("Success"), [this]
		{
			TValueOrError<TArray<TSharedPtr<FTestDepthNode>>, FText> NodesCreated = CreateTestDepthNodeSuccess();

			TEST_AND_RETURN(TestTrue(TEXT("Parsing of the take should succeed"), NodesCreated.IsValid()));

			TArray<TSharedPtr<FTestDepthNode>> Nodes = NodesCreated.StealValue();

			TSharedPtr<FCaptureManagerPipeline> CaptureManagerPipeline
				= MakeShared<FCaptureManagerPipeline>(EPipelineExecutionPolicy::Asynchronous);

			for (TSharedPtr<FTestDepthNode> Node : Nodes)
			{
				CaptureManagerPipeline->AddConvertDepthNode(MoveTemp(Node));
			}

			FCaptureManagerPipeline::FResult Results = CaptureManagerPipeline->Run();
			TestFalse(TEXT("Results should NOT be empty"), Results.IsEmpty());

			for (const TPair<FGuid, FTestDepthNode::FResult>& NodeResultPair : Results)
			{
				FTestDepthNode::FResult Result = NodeResultPair.Value;
				TestFalse(TEXT("Result should NOT have an error"), Result.HasError());
			}
		});

		It(TEXT("Validation_Failure"), [this]
		{
			TValueOrError<TArray<TSharedPtr<FTestDepthNode>>, FText> NodesCreated = CreateTestDepthNodeFailure();

			TEST_AND_RETURN(TestTrue(TEXT("Parsing of the take should succeed"), NodesCreated.IsValid()));

			TArray<TSharedPtr<FTestDepthNode>> Nodes = NodesCreated.StealValue();

			TSharedPtr<FCaptureManagerPipeline> CaptureManagerPipeline
				= MakeShared<FCaptureManagerPipeline>(EPipelineExecutionPolicy::Asynchronous);

			for (TSharedPtr<FTestDepthNode> Node : Nodes)
			{
				CaptureManagerPipeline->AddConvertDepthNode(MoveTemp(Node));
			}

			FCaptureManagerPipeline::FResult Results = CaptureManagerPipeline->Run();
			TestFalse(TEXT("Results should NOT be empty"), Results.IsEmpty());

			for (const TPair<FGuid, FTestDepthNode::FResult>& NodeResultPair : Results)
			{
				FTestDepthNode::FResult Result = NodeResultPair.Value;
				TestTrue(TEXT("Result should have an error"), Result.HasError());
			}
		});
	});
}

#endif // WITH_AUTOMATION_TESTS