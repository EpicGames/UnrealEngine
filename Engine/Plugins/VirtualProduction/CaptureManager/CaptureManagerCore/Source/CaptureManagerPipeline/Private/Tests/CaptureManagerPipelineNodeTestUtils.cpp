// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerPipelineNodeTestUtils.h"

FTestVideoNode::FTestVideoNode(const FString& InTakeName,
							   const FTakeMetadata::FVideo& InVideo,
							   const FString& InOutputDirectory) 
	: FConvertVideoNode(InVideo, InOutputDirectory)
{
}

FTestVideoNode::FResult FTestVideoNode::Run()
{
	return MakeValue();
}


FTestAudioNode::FTestAudioNode(const FString& InTakeName,
							   const FTakeMetadata::FAudio& InAudio,
							   const FString& InOutputDirectory)
	: FConvertAudioNode(InAudio, InOutputDirectory)
{
}

FTestVideoNode::FResult FTestAudioNode::Run()
{
	return MakeValue();
}

FTestDepthNode::FTestDepthNode(const FString& InTakeName,
							   const FTakeMetadata::FVideo& InDepth,
							   const FString& InOutputDirectory)
	: FConvertDepthNode(InDepth, InOutputDirectory)
{
}

FTestDepthNode::FResult FTestDepthNode::Run()
{
	return MakeValue();
}
