// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/ConvertVideoNode.h"
#include "Nodes/ConvertAudioNode.h"
#include "Nodes/ConvertDepthNode.h"

class FTestVideoNode final : public FConvertVideoNode
{
public:

	FTestVideoNode(const FString& InTakeName,
				   const FTakeMetadata::FVideo& InVideo,
				   const FString& InOutputDirectory);

	virtual FTestVideoNode::FResult Run() override;
};

class FTestAudioNode final : public FConvertAudioNode
{
public:

	FTestAudioNode(const FString& InTakeName,
				   const FTakeMetadata::FAudio& InAudio,
				   const FString& InOutputDirectory);

	virtual FTestAudioNode::FResult Run() override;
};

class FTestDepthNode final : public FConvertDepthNode
{
public:

	FTestDepthNode(const FString& InTakeName,
				   const FTakeMetadata::FVideo& InDepth,
				   const FString& InOutputDirectory);

	virtual FTestDepthNode::FResult Run() override;
};