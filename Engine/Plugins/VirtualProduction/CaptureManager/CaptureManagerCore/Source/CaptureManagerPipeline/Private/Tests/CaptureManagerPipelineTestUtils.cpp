// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerPipelineTestUtils.h"


FNodeTestBase::FNodeTestBase(const FString& InName)
	: FCaptureManagerPipelineNode(InName)
{
}

FNodeTestBase::FResult FNodeTestBase::Prepare()
{
	return MakeValue();
}

FNodeTestBase::FResult FNodeTestBase::Run()
{
	return MakeValue();
}

FNodeTestBase::FResult FNodeTestBase::Validate()
{
	return MakeValue();
}

FNodeTestSuccess::FNodeTestSuccess()
	: FNodeTestBase(TEXT("NodeTestSuccess"))
{
}

FNodeTestPrepareFailed::FNodeTestPrepareFailed()
	: FNodeTestBase(TEXT("NodeTestPrepareFailed"))
{
}

FNodeTestPrepareFailed::FResult FNodeTestPrepareFailed::Prepare()
{
	return MakeError(FText::FromString(TEXT("PrepareFailed")), PrepareFailCode);
}


FNodeTestRunFailed::FNodeTestRunFailed()
	: FNodeTestBase(TEXT("NodeTestRunFailed"))
{
}

FNodeTestRunFailed::FResult FNodeTestRunFailed::Run()
{
	return MakeError(FText::FromString(TEXT("RunFailed")), RunFailCode);
}


FNodeTestValidateFailed::FNodeTestValidateFailed()
	: FNodeTestBase(TEXT("NodeTestValidateFailed"))
{
}

FNodeTestValidateFailed::FResult FNodeTestValidateFailed::Validate()
{
	return MakeError(FText::FromString(TEXT("ValidateFailed")), ValidateFailCode);
}

