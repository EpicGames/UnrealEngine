// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerPipelineNode.h"

class FNodeTestBase : public FCaptureManagerPipelineNode
{
public:

	using FCaptureManagerPipelineNode::FCaptureManagerPipelineNode;

	FNodeTestBase(const FString& InName);

	virtual FNodeTestBase::FResult Prepare() override;
	virtual FNodeTestBase::FResult Run() override;
	virtual FNodeTestBase::FResult Validate() override;
};

class FNodeTestSuccess final : public FNodeTestBase
{
public:

	FNodeTestSuccess();
};

class FNodeTestPrepareFailed final : public FNodeTestBase
{
public:

	constexpr static int32 PrepareFailCode = -1;

	FNodeTestPrepareFailed();

	virtual FNodeTestPrepareFailed::FResult Prepare() override final;
};

class FNodeTestRunFailed final : public FNodeTestBase
{
public:

	constexpr static int32 RunFailCode = -1;

	FNodeTestRunFailed();

	virtual FNodeTestRunFailed::FResult Run() override final;
};

class FNodeTestValidateFailed final : public FNodeTestBase
{
public:

	constexpr static int32 ValidateFailCode = -1;

	FNodeTestValidateFailed();

	virtual FNodeTestValidateFailed::FResult Validate() override final;
};