// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#include "Nodes/ConvertAudioNode.h"
#include "Nodes/ConvertDepthNode.h"
#include "Nodes/ConvertVideoNode.h"
#include "Nodes/ConvertCalibrationNode.h"

#include "Templates/SharedPointer.h"
#include "Misc/Guid.h"

#include "Async/Monitor.h"

#define UE_API CAPTUREMANAGERPIPELINE_API

enum class EPipelineExecutionPolicy
{
	Asynchronous = 0,
	Synchronous
};

class FCaptureManagerPipeline : public TSharedFromThis<FCaptureManagerPipeline>
{
public:
	using FResult = TMap<FGuid, FCaptureManagerPipelineNode::FResult>;

	UE_API FCaptureManagerPipeline(EPipelineExecutionPolicy InExecutionPolicy);
	UE_API ~FCaptureManagerPipeline();

	UE_API FGuid AddGenericNode(TSharedPtr<FCaptureManagerPipelineNode> InNode);
	UE_API FGuid AddConvertVideoNode(TSharedPtr<FConvertVideoNode> InNode);
	UE_API FGuid AddConvertAudioNode(TSharedPtr<FConvertAudioNode> InNode);
	UE_API FGuid AddConvertDepthNode(TSharedPtr<FConvertDepthNode> InNode);
	UE_API FGuid AddConvertCalibrationNode(TSharedPtr<FConvertCalibrationNode> InNode);

	UE_API FGuid AddSyncedNode(TSharedPtr<FCaptureManagerPipelineNode> InNode);

	// Blocking function
	[[nodiscard]] UE_API FCaptureManagerPipeline::FResult Run();

	UE_API void Cancel();

private:

	FGuid AddParallelPipelineNode(TSharedPtr<FCaptureManagerPipelineNode> InNode);

	TPimplPtr<class FCaptureManagerPipelineImpl> Impl;

	
	using FNodeMap = TMap<FGuid, TSharedPtr<FCaptureManagerPipelineNode>>;

	UE::CaptureManager::TMonitor<FNodeMap> ParallelNodes;
	UE::CaptureManager::TMonitor<FNodeMap> SyncNodes;

	EPipelineExecutionPolicy ExecutionPolicy;
};

#undef UE_API