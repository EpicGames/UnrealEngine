// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"

#include "Templates/PimplPtr.h"

#define UE_API CAPTUREMANAGERPIPELINE_API

class FCaptureManagerPipelineError
{
public:
	UE_API FCaptureManagerPipelineError(FText InMessage, int32 InCode = 0);

	UE_API const FText& GetMessage() const;

	UE_API int32 GetCode() const;

private:

	FText Message;
	int32 Code;
};

class FCaptureManagerPipelineNode : public TSharedFromThis<FCaptureManagerPipelineNode>
{
public:

	using FResult = TValueOrError<void, FCaptureManagerPipelineError>;

	UE_API FCaptureManagerPipelineNode(const FString& InName);
	UE_API virtual ~FCaptureManagerPipelineNode();

	UE_API FString GetName() const;

	UE_API FResult Execute();
	UE_API void Cancel();

protected:

	virtual FResult Prepare() = 0;
	virtual FResult Run() = 0;
	virtual FResult Validate() = 0;

private:
	class FCaptureManagerPipelineNodeImpl;
	TPimplPtr<FCaptureManagerPipelineNodeImpl> Impl;
};

#undef UE_API