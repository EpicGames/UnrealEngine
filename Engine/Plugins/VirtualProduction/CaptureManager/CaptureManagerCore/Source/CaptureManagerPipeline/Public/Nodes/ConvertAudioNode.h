// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerPipelineNode.h"

#include "CaptureManagerTakeMetadata.h"

#define UE_API CAPTUREMANAGERPIPELINE_API

class FConvertAudioNode : public FCaptureManagerPipelineNode
{
public:

	UE_API FConvertAudioNode(const FTakeMetadata::FAudio& InAudio,
					  const FString& InOutputDirectory);

	UE_API virtual ~FConvertAudioNode() override;

protected:

	FTakeMetadata::FAudio Audio;
	FString OutputDirectory;

private:

	UE_API virtual FResult Prepare() override final;
	UE_API virtual FResult Validate() override final;

	FString GetAudioDirectory() const;

	static FResult CheckForAudioFile(const FString& InAudioPath);
};

#undef UE_API