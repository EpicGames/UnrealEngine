// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerPipelineNode.h"

#include "CaptureManagerTakeMetadata.h"

#define UE_API CAPTUREMANAGERPIPELINE_API

class FConvertCalibrationNode : public FCaptureManagerPipelineNode
{
public:

	UE_API FConvertCalibrationNode(const FTakeMetadata::FCalibration& InCalibration,
							const FString& InOutputDirectory);
protected:

	FTakeMetadata::FCalibration Calibration;
	FString OutputDirectory;

private:

	UE_API virtual FResult Prepare() override final;
	UE_API virtual FResult Validate() override final;

	FString GetCalibrationDirectory() const;
};

#undef UE_API