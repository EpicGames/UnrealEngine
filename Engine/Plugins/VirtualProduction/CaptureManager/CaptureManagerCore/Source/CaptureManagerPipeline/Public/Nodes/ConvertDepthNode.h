// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerPipelineNode.h"

#include "CaptureManagerTakeMetadata.h"

#define UE_API CAPTUREMANAGERPIPELINE_API

class FConvertDepthNode : public FCaptureManagerPipelineNode
{
public:

	UE_API FConvertDepthNode(const FTakeMetadata::FVideo& InDepth,
					  const FString& InOutputDirectory);

	UE_API virtual ~FConvertDepthNode() override;

protected:

	FTakeMetadata::FVideo Depth;
	FString OutputDirectory;

private:

	UE_API virtual FResult Prepare() override final;
	UE_API virtual FResult Validate() override final;

	FString GetDepthDirectory() const;

	static FResult CheckImagesForDepth(const FString& InDepthPath);
};

#undef UE_API