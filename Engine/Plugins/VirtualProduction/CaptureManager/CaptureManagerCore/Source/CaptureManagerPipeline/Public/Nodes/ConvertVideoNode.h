// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerPipelineNode.h"

#include "CaptureManagerTakeMetadata.h"

#define UE_API CAPTUREMANAGERPIPELINE_API

class FConvertVideoNode : public FCaptureManagerPipelineNode
{
public:

	UE_API FConvertVideoNode(const FTakeMetadata::FVideo& InVideo,
					  const FString& InOutputDirectory);

	UE_API virtual ~FConvertVideoNode() override;

protected:

	FTakeMetadata::FVideo Video;
	FString OutputDirectory;

private:

	UE_API virtual FResult Prepare() override final;
	UE_API virtual FResult Validate() override final;

	FString GetVideoDirectory() const;

	static FResult CheckImagesForVideo(const FString& InVideoPath);
};

#undef UE_API