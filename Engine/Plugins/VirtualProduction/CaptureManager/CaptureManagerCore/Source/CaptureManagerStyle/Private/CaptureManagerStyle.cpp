// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/SlateGameResources.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleMacros.h"

FName FCaptureManagerStyle::StyleName = TEXT("CaptureManagerStyle");

FCaptureManagerStyle::FCaptureManagerStyle()
	: FSlateStyleSet(FCaptureManagerStyle::StyleName)
{
	const FVector2D Icon16x16(16.0f, 16.0f);

	SetContentRoot(IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetContentDir());

	Set("CaptureManagerIcon", new IMAGE_BRUSH_SVG("Icons/CaptureManager_16", Icon16x16));
}

const FName& FCaptureManagerStyle::GetStyleSetName() const
{
	return StyleName;
}

const FCaptureManagerStyle& FCaptureManagerStyle::Get()
{
	static FCaptureManagerStyle StyleInstance;

	return StyleInstance;
}

void FCaptureManagerStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	}
}
