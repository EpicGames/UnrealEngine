// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerStyleModule.h"

#include "Styling/SlateStyleRegistry.h"
#include "CaptureManagerStyle.h"


IMPLEMENT_MODULE(FCaptureManagerStyleModule, CaptureManagerStyle)

void FCaptureManagerStyleModule::StartupModule()
{
	FSlateStyleRegistry::RegisterSlateStyle(FCaptureManagerStyle::Get());
	FCaptureManagerStyle::ReloadTextures();
}

void FCaptureManagerStyleModule::ShutdownModule()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(FCaptureManagerStyle::Get());
}
