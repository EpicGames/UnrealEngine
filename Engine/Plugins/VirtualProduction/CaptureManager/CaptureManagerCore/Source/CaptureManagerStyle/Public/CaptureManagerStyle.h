// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

#define UE_API CAPTUREMANAGERSTYLE_API

/** A style set for Capture Manager */
class FCaptureManagerStyle : public FSlateStyleSet
{
public:

	UE_API virtual const FName& GetStyleSetName() const override;
	static UE_API const FCaptureManagerStyle& Get();

	static UE_API void ReloadTextures();

private:

	UE_API FCaptureManagerStyle();

	static UE_API FName StyleName;
};

#undef UE_API
