// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CaptureManagerTakeMetadata : ModuleRules
{
	public CaptureManagerTakeMetadata(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[]
		{
			"ImageCore"
		});

		PrivateDependencyModuleNames.AddRange(new string[] {
			"Core",
			"Projects",
			"RapidJSON"
		});
	}
}
