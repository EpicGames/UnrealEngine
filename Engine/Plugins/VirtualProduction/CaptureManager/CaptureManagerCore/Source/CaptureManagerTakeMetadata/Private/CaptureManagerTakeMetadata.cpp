// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerTakeMetadata.h"

#include "Misc/FileHelper.h"

#include "rapidjson/schema.h"
#include "rapidjson/stringbuffer.h"
#include "rapidjson/prettywriter.h"

#include "Interfaces/IPluginManager.h"

#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "CaptureManagerTakeMetadata"

DEFINE_LOG_CATEGORY_STATIC(LogTakeMetadataPathUtils, Log, All);

FTakeThumbnailData::FTakeThumbnailData() = default;

FTakeThumbnailData::FTakeThumbnailData(FString InThumbnailPath)
	: Thumbnail(TInPlaceType<FString>(), MoveTemp(InThumbnailPath))
{
}

FTakeThumbnailData::FTakeThumbnailData(TArray<uint8> InThumbnailData)
	: Thumbnail(TInPlaceType<TArray<uint8>>(), MoveTemp(InThumbnailData))
{
}

FTakeThumbnailData::FTakeThumbnailData(TArray<FColor> InDecompressedImageData, uint32 InWidth, uint32 InHeight, ERawImageFormat::Type InFormat)
	: Thumbnail(TInPlaceType<FRawImage>(), MoveTemp(InDecompressedImageData), InWidth, InHeight, InFormat)
{
}

void FTakeThumbnailData::operator=(FString InImagePath)
{
	Thumbnail.Set<FString>(MoveTemp(InImagePath));
}

void FTakeThumbnailData::operator=(TArray<uint8> InCompressedImageData)
{
	Thumbnail.Set<TArray<uint8>>(MoveTemp(InCompressedImageData));
}

void FTakeThumbnailData::operator=(FRawImage InRawImage)
{
	Thumbnail.Set<FRawImage>(MoveTemp(InRawImage));
}

TOptional<TArray<uint8>> FTakeThumbnailData::GetThumbnailData() const
{
	if (Thumbnail.IsType<FString>())
	{
		FString ThumbnailPath = Thumbnail.Get<FString>();

		TArray<uint8> Data;
		if (FFileHelper::LoadFileToArray(Data, *ThumbnailPath))
		{
			return Data;
		}
	}
	else if (Thumbnail.IsType<TArray<uint8>>())
	{
		return Thumbnail.Get<TArray<uint8>>();
	}

	return {};
}

TOptional<FString> FTakeThumbnailData::GetThumbnailPath() const
{
	if (Thumbnail.IsType<FString>())
	{
		return Thumbnail.Get<FString>();
	}

	return {};
}

TOptional<FTakeThumbnailData::FRawImage> FTakeThumbnailData::GetRawImage() const
{
	if (Thumbnail.IsType<FRawImage>())
	{
		return Thumbnail.Get<FRawImage>();
	}

	return {};
}

const FString FTakeMetadata::FileExtension = TEXT("cptake");

static FTakeMetadata::FSchemaVersion ParseSchemaVersion(const rapidjson::Value& InVersionObject)
{
	uint32 Major = InVersionObject["Major"].GetUint();
	uint32 Minor = InVersionObject["Minor"].GetUint();
	return FTakeMetadata::FSchemaVersion{ Major, Minor };
}

static FString ParseVersion(const rapidjson::Value& InVersionObject)
{
	uint32 Major = InVersionObject["Major"].GetUint();
	uint32 Minor = InVersionObject["Minor"].GetUint();
	uint32 Patch = InVersionObject["Patch"].GetUint();
	return FString::Format(TEXT("{0}.{1}.{2}"), { Major, Minor, Patch });
}

static FTakeMetadata::FDevice ParseDevice(const rapidjson::Value& InDeviceObject, const FTakeMetadata::FSchemaVersion& InVersion)
{
	FTakeMetadata::FDevice TakeDevice;
	TakeDevice.Type = InDeviceObject["Type"].GetString();
	TakeDevice.Model = InDeviceObject["Model"].GetString();
	if (InVersion < FTakeMetadata::FSchemaVersion{ 4, 2 })
	{
		TakeDevice.Name = InDeviceObject["UserId"].GetString();
	}
	else
	{
		TakeDevice.Name = InDeviceObject["Name"].GetString();
	}

	if (InDeviceObject.HasMember("Platform"))
	{
		FTakeMetadata::FDevice::FPlatform Platform;
		const rapidjson::Value& PlatformObject = InDeviceObject["Platform"];
		Platform.Name = PlatformObject["Name"].GetString();
		if (PlatformObject.HasMember("Version"))
		{
			if (InVersion.Major >= 4)
			{
				Platform.Version = PlatformObject["Version"].GetString();
			}
			else
			{
				Platform.Version = ParseVersion(PlatformObject["Version"]);
			}
		}
		TakeDevice.Platform = Platform;
	}

	for (const rapidjson::Value& SwObject : InDeviceObject["Software"].GetArray())
	{
		FString Name = SwObject["Name"].GetString();
		TOptional<FString> SoftwareVersion;
		if (SwObject.HasMember("Version"))
		{
			if (InVersion.Major >= 4)
			{
				SoftwareVersion = SwObject["Version"].GetString();
			}
			else
			{
				SoftwareVersion = ParseVersion(SwObject["Version"]);
			}
		}
		FTakeMetadata::FDevice::FSoftware DeviceSoftware{ Name, SoftwareVersion };
		TakeDevice.Software.Push(DeviceSoftware);
	}

	return TakeDevice;
}

static FTakeMetadata::FVideo ParseVideo(const rapidjson::Value& InVideoObject, const FTakeMetadata::FSchemaVersion& InVersion)
{
	FTakeMetadata::FVideo Video;

	if (InVersion < FTakeMetadata::FSchemaVersion{ 4, 2 })
	{
		Video.Name = InVideoObject["UserId"].GetString();
	}
	else
	{
		Video.Name = InVideoObject["Name"].GetString();
	}
	Video.Path = InVideoObject["Path"].GetString();

	if (InVideoObject.HasMember("PathType"))
	{
		FString PathTypeString = InVideoObject["PathType"].GetString();
		if (PathTypeString == TEXT("Folder"))
		{
			Video.PathType = FTakeMetadata::FVideo::EPathType::Folder;
		}
		else if (PathTypeString == TEXT("File"))
		{
			Video.PathType = FTakeMetadata::FVideo::EPathType::File;
		}
	}

	if (InVideoObject.HasMember("Format"))
	{
		Video.Format = InVideoObject["Format"].GetString();
	}

	if (InVideoObject.HasMember("Orientation"))
	{
		FString OrientationString = InVideoObject["Orientation"].GetString();
		if (OrientationString == TEXT("CW90"))
		{
			Video.Orientation = FTakeMetadata::FVideo::EOrientation::CW90;
		}
		else if (OrientationString == TEXT("CW180"))
		{
			Video.Orientation = FTakeMetadata::FVideo::EOrientation::CW180;
		}
		else if (OrientationString == TEXT("CW270"))
		{
			Video.Orientation = FTakeMetadata::FVideo::EOrientation::CW270;
		}
		else if (OrientationString == TEXT("Original"))
		{
			Video.Orientation = FTakeMetadata::FVideo::EOrientation::Original;
		}
	}

	if (InVideoObject.HasMember("FramesCount"))
	{
		Video.FramesCount = InVideoObject["FramesCount"].GetUint();
	}

	if (InVideoObject.HasMember("DroppedFrames"))
	{
		TArray<uint32> DroppedFrames;
		for (const rapidjson::Value& DroppedFrame : InVideoObject["DroppedFrames"].GetArray())
		{
			DroppedFrames.Add(DroppedFrame.GetUint());
		}
		Video.DroppedFrames = MoveTemp(DroppedFrames);
	}

	if (InVideoObject.HasMember("FrameWidth") && InVideoObject.HasMember("FrameHeight"))
	{
		Video.FrameWidth = InVideoObject["FrameWidth"].GetUint();
		Video.FrameHeight = InVideoObject["FrameHeight"].GetUint();
	}

	Video.FrameRate = InVideoObject["FrameRate"].GetFloat();

	if (InVideoObject.HasMember("TimecodeStart"))
	{
		Video.TimecodeStart = InVideoObject["TimecodeStart"].GetString();
	}

	return Video;
}

static TArray<FTakeMetadata::FCalibration> ParseCalibration(const rapidjson::Value& InCalibrationArray, const FTakeMetadata::FSchemaVersion& InVersion)
{
	TArray<FTakeMetadata::FCalibration> Calibrations;

	if (InVersion == FTakeMetadata::FSchemaVersion{ 1, 0 })
	{
		FTakeMetadata::FCalibration Calibration;
		Calibration.Name = "undefined";
		Calibration.Path = InCalibrationArray["Path"].GetString();
		Calibrations.Push(Calibration);
	}
	else if (InVersion == FTakeMetadata::FSchemaVersion{ 2, 0 })
	{
		for (const rapidjson::Value& CalibrationObject : InCalibrationArray.GetArray())
		{
			FTakeMetadata::FCalibration Calibration;
			Calibration.Name = CalibrationObject["UserId"].GetString();
			Calibration.Path = CalibrationObject["Path"].GetString();
			Calibrations.Push(Calibration);
		}
	}
	else
	{
		for (const rapidjson::Value& CalibrationObject : InCalibrationArray.GetArray())
		{
			FTakeMetadata::FCalibration Calibration;
			if (InVersion < FTakeMetadata::FSchemaVersion{ 4, 2 })
			{
				Calibration.Name = CalibrationObject["UserId"].GetString();
			}
			else
			{
				Calibration.Name = CalibrationObject["Name"].GetString();
			}
			Calibration.Format = CalibrationObject["Format"].GetString();
			Calibration.Path = CalibrationObject["Path"].GetString();
			Calibrations.Push(Calibration);
		}
	}

	return Calibrations;
}

static TArray<FTakeMetadata::FAudio> ParseAudio(const rapidjson::Value& InAudioArray, const FTakeMetadata::FSchemaVersion& InVersion)
{
	TArray<FTakeMetadata::FAudio> Audios;
	for (const rapidjson::Value& AudioObject : InAudioArray.GetArray())
	{
		FTakeMetadata::FAudio Audio;
		if (InVersion < FTakeMetadata::FSchemaVersion{ 4, 2 })
		{
			Audio.Name = AudioObject["UserId"].GetString();
		}
		else
		{
			Audio.Name = AudioObject["Name"].GetString();
		}

		Audio.Path = AudioObject["Path"].GetString();
		if (AudioObject.HasMember("Duration"))
		{
			Audio.Duration = AudioObject["Duration"].GetFloat();
		}

		if (AudioObject.HasMember("TimecodeRate"))
		{
			Audio.TimecodeRate = AudioObject["TimecodeRate"].GetFloat();
		}

		if (AudioObject.HasMember("TimecodeStart"))
		{
			Audio.TimecodeStart = AudioObject["TimecodeStart"].GetString();
		}

		Audios.Push(Audio);
	}
	return Audios;
}

static FTakeMetadata ParseTakeMetadata(const rapidjson::Document& InDocument, const FTakeMetadata::FSchemaVersion& Version)
{
	FTakeMetadata TakeMetadata;
	TakeMetadata.Version = Version;

	if (InDocument.HasMember("DateTime"))
	{
		FDateTime DateTime;
		FString DateTimeString = InDocument["DateTime"].GetString();
		FDateTime::ParseIso8601(*DateTimeString, DateTime);
		TakeMetadata.DateTime = MoveTemp(DateTime);
	}

	if (InDocument.HasMember("Thumbnail"))
	{
		TakeMetadata.Thumbnail = FTakeThumbnailData(InDocument["Thumbnail"].GetString());
	}

	TakeMetadata.UniqueId = InDocument["UniqueId"].GetString();
	TakeMetadata.TakeNumber = InDocument["TakeNumber"].GetUint();
	TakeMetadata.Slate = InDocument["Slate"].GetString();

	TakeMetadata.Device = ParseDevice(InDocument["Device"], Version);

	for (const rapidjson::Value& VideoObject : InDocument["Video"].GetArray())
	{
		TakeMetadata.Video.Push(ParseVideo(VideoObject, Version));
	}

	for (const rapidjson::Value& DepthObject : InDocument["Depth"].GetArray())
	{
		TakeMetadata.Depth.Push(ParseVideo(DepthObject, Version));
	}

	if (InDocument.HasMember("Calibration"))
	{
		TakeMetadata.Calibration = ParseCalibration(InDocument["Calibration"], Version);
	}

	if (InDocument.HasMember("Audio"))
	{
		TakeMetadata.Audio = ParseAudio(InDocument["Audio"], Version);
	}

	return TakeMetadata;
}

rapidjson::Value SerializeString(const FString& InStr, rapidjson::Document::AllocatorType& InAllocator)
{
	rapidjson::Value StringValue(rapidjson::kStringType);
	auto Conversion = StringCast<ANSICHAR>(*InStr, InStr.Len());

	StringValue.SetString(Conversion.Get(), Conversion.Length(), InAllocator);

	return StringValue;
}

rapidjson::Value SerializeSchemaVersion(const FTakeMetadata::FSchemaVersion& InVersion, rapidjson::Document::AllocatorType& InAllocator)
{
	rapidjson::Value VersionObject(rapidjson::kObjectType);

	VersionObject.AddMember("Major", InVersion.Major, InAllocator);
	VersionObject.AddMember("Minor", InVersion.Minor, InAllocator);

	return VersionObject;
}

rapidjson::Value SerializeDevice(const FTakeMetadata::FDevice& InDevice, rapidjson::Document::AllocatorType& InAllocator)
{
	rapidjson::Value DeviceObject(rapidjson::kObjectType);

	DeviceObject.AddMember("Type", SerializeString(InDevice.Type, InAllocator), InAllocator);
	DeviceObject.AddMember("Model", SerializeString(InDevice.Model, InAllocator), InAllocator);
	DeviceObject.AddMember("Name", SerializeString(InDevice.Name, InAllocator), InAllocator);

	if (InDevice.Platform.IsSet())
	{
		rapidjson::Value PlatformObject(rapidjson::kObjectType);

		PlatformObject.AddMember("Name", SerializeString(InDevice.Platform.GetValue().Name, InAllocator), InAllocator);
		if (InDevice.Platform.GetValue().Version.IsSet())
		{
			PlatformObject.AddMember("Version", SerializeString(InDevice.Platform.GetValue().Version.GetValue(), InAllocator), InAllocator);
		}

		DeviceObject.AddMember("Platform", PlatformObject, InAllocator);
	}

	if (!InDevice.Software.IsEmpty())
	{
		rapidjson::Value SoftwareArray(rapidjson::kArrayType);

		for (const FTakeMetadata::FDevice::FSoftware& Software : InDevice.Software)
		{
			rapidjson::Value SoftwareObject(rapidjson::kObjectType);
			SoftwareObject.AddMember("Name", SerializeString(Software.Name, InAllocator), InAllocator);
			if (Software.Version.IsSet())
			{
				SoftwareObject.AddMember("Version", SerializeString(Software.Version.GetValue(), InAllocator), InAllocator);
			}

			SoftwareArray.PushBack(MoveTemp(SoftwareObject), InAllocator);
		}

		DeviceObject.AddMember("Software", SoftwareArray, InAllocator);
	}

	return DeviceObject;
}

FString GetPathTypeString(FTakeMetadata::FVideo::EPathType InPathType)
{
	switch (InPathType)
	{
	case FTakeMetadata::FVideo::EPathType::Folder:
		return TEXT("Folder");
	case FTakeMetadata::FVideo::EPathType::File:
	default:
		return TEXT("File");
	}
}

FString GetOrientationString(FTakeMetadata::FVideo::EOrientation InOrientation)
{
	switch (InOrientation)
	{
	case FTakeMetadata::FVideo::EOrientation::CW90:
		return TEXT("CW90");
	case FTakeMetadata::FVideo::EOrientation::CW180:
		return TEXT("CW180");
	case FTakeMetadata::FVideo::EOrientation::CW270:
		return TEXT("CW270");
	case FTakeMetadata::FVideo::EOrientation::Original:
	default:
		return TEXT("Original");
	}
}

rapidjson::Value SerializeVideo(const TArray<FTakeMetadata::FVideo>& InVideos, rapidjson::Document::AllocatorType& InAllocator)
{
	rapidjson::Value VideoArray(rapidjson::kArrayType);

	for (const FTakeMetadata::FVideo& Video : InVideos)
	{
		rapidjson::Value VideoObject(rapidjson::kObjectType);
		VideoObject.AddMember("Name", SerializeString(Video.Name, InAllocator), InAllocator);
		VideoObject.AddMember("Path", SerializeString(Video.Path, InAllocator), InAllocator);

		if (Video.PathType.IsSet())
		{
			VideoObject.AddMember("PathType", SerializeString(GetPathTypeString(Video.PathType.GetValue()), InAllocator), InAllocator);
		}

		if (!Video.Format.IsEmpty())
		{
			VideoObject.AddMember("Format", SerializeString(Video.Format, InAllocator), InAllocator);
		}

		if (Video.Orientation.IsSet())
		{
			VideoObject.AddMember("Orientation", SerializeString(GetOrientationString(Video.Orientation.GetValue()), InAllocator), InAllocator);
		}

		if (Video.FramesCount.IsSet())
		{
			VideoObject.AddMember("FramesCount", Video.FramesCount.GetValue(), InAllocator);
		}

		if (Video.DroppedFrames.IsSet())
		{
			rapidjson::Value DroppedFramesArray(rapidjson::kArrayType);
			for (uint32 DroppedFrame : Video.DroppedFrames.GetValue())
			{
				DroppedFramesArray.PushBack(DroppedFrame, InAllocator);
			}
			VideoObject.AddMember("DroppedFrames", DroppedFramesArray, InAllocator);
		}

		if (Video.FrameWidth.IsSet() && Video.FrameHeight.IsSet())
		{
			VideoObject.AddMember("FrameWidth", Video.FrameWidth.GetValue(), InAllocator);
			VideoObject.AddMember("FrameHeight", Video.FrameHeight.GetValue(), InAllocator);
		}

		VideoObject.AddMember("FrameRate", Video.FrameRate, InAllocator);

		if (Video.TimecodeStart.IsSet())
		{
			VideoObject.AddMember("TimecodeStart", SerializeString(Video.TimecodeStart.GetValue(), InAllocator), InAllocator);
		}

		VideoArray.PushBack(MoveTemp(VideoObject), InAllocator);
	}

	return VideoArray;
}

rapidjson::Value SerializeCalibration(const TArray <FTakeMetadata::FCalibration>& InCalibration, rapidjson::Document::AllocatorType& InAllocator)
{
	rapidjson::Value CalibrationArray(rapidjson::kArrayType);

	for (const FTakeMetadata::FCalibration& Calibration : InCalibration)
	{
		rapidjson::Value CalibrationObject(rapidjson::kObjectType);
		CalibrationObject.AddMember("Name", SerializeString(Calibration.Name, InAllocator), InAllocator);
		CalibrationObject.AddMember("Format", SerializeString(Calibration.Format, InAllocator), InAllocator);
		CalibrationObject.AddMember("Path", SerializeString(Calibration.Path, InAllocator), InAllocator);

		CalibrationArray.PushBack(MoveTemp(CalibrationObject), InAllocator);
	}

	return CalibrationArray;
}

rapidjson::Value SerializeAudio(const TArray<FTakeMetadata::FAudio>& InAudio, rapidjson::Document::AllocatorType& InAllocator)
{
	rapidjson::Value AudioArray(rapidjson::kArrayType);

	for (const FTakeMetadata::FAudio& Audio : InAudio)
	{
		rapidjson::Value AudioObject(rapidjson::kObjectType);
		AudioObject.AddMember("Name", SerializeString(Audio.Name, InAllocator), InAllocator);
		AudioObject.AddMember("Path", SerializeString(Audio.Path, InAllocator), InAllocator);
		
		if (Audio.Duration.IsSet())
		{
			AudioObject.AddMember("Duration", Audio.Duration.GetValue(), InAllocator);
		}

		if (Audio.TimecodeRate.IsSet())
		{
			AudioObject.AddMember("TimecodeRate", Audio.TimecodeRate.GetValue(), InAllocator);
		}

		if (Audio.TimecodeStart.IsSet())
		{
			AudioObject.AddMember("TimecodeStart", SerializeString(Audio.TimecodeStart.GetValue(), InAllocator), InAllocator);
		}

		AudioArray.PushBack(MoveTemp(AudioObject), InAllocator);
	}

	return AudioArray;
}

rapidjson::Document Serialize(const FTakeMetadata& InMetadata)
{
	rapidjson::Document Document(rapidjson::kObjectType);
	rapidjson::Document::AllocatorType& Allocator = Document.GetAllocator();

	Document.AddMember("Version", SerializeSchemaVersion(InMetadata.Version, Allocator), Allocator);

	if (InMetadata.DateTime.IsSet())
	{
		Document.AddMember("DateTime", SerializeString(InMetadata.DateTime.GetValue().ToIso8601(), Allocator), Allocator);
	}

	TOptional<FString> ThumbnailPathOpt = InMetadata.Thumbnail.GetThumbnailPath();
	if (ThumbnailPathOpt.IsSet())
	{
		Document.AddMember("Thumbnail", SerializeString(ThumbnailPathOpt.GetValue(), Allocator), Allocator);
	}

	Document.AddMember("UniqueId", SerializeString(InMetadata.UniqueId, Allocator), Allocator);
	Document.AddMember("TakeNumber", InMetadata.TakeNumber, Allocator);
	Document.AddMember("Slate", SerializeString(InMetadata.Slate, Allocator), Allocator);

	Document.AddMember("Device", SerializeDevice(InMetadata.Device, Allocator), Allocator);

	if (!InMetadata.Video.IsEmpty())
	{
		Document.AddMember("Video", SerializeVideo(InMetadata.Video, Allocator), Allocator);
	}

	if (!InMetadata.Depth.IsEmpty())
	{
		Document.AddMember("Depth", SerializeVideo(InMetadata.Depth, Allocator), Allocator);
	}

	if (!InMetadata.Calibration.IsEmpty())
	{
		Document.AddMember("Calibration", SerializeCalibration(InMetadata.Calibration, Allocator), Allocator);
	}

	if (!InMetadata.Audio.IsEmpty())
	{
		Document.AddMember("Audio", SerializeAudio(InMetadata.Audio, Allocator), Allocator);
	}

	return Document;
}

TOptional<FTakeMetadataSerializerError> SerializeTakeMetadata(const FString& InFilePath, const FTakeMetadata& InMetadata)
{
	FTakeMetadataSerializerError Error;

	rapidjson::Document Document = Serialize(InMetadata);

	rapidjson::StringBuffer StringBuffer;
	rapidjson::PrettyWriter<rapidjson::StringBuffer> StringWriter(StringBuffer);
	StringWriter.SetIndent(' ', 2);

	Document.Accept(StringWriter);

	FString String = StringBuffer.GetString();

	bool Success = FFileHelper::SaveStringToFile(String, *InFilePath, FFileHelper::EEncodingOptions::ForceUTF8);

	if (!Success)
	{
		Error.Message = LOCTEXT("TakeMetadata_FailedToWriteFile", "Failed to create a take.json file");
		return MoveTemp(Error);
	}

	return TOptional<FTakeMetadataSerializerError>();
}

FTakeMetadataParser::FTakeMetadataParser() : ImplPtr(MakeUnique<FTakeMetadataParser::FImpl>())
{
}

FTakeMetadataParser::~FTakeMetadataParser() = default;
FTakeMetadataParser::FTakeMetadataParser(FTakeMetadataParser&& other) = default;
FTakeMetadataParser& FTakeMetadataParser::operator=(FTakeMetadataParser&& other) = default;

struct FTakeMetadataValidationError
{
	FString SchemaPointerName;
	FString DocumentPointerName;
	FString Keyword;

	FString ToString() const
	{
		return FText::Format(LOCTEXT("TakeMetadata_ValidationError", "Rule '{0}' at '{1}' does not comply with schema at '{2}'."),
			FText::FromString(Keyword), FText::FromString(DocumentPointerName), FText::FromString(SchemaPointerName)).ToString();
	}
};

TOptional<FTakeMetadataValidationError> Validate(const rapidjson::Document& InDocument, rapidjson::SchemaValidator& InValidator)
{
	TOptional<FTakeMetadataValidationError> Error;

	if (!InDocument.Accept(InValidator))
	{
		rapidjson::StringBuffer Buffer;
		InValidator.GetInvalidSchemaPointer().StringifyUriFragment(Buffer);
		FTakeMetadataValidationError ValidationError;
		ValidationError.SchemaPointerName = Buffer.GetString();

		ValidationError.Keyword = InValidator.GetInvalidSchemaKeyword();

		Buffer.Clear();
		InValidator.GetInvalidDocumentPointer().StringifyUriFragment(Buffer);
		ValidationError.DocumentPointerName = Buffer.GetString();

		Error = MoveTemp(ValidationError);
	}

	return Error;
}

struct FTakeMetadataParser::FImpl
{
	struct FRapidjsonValidatorBundle
	{
		using FPtr = TUniquePtr<FRapidjsonValidatorBundle>;

		rapidjson::Document Document;
		rapidjson::SchemaDocument SchemaDocument;
		rapidjson::SchemaValidator Validator;

		FRapidjsonValidatorBundle(rapidjson::Document&& InDocument) : Document(MoveTemp(InDocument)), SchemaDocument(Document), Validator(SchemaDocument)
		{}

		FRapidjsonValidatorBundle(FRapidjsonValidatorBundle&& InOther) = delete;
		FRapidjsonValidatorBundle(const FRapidjsonValidatorBundle& InOther) = delete;
		FRapidjsonValidatorBundle& operator=(FRapidjsonValidatorBundle&& InOther) = delete;
		FRapidjsonValidatorBundle& operator=(const FRapidjsonValidatorBundle& InOther) = delete;
	};

	struct FValidator
	{
		const FTakeMetadata::FSchemaVersion Version;
		rapidjson::SchemaValidator& Validator;
	};

	TOptional<FRapidjsonValidatorBundle::FPtr> VersionValidatorBundle;

	TMap<FTakeMetadata::FSchemaVersion, FRapidjsonValidatorBundle::FPtr> ValidatorBundles;

	TValueOrError<FRapidjsonValidatorBundle::FPtr, FTakeMetadataParserError> CreateValidator(const FString& InSchemaFileName)
	{
		FString SchemaString;

		if (!FFileHelper::LoadFileToString(SchemaString, *InSchemaFileName))
		{
			FTakeMetadataParserError Error;
			Error.Origin = FTakeMetadataParserError::Reader;
			Error.Message = FText::Format(LOCTEXT("TakeMetadata_SchemaNotReadable", "Cannot read version schema file: '{0}'."), FText::FromString(InSchemaFileName));
			return MakeError(MoveTemp(Error));
		}

		rapidjson::Document Document;
		if (Document.Parse(TCHAR_TO_UTF8(*SchemaString)).HasParseError())
		{
			FTakeMetadataParserError Error;
			Error.Origin = FTakeMetadataParserError::Parser;
			Error.Message = LOCTEXT("TakeMetadata_SchemaInvalid", "Schema content is not a valid.");
			return MakeError(MoveTemp(Error));
		}

		return MakeValue(MakeUnique<FRapidjsonValidatorBundle>(MoveTemp(Document)));
	}


	TValueOrError<FTakeMetadata::FSchemaVersion, FTakeMetadataParserError> DetermineDocumentVersion(const rapidjson::Document& InDocument, const FString& InSchemasDir)
	{
		if (!VersionValidatorBundle.IsSet())
		{
			const FString VersionSchemaFilePath = InSchemasDir / TEXT("version.json");

			TValueOrError<FRapidjsonValidatorBundle::FPtr, FTakeMetadataParserError> VersionValidatorResult = CreateValidator(VersionSchemaFilePath);

			if (VersionValidatorResult.HasError())
			{
				return MakeError(VersionValidatorResult.StealError());
			}

			VersionValidatorBundle = VersionValidatorResult.StealValue();
		}

		VersionValidatorBundle.GetValue()->Validator.Reset();
		VersionValidatorBundle.GetValue()->Validator.ResetError();

		TOptional<FTakeMetadataValidationError> ValidationError = Validate(InDocument, VersionValidatorBundle.GetValue()->Validator);
		if (ValidationError.IsSet())
		{
			FTakeMetadataParserError Error;
			Error.Origin = FTakeMetadataParserError::Validator;
			Error.Message = FText::Format(LOCTEXT("TakeMetadata_VersionSchemaValidationFailed", "Validation against version schema failed: {0}"), FText::FromString(ValidationError.GetValue().ToString()));
			return MakeError(MoveTemp(Error));
		}

		FTakeMetadata::FSchemaVersion Version = ParseSchemaVersion(InDocument["Version"]);
		return MakeValue(MoveTemp(Version));
	}

	TValueOrError<FValidator, FTakeMetadataParserError> GetDocumentValidator(const rapidjson::Document& InDocument)
	{
		const FString ContentDir = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetContentDir();
		const FString SchemasDir = ContentDir / TEXT("TakeMetadata") / TEXT("Schema");

		TValueOrError<FTakeMetadata::FSchemaVersion, FTakeMetadataParserError> VersionResult = DetermineDocumentVersion(InDocument, SchemasDir);
		if (VersionResult.HasError())
		{
			return MakeError(VersionResult.StealError());
		}

		FTakeMetadata::FSchemaVersion Version = VersionResult.StealValue();
		FRapidjsonValidatorBundle::FPtr* ValidatorBundle = ValidatorBundles.Find(Version);

		FString SchemaFileName = "v";
		SchemaFileName += FString::FromInt(Version.Major) + ".";
		SchemaFileName += FString::FromInt(Version.Minor) + ".json";

		if (!ValidatorBundle)
		{
			FString SchemaFilePath = SchemasDir / SchemaFileName;

			TValueOrError<FRapidjsonValidatorBundle::FPtr, FTakeMetadataParserError> ValidatorResult = CreateValidator(SchemaFilePath);

			if (ValidatorResult.HasError())
			{
				return MakeError(ValidatorResult.StealError());
			}

			ValidatorBundles.Emplace(Version, ValidatorResult.StealValue());

			ValidatorBundle = ValidatorBundles.Find(Version);
		}

		(*ValidatorBundle)->Validator.Reset();
		(*ValidatorBundle)->Validator.ResetError();

		return MakeValue(FValidator{ MoveTemp(Version), (*ValidatorBundle)->Validator });
	}
};

TValueOrError<FTakeMetadata, FTakeMetadataParserError> FTakeMetadataParser::Parse(const FString& InJsonFile)
{
	FTakeMetadataParserError Error;

	FString Extension = FPaths::GetExtension(InJsonFile);
	if (Extension != FTakeMetadata::FileExtension)
	{
		Error.Origin = FTakeMetadataParserError::Reader;
		Error.Message = FText::Format(LOCTEXT("TakeMetadata_InvalidFile", "Invalid file format (found '{0}', expected '{1}')"),
									  FText::FromString(Extension), FText::FromString(FTakeMetadata::FileExtension));
		return MakeError(MoveTemp(Error));
	}

	FString JsonString;
	if (!FFileHelper::LoadFileToString(JsonString, *InJsonFile))
	{
		Error.Origin = FTakeMetadataParserError::Reader;
		Error.Message = LOCTEXT("TakeMetadata_JsonFileNotFound", "Json file not found.");
		return MakeError(MoveTemp(Error));
	}

	rapidjson::Document Document;
	if (Document.Parse(TCHAR_TO_UTF8(*JsonString)).HasParseError())
	{
		Error.Origin = FTakeMetadataParserError::Parser;
		Error.Message = LOCTEXT("TakeMetadata_InvalidJson", "Json file is not valid.");
		return MakeError(MoveTemp(Error));
	}

	TValueOrError<FTakeMetadataParser::FImpl::FValidator, FTakeMetadataParserError> ValidatorResult = ImplPtr->GetDocumentValidator(Document);
	if (ValidatorResult.HasError())
	{
		return MakeError(ValidatorResult.StealError());
	}

	FTakeMetadataParser::FImpl::FValidator Validator = ValidatorResult.StealValue();

	TOptional<FTakeMetadataValidationError> ValidationError = Validate(Document, Validator.Validator);
	if (ValidationError.IsSet())
	{
		Error.Origin = FTakeMetadataParserError::Validator;
		Error.Message = FText::Format(LOCTEXT("TakeMetadata_SchemaValidationFailed", "Validation against take metadata schema failed: {0}"), FText::FromString(ValidationError.GetValue().ToString()));
		return MakeError(MoveTemp(Error));
	}

	return MakeValue(ParseTakeMetadata(Document, Validator.Version));
}

FTakeMetadata::FVideo::EPathType FTakeMetadataPathUtils::DetectPathType(const FString& InPath)
{
	IFileManager& FileManager = IFileManager::Get();
	return FileManager.FileExists(*InPath) ? FTakeMetadata::FVideo::EPathType::File : FTakeMetadata::FVideo::EPathType::Folder;
}

bool FTakeMetadataPathUtils::ValidatePathType(const FString& InPath, FTakeMetadata::FVideo::EPathType InPathType)
{
	bool bSpecifiedTypeIsFile = InPathType == FTakeMetadata::FVideo::EPathType::File;
	IFileManager& FileManager = IFileManager::Get();
	if (FileManager.FileExists(*InPath) && !bSpecifiedTypeIsFile)
	{
		UE_LOGF(LogTakeMetadataPathUtils, Warning, "Specified PathType \"Folder\" does not match detected type \"File\" for %ls", *InPath);
		return false;
	}
	else if (FileManager.DirectoryExists(*InPath) && bSpecifiedTypeIsFile)
	{
		UE_LOGF(LogTakeMetadataPathUtils, Warning, "Specified PathType \"File\" does not match detected type \"Folder\" for %ls", *InPath);
		return false;
	}
	return true;
}

FString FTakeMetadataPathUtils::PathTypeToString(FTakeMetadata::FVideo::EPathType InPathType)
{
	return InPathType == FTakeMetadata::FVideo::EPathType::Folder ? TEXT("Folder") : TEXT("File");
}

#undef LOCTEXT_NAMESPACE
