// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FCaptureManagerTakeMetadataModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FCaptureManagerTakeMetadataModule, CaptureManagerTakeMetadata)
