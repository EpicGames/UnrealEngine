// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#include "Interfaces/IPluginManager.h"

#include "CaptureManagerTakeMetadata.h"

#if WITH_AUTOMATION_TESTS

namespace CaptureManagerTakeMetadata_Spec
{
FString& GetJsonTestsDir()
{
	static FString JsonTestsDir = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetContentDir() / TEXT("TakeMetadata") / TEXT("TestInputs");
	return JsonTestsDir;
}

FTakeMetadataParser& GetTakeMetadataParser()
{
	static FTakeMetadataParser TakeMetadataParser;
	return TakeMetadataParser;
}

BEGIN_DEFINE_SPEC(TestTakeMetadataSchemaSpec, TEXT("TestTakeMetadataSchema"), EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

struct TestSpecification
{
	FString Name;
	FTakeMetadataParserError::EOrigin ExpectedErrorOrigin;
	TArray<FString> ExpectedMessageKeywords;
};

END_DEFINE_SPEC(TestTakeMetadataSchemaSpec)
}


void CaptureManagerTakeMetadata_Spec::TestTakeMetadataSchemaSpec::Define()
{
	using namespace CaptureManagerTakeMetadata_Spec;

	Describe(TEXT("TestTakeMetadataSchemaTests"), [this]
	{
			It(TEXT("take_valid_v4_2_mandatory_media"), [this]
				{
					TValueOrError<FTakeMetadata, FTakeMetadataParserError> Result =
						GetTakeMetadataParser().Parse(GetJsonTestsDir() / TEXT("take_valid_v4_2_mandatory_media.cptake"));

					UTEST_TRUE(TEXT("Should have value."), Result.HasValue());

					FTakeMetadata TakeMetadata = Result.GetValue();

					TestEqual(TEXT("Version.Major has unexpected value."), TakeMetadata.Version.Major, 4);
					TestEqual(TEXT("Version.Minor has unexpected value."), TakeMetadata.Version.Minor, 2);

					TestEqual(TEXT("UniqueId has unexpected value."), TakeMetadata.UniqueId, TEXT("a78613f3-e660-47e4-af6a-1298cde7c947"));
					TestEqual(TEXT("TakeNumber has unexpected value."), TakeMetadata.TakeNumber, 1);
					TestEqual(TEXT("Slate has unexpected value."), TakeMetadata.Slate, TEXT("Fox Jumps Over The Lazy Dog"));

					TestEqual(TEXT("Device.Type has unexpected value."), TakeMetadata.Device.Type, TEXT("HMC"));
					TestEqual(TEXT("Device.Model has unexpected value."), TakeMetadata.Device.Model, TEXT("StereoHMC"));
					TestEqual(TEXT("Device.Name has unexpected value."), TakeMetadata.Device.Name, TEXT("UserDev001"));

					UTEST_EQUAL(TEXT("Video array is expected to have one element."), TakeMetadata.Video.Num(), 1);
					TestEqual(TEXT("Video[0].Name has unexpected value."), TakeMetadata.Video[0].Name, TEXT("secondary"));
					TestEqual(TEXT("Video[0].Path has unexpected value."), TakeMetadata.Video[0].Path, TEXT("folder_or_file_name"));
					TestEqual(TEXT("Video[0].Format has unexpected value."), TakeMetadata.Video[0].Format, TEXT("mov"));
					TestEqual(TEXT("Video[0].FrameRate has unexpected value."), TakeMetadata.Video[0].FrameRate, 60.0f);

					UTEST_EQUAL(TEXT("Calibration array is expected to have one element."), TakeMetadata.Calibration.Num(), 1);
					TestEqual(TEXT("Calibration[0].Name has unexpected value."), TakeMetadata.Calibration[0].Name, TEXT("calibration_user_id"));
					TestEqual(TEXT("Calibration[0].Format has unexpected value."), TakeMetadata.Calibration[0].Format, TEXT("opencv"));
					TestEqual(TEXT("Calibration[0].Path has unexpected value."), TakeMetadata.Calibration[0].Path, TEXT("calib.json"));

					UTEST_EQUAL(TEXT("Audio array is expected to have one element."), TakeMetadata.Audio.Num(), 1);
					TestEqual(TEXT("Audio[0].Name has unexpected value."), TakeMetadata.Audio[0].Name, TEXT("primary"));
					TestEqual(TEXT("Audio[0].Path has unexpected value."), TakeMetadata.Audio[0].Path, TEXT("audio.wav"));

					return true;
				});

			It(TEXT("take_valid_v4_0_mandatory"), [this]
				{
					TValueOrError<FTakeMetadata, FTakeMetadataParserError> Result =
						GetTakeMetadataParser().Parse(GetJsonTestsDir() / TEXT("take_valid_v4_0_mandatory.cptake"));

					UTEST_TRUE(TEXT("Should have value."), Result.HasValue());

					FTakeMetadata TakeMetadata = Result.GetValue();

					TestEqual(TEXT("Version.Major has unexpected value."), TakeMetadata.Version.Major, 4);
					TestEqual(TEXT("Version.Minor has unexpected value."), TakeMetadata.Version.Minor, 0);

					TestEqual(TEXT("UniqueId has unexpected value."), TakeMetadata.UniqueId, TEXT("a78613f3-e660-47e4-af6a-1298cde7c947"));
					TestEqual(TEXT("TakeNumber has unexpected value."), TakeMetadata.TakeNumber, 1);
					TestEqual(TEXT("Slate has unexpected value."), TakeMetadata.Slate, TEXT("Fox Jumps Over The Lazy Dog"));

					TestEqual(TEXT("Device.Type has unexpected value."), TakeMetadata.Device.Type, TEXT("HMC"));
					TestEqual(TEXT("Device.Model has unexpected value."), TakeMetadata.Device.Model, TEXT("StereoHMC"));
					TestEqual(TEXT("Device.Name has unexpected value."), TakeMetadata.Device.Name, TEXT("UserDev001"));

					return true;
				});

			It(TEXT("take_valid_v4_0_mandatory_media"), [this]
				{
					TValueOrError<FTakeMetadata, FTakeMetadataParserError> Result =
						GetTakeMetadataParser().Parse(GetJsonTestsDir() / TEXT("take_valid_v4_0_mandatory_media.cptake"));

					UTEST_TRUE(TEXT("Should have value."), Result.HasValue());

					FTakeMetadata TakeMetadata = Result.GetValue();

					TestEqual(TEXT("Version.Major has unexpected value."), TakeMetadata.Version.Major, 4);
					TestEqual(TEXT("Version.Minor has unexpected value."), TakeMetadata.Version.Minor, 0);

					TestEqual(TEXT("UniqueId has unexpected value."), TakeMetadata.UniqueId, TEXT("a78613f3-e660-47e4-af6a-1298cde7c947"));
					TestEqual(TEXT("TakeNumber has unexpected value."), TakeMetadata.TakeNumber, 1);
					TestEqual(TEXT("Slate has unexpected value."), TakeMetadata.Slate, TEXT("Fox Jumps Over The Lazy Dog"));

					TestEqual(TEXT("Device.Type has unexpected value."), TakeMetadata.Device.Type, TEXT("HMC"));
					TestEqual(TEXT("Device.Model has unexpected value."), TakeMetadata.Device.Model, TEXT("StereoHMC"));
					TestEqual(TEXT("Device.Name has unexpected value."), TakeMetadata.Device.Name, TEXT("UserDev001"));

					UTEST_EQUAL(TEXT("Video array is expected to have one element."), TakeMetadata.Video.Num(), 1);
					TestEqual(TEXT("Video[0].Name has unexpected value."), TakeMetadata.Video[0].Name, TEXT("secondary"));
					TestEqual(TEXT("Video[0].Path has unexpected value."), TakeMetadata.Video[0].Path, TEXT("folder_or_file_name"));
					TestEqual(TEXT("Video[0].Format has unexpected value."), TakeMetadata.Video[0].Format, TEXT("mov"));
					TestEqual(TEXT("Video[0].FrameRate has unexpected value."), TakeMetadata.Video[0].FrameRate, 60.0f);

					UTEST_EQUAL(TEXT("Calibration array is expected to have one element."), TakeMetadata.Calibration.Num(), 1);
					TestEqual(TEXT("Calibration[0].Name has unexpected value."), TakeMetadata.Calibration[0].Name, TEXT("calibration_user_id"));
					TestEqual(TEXT("Calibration[0].Format has unexpected value."), TakeMetadata.Calibration[0].Format, TEXT("opencv"));
					TestEqual(TEXT("Calibration[0].Path has unexpected value."), TakeMetadata.Calibration[0].Path, TEXT("calib.json"));

					UTEST_EQUAL(TEXT("Audio array is expected to have one element."), TakeMetadata.Audio.Num(), 1);
					TestEqual(TEXT("Audio[0].Name has unexpected value."), TakeMetadata.Audio[0].Name, TEXT("primary"));
					TestEqual(TEXT("Audio[0].Path has unexpected value."), TakeMetadata.Audio[0].Path, TEXT("audio.wav"));

					return true;
				});
			
			It(TEXT("take_valid_v4_0"), [this]
				{
					TValueOrError<FTakeMetadata, FTakeMetadataParserError> Result =
						GetTakeMetadataParser().Parse(GetJsonTestsDir() / TEXT("take_valid_v4_0.cptake"));

					UTEST_TRUE(TEXT("Should have value."), Result.HasValue());

					FTakeMetadata TakeMetadata = Result.GetValue();

					TestEqual(TEXT("Version.Major has unexpected value."), TakeMetadata.Version.Major, 4);
					TestEqual(TEXT("Version.Minor has unexpected value."), TakeMetadata.Version.Minor, 0);

					FDateTime DateTime;
					FDateTime::ParseIso8601(TEXT("2023-02-27T08:57:17.796000Z"), DateTime);
					TestEqual(TEXT("DateTime has unexpected value."), TakeMetadata.DateTime.GetValue(), DateTime);
					UTEST_TRUE(TEXT("Thumbnail not set."), TakeMetadata.Thumbnail.GetThumbnailPath().IsSet());
					TestEqual(TEXT("Thumbnail has unexpected value."), TakeMetadata.Thumbnail.GetThumbnailPath().GetValue(), TEXT("thumbnail.jpg"));
					TestEqual(TEXT("UniqueId has unexpected value."), TakeMetadata.UniqueId, TEXT("a78613f3-e660-47e4-af6a-1298cde7c947"));
					TestEqual(TEXT("TakeNumber has unexpected value."), TakeMetadata.TakeNumber, 1);
					TestEqual(TEXT("Slate has unexpected value."), TakeMetadata.Slate, TEXT("Fox Jumps Over The Lazy Dog"));

					TestEqual(TEXT("Device.Type has unexpected value."), TakeMetadata.Device.Type, TEXT("HMC"));
					TestEqual(TEXT("Device.Model has unexpected value."), TakeMetadata.Device.Model, TEXT("StereoHMC"));
					TestEqual(TEXT("Device.Name has unexpected value."), TakeMetadata.Device.Name, TEXT("UserDev001"));
					TestEqual(TEXT("Device.Platform.Name has unexpected value."), TakeMetadata.Device.Platform.GetValue().Name, TEXT("iOS"));
					TestEqual(TEXT("Device.Platform.Version.Major has unexpected value."), TakeMetadata.Device.Platform.GetValue().Version.GetValue(), "17.2.1");

					UTEST_EQUAL(TEXT("Device.Software array is expected to have one element."), TakeMetadata.Device.Software.Num(), 1);
					TestEqual(TEXT("Device.Software.Name has unexpected value."), TakeMetadata.Device.Software[0].Name, TEXT("Live Link Face"));
					TestEqual(TEXT("Device.Software.Version.Major has unexpected value."), TakeMetadata.Device.Software[0].Version.GetValue(), "2.5.7");

					UTEST_EQUAL(TEXT("Video array is expected to have one element."), TakeMetadata.Video.Num(), 1);
					TestEqual(TEXT("Video[0].Name has unexpected value."), TakeMetadata.Video[0].Name, TEXT("secondary"));
					TestEqual(TEXT("Video[0].Path has unexpected value."), TakeMetadata.Video[0].Path, TEXT("folder_or_file_name"));
					TestEqual(TEXT("Video[0].PathType has unexpected value."), TakeMetadata.Video[0].PathType.GetValue(), FTakeMetadata::FVideo::EPathType::Folder);
					TestEqual(TEXT("Video[0].Format has unexpected value."), TakeMetadata.Video[0].Format, TEXT("mov"));
					TestEqual(TEXT("Video[0].Orientation has unexpected value."), TakeMetadata.Video[0].Orientation.GetValue(), FTakeMetadata::FVideo::EOrientation::CW90);

					TestEqual(TEXT("Video[0].FramesCount has unexpected value."), TakeMetadata.Video[0].FramesCount.GetValue(), 730);

					UTEST_EQUAL(TEXT("Video[0].DroppedFrames array is expected to have four elements."), TakeMetadata.Video[0].DroppedFrames.GetValue().Num(), 4);
					TestEqual(TEXT("Video[0].DroppedFrames[0] has unexpected value."), TakeMetadata.Video[0].DroppedFrames.GetValue()[0], 10);
					TestEqual(TEXT("Video[0].DroppedFrames[1] has unexpected value."), TakeMetadata.Video[0].DroppedFrames.GetValue()[1], 11);
					TestEqual(TEXT("Video[0].DroppedFrames[2] has unexpected value."), TakeMetadata.Video[0].DroppedFrames.GetValue()[2], 12);
					TestEqual(TEXT("Video[0].DroppedFrames[3] has unexpected value."), TakeMetadata.Video[0].DroppedFrames.GetValue()[3], 109);

					TestEqual(TEXT("Video[0].FrameWidth has unexpected value."), TakeMetadata.Video[0].FrameWidth.GetValue(), 1280);
					TestEqual(TEXT("Video[0].FrameHeight has unexpected value."), TakeMetadata.Video[0].FrameHeight.GetValue(), 720);
					TestEqual(TEXT("Video[0].FrameRate has unexpected value."), TakeMetadata.Video[0].FrameRate, 60.0f);
					TestEqual(TEXT("Video[0].TimecodeStart has unexpected value."), TakeMetadata.Video[0].TimecodeStart.GetValue(), TEXT("09:01:41:01.300"));

					UTEST_EQUAL(TEXT("Calibration array is expected to have one element."), TakeMetadata.Calibration.Num(), 1);
					TestEqual(TEXT("Calibration[0].Name has unexpected value."), TakeMetadata.Calibration[0].Name, TEXT("calibration_user_id"));
					TestEqual(TEXT("Calibration[0].Format has unexpected value."), TakeMetadata.Calibration[0].Format, TEXT("opencv"));
					TestEqual(TEXT("Calibration[0].Path has unexpected value."), TakeMetadata.Calibration[0].Path, TEXT("calib.json"));

					UTEST_EQUAL(TEXT("Audio array is expected to have one element."), TakeMetadata.Audio.Num(), 1);
					TestEqual(TEXT("Audio[0].Name has unexpected value."), TakeMetadata.Audio[0].Name, TEXT("primary"));
					TestEqual(TEXT("Audio[0].Path has unexpected value."), TakeMetadata.Audio[0].Path, TEXT("audio.wav"));
					TestEqual(TEXT("Audio[0].Duration has unexpected value."), TakeMetadata.Audio[0].Duration.GetValue(), 420.0f);
					TestEqual(TEXT("Audio[0].TimecodeRate has unexpected value."), TakeMetadata.Audio[0].TimecodeRate.GetValue(), 60.0f);
					TestEqual(TEXT("Audio[0].TimecodeStart has unexpected value."), TakeMetadata.Audio[0].TimecodeStart.GetValue(), TEXT("09:01:41:08.600"));

					return true;
				});
			
			It(TEXT("take_valid_v3_0"), [this]
				{
					TValueOrError<FTakeMetadata, FTakeMetadataParserError> Result =
						GetTakeMetadataParser().Parse(GetJsonTestsDir() / TEXT("take_valid_v3_0.cptake"));

					UTEST_TRUE(TEXT("Should have value."), Result.HasValue());

					FTakeMetadata TakeMetadata = Result.GetValue();

					TestEqual(TEXT("Version.Major has unexpected value."), TakeMetadata.Version.Major, 3);
					TestEqual(TEXT("Version.Minor has unexpected value."), TakeMetadata.Version.Minor, 0);

					FDateTime DateTime;
					FDateTime::ParseIso8601(TEXT("2023-02-27T08:57:17.796000Z"), DateTime);
					TestEqual(TEXT("DateTime has unexpected value."), TakeMetadata.DateTime.GetValue(), DateTime);
					UTEST_TRUE(TEXT("Thumbnail not set."), TakeMetadata.Thumbnail.GetThumbnailPath().IsSet());
					TestEqual(TEXT("Thumbnail has unexpected value."), TakeMetadata.Thumbnail.GetThumbnailPath().GetValue(), TEXT("thumbnail.jpg"));
					TestEqual(TEXT("UniqueId has unexpected value."), TakeMetadata.UniqueId, TEXT("a78613f3-e660-47e4-af6a-1298cde7c947"));
					TestEqual(TEXT("TakeNumber has unexpected value."), TakeMetadata.TakeNumber, 1);
					TestEqual(TEXT("Slate has unexpected value."), TakeMetadata.Slate, TEXT("Fox Jumps Over The Lazy Dog"));

					TestEqual(TEXT("Device.Type has unexpected value."), TakeMetadata.Device.Type, TEXT("HMC"));
					TestEqual(TEXT("Device.Model has unexpected value."), TakeMetadata.Device.Model, TEXT("StereoHMC"));
					TestEqual(TEXT("Device.Name has unexpected value."), TakeMetadata.Device.Name, TEXT("UserDev001"));
					TestEqual(TEXT("Device.Platform.Name has unexpected value."), TakeMetadata.Device.Platform.GetValue().Name, TEXT("iOS"));
					TestEqual(TEXT("Device.Platform.Version.Major has unexpected value."), TakeMetadata.Device.Platform.GetValue().Version.GetValue(), "17.2.1");

					UTEST_EQUAL(TEXT("Device.Software array is expected to have one element."), TakeMetadata.Device.Software.Num(), 1);
					TestEqual(TEXT("Device.Software.Name has unexpected value."), TakeMetadata.Device.Software[0].Name, TEXT("Live Link Face"));
					TestEqual(TEXT("Device.Software.Version.Major has unexpected value."), TakeMetadata.Device.Software[0].Version.GetValue(), "2.5.7");

					UTEST_EQUAL(TEXT("Video array is expected to have one element."), TakeMetadata.Video.Num(), 1);
					TestEqual(TEXT("Video[0].Name has unexpected value."), TakeMetadata.Video[0].Name, TEXT("secondary"));
					TestEqual(TEXT("Video[0].Path has unexpected value."), TakeMetadata.Video[0].Path, TEXT("folder_or_file_name"));
					TestEqual(TEXT("Video[0].PathType has unexpected value."), TakeMetadata.Video[0].PathType.GetValue(), FTakeMetadata::FVideo::EPathType::Folder);
					TestEqual(TEXT("Video[0].Format has unexpected value."), TakeMetadata.Video[0].Format, TEXT("mov"));
					TestEqual(TEXT("Video[0].Orientation has unexpected value."), TakeMetadata.Video[0].Orientation.GetValue(), FTakeMetadata::FVideo::EOrientation::CW90);

					TestEqual(TEXT("Video[0].FramesCount has unexpected value."), TakeMetadata.Video[0].FramesCount.GetValue(), 730);

					UTEST_EQUAL(TEXT("Video[0].DroppedFrames array is expected to have four elements."), TakeMetadata.Video[0].DroppedFrames.GetValue().Num(), 4);
					TestEqual(TEXT("Video[0].DroppedFrames[0] has unexpected value."), TakeMetadata.Video[0].DroppedFrames.GetValue()[0], 10);
					TestEqual(TEXT("Video[0].DroppedFrames[1] has unexpected value."), TakeMetadata.Video[0].DroppedFrames.GetValue()[1], 11);
					TestEqual(TEXT("Video[0].DroppedFrames[2] has unexpected value."), TakeMetadata.Video[0].DroppedFrames.GetValue()[2], 12);
					TestEqual(TEXT("Video[0].DroppedFrames[3] has unexpected value."), TakeMetadata.Video[0].DroppedFrames.GetValue()[3], 109);

					TestEqual(TEXT("Video[0].FrameWidth has unexpected value."), TakeMetadata.Video[0].FrameWidth.GetValue(), 1280);
					TestEqual(TEXT("Video[0].FrameHeight has unexpected value."), TakeMetadata.Video[0].FrameHeight.GetValue(), 720);
					TestEqual(TEXT("Video[0].FrameRate has unexpected value."), TakeMetadata.Video[0].FrameRate, 60.0f);
					TestEqual(TEXT("Video[0].TimecodeStart has unexpected value."), TakeMetadata.Video[0].TimecodeStart.GetValue(), TEXT("09:01:41:01.300"));

					UTEST_EQUAL(TEXT("Calibration array is expected to have one element."), TakeMetadata.Calibration.Num(), 1);
					TestEqual(TEXT("Calibration[0].Name has unexpected value."), TakeMetadata.Calibration[0].Name, TEXT("calibration_user_id"));
					TestEqual(TEXT("Calibration[0].Format has unexpected value."), TakeMetadata.Calibration[0].Format, TEXT("opencv"));
					TestEqual(TEXT("Calibration[0].Path has unexpected value."), TakeMetadata.Calibration[0].Path, TEXT("calib.json"));

					UTEST_EQUAL(TEXT("Audio array is expected to have one element."), TakeMetadata.Audio.Num(), 1);
					TestEqual(TEXT("Audio[0].Name has unexpected value."), TakeMetadata.Audio[0].Name, TEXT("primary"));
					TestEqual(TEXT("Audio[0].Path has unexpected value."), TakeMetadata.Audio[0].Path, TEXT("audio.wav"));
					TestEqual(TEXT("Audio[0].Duration has unexpected value."), TakeMetadata.Audio[0].Duration.GetValue(), 420.0f);
					TestEqual(TEXT("Audio[0].TimecodeRate has unexpected value."), TakeMetadata.Audio[0].TimecodeRate.GetValue(), 60.0f);
					TestEqual(TEXT("Audio[0].TimecodeStart has unexpected value."), TakeMetadata.Audio[0].TimecodeStart.GetValue(), TEXT("09:01:41:08.600"));

					return true;
				});
		
		It(TEXT("take_valid_v2_0"), [this]
			{
				TValueOrError<FTakeMetadata, FTakeMetadataParserError> Result =
					GetTakeMetadataParser().Parse(GetJsonTestsDir() / TEXT("take_valid_v2_0.cptake"));

				UTEST_TRUE(TEXT("Should have value."), Result.HasValue());

				FTakeMetadata TakeMetadata = Result.GetValue();

				TestEqual(TEXT("Version.Major has unexpected value."), TakeMetadata.Version.Major, 2);
				TestEqual(TEXT("Version.Minor has unexpected value."), TakeMetadata.Version.Minor, 0 );

				FDateTime DateTime;
				FDateTime::ParseIso8601(TEXT("2023-02-27T08:57:17.796000Z"), DateTime);
				TestEqual(TEXT("DateTime has unexpected value."), TakeMetadata.DateTime.GetValue(), DateTime);
				UTEST_TRUE(TEXT("Thumbnail not set."), TakeMetadata.Thumbnail.GetThumbnailPath().IsSet());
				TestEqual(TEXT("Thumbnail has unexpected value."), TakeMetadata.Thumbnail.GetThumbnailPath().GetValue(), TEXT("thumbnail.jpg"));
				TestEqual(TEXT("UniqueId has unexpected value."), TakeMetadata.UniqueId, TEXT("a78613f3-e660-47e4-af6a-1298cde7c947"));
				TestEqual(TEXT("TakeNumber has unexpected value."), TakeMetadata.TakeNumber, 1);
				TestEqual(TEXT("Slate has unexpected value."), TakeMetadata.Slate, TEXT("Fox Jumps Over The Lazy Dog"));

				TestEqual(TEXT("Device.Type has unexpected value."), TakeMetadata.Device.Type, TEXT("HMC"));
				TestEqual(TEXT("Device.Model has unexpected value."), TakeMetadata.Device.Model, TEXT("StereoHMC"));
				TestEqual(TEXT("Device.Name has unexpected value."), TakeMetadata.Device.Name, TEXT("UserDev001"));
				TestEqual(TEXT("Device.Platform.Name has unexpected value."), TakeMetadata.Device.Platform.GetValue().Name, TEXT("iOS"));
				TestEqual(TEXT("Device.Platform.Version.Major has unexpected value."), TakeMetadata.Device.Platform.GetValue().Version.GetValue(), "17.2.1");
					
				UTEST_EQUAL(TEXT("Device.Software array is expected to have one element."), TakeMetadata.Device.Software.Num(), 1);
				TestEqual(TEXT("Device.Software.Name has unexpected value."), TakeMetadata.Device.Software[0].Name, TEXT("Live Link Face"));
				TestEqual(TEXT("Device.Software.Version.Major has unexpected value."), TakeMetadata.Device.Software[0].Version.GetValue(), "2.5.7");
					
				UTEST_EQUAL(TEXT("Video array is expected to have one element."), TakeMetadata.Video.Num(), 1);
				TestEqual(TEXT("Video[0].Name has unexpected value."), TakeMetadata.Video[0].Name, TEXT("secondary"));
				TestEqual(TEXT("Video[0].Path has unexpected value."), TakeMetadata.Video[0].Path, TEXT("folder_or_file_name"));
				TestEqual(TEXT("Video[0].PathType has unexpected value."), TakeMetadata.Video[0].PathType.GetValue(), FTakeMetadata::FVideo::EPathType::Folder);
				TestEqual(TEXT("Video[0].Format has unexpected value."), TakeMetadata.Video[0].Format, TEXT("mov"));
				TestEqual(TEXT("Video[0].Orientation has unexpected value."), TakeMetadata.Video[0].Orientation.GetValue(), FTakeMetadata::FVideo::EOrientation::CW90);

				TestEqual(TEXT("Video[0].FramesCount has unexpected value."), TakeMetadata.Video[0].FramesCount.GetValue(), 730);

				UTEST_EQUAL(TEXT("Video[0].DroppedFrames array is expected to have four elements."), TakeMetadata.Video[0].DroppedFrames.GetValue().Num(), 4);
				TestEqual(TEXT("Video[0].DroppedFrames[0] has unexpected value."), TakeMetadata.Video[0].DroppedFrames.GetValue()[0], 10);
				TestEqual(TEXT("Video[0].DroppedFrames[1] has unexpected value."), TakeMetadata.Video[0].DroppedFrames.GetValue()[1], 11);
				TestEqual(TEXT("Video[0].DroppedFrames[2] has unexpected value."), TakeMetadata.Video[0].DroppedFrames.GetValue()[2], 12);
				TestEqual(TEXT("Video[0].DroppedFrames[3] has unexpected value."), TakeMetadata.Video[0].DroppedFrames.GetValue()[3], 109);

				TestEqual(TEXT("Video[0].FrameWidth has unexpected value."), TakeMetadata.Video[0].FrameWidth.GetValue(), 1280);
				TestEqual(TEXT("Video[0].FrameHeight has unexpected value."), TakeMetadata.Video[0].FrameHeight.GetValue(), 720);
				TestEqual(TEXT("Video[0].FrameRate has unexpected value."), TakeMetadata.Video[0].FrameRate, 60.0f);
				TestEqual(TEXT("Video[0].TimecodeStart has unexpected value."), TakeMetadata.Video[0].TimecodeStart.GetValue(), TEXT("09:01:41:01.300"));

				UTEST_EQUAL(TEXT("Calibration array is expected to have one element."), TakeMetadata.Calibration.Num(), 1);
				TestEqual(TEXT("Calibration[0].Name has unexpected value."), TakeMetadata.Calibration[0].Name, TEXT("calibration_user_id"));
				TestEqual(TEXT("Calibration[0].Path has unexpected value."), TakeMetadata.Calibration[0].Path, TEXT("calib.json"));

				UTEST_EQUAL(TEXT("Audio array is expected to have one element."), TakeMetadata.Audio.Num(), 1);
				TestEqual(TEXT("Audio[0].Name has unexpected value."), TakeMetadata.Audio[0].Name, TEXT("primary"));
				TestEqual(TEXT("Audio[0].Path has unexpected value."), TakeMetadata.Audio[0].Path, TEXT("audio.wav"));
				TestEqual(TEXT("Audio[0].Duration has unexpected value."), TakeMetadata.Audio[0].Duration.GetValue(), 420.0f);
				TestEqual(TEXT("Audio[0].TimecodeRate has unexpected value."), TakeMetadata.Audio[0].TimecodeRate.GetValue(), 60.0f);
				TestEqual(TEXT("Audio[0].TimecodeStart has unexpected value."), TakeMetadata.Audio[0].TimecodeStart.GetValue(), TEXT("09:01:41:08.600"));

				return true;
			});

		It(TEXT("take_valid_v1_0_calibration"), [this]
			{
				TValueOrError<FTakeMetadata, FTakeMetadataParserError> Result = GetTakeMetadataParser().Parse(GetJsonTestsDir() / TEXT("take_valid_v1_0_calibration.cptake"));
				TestTrue(TEXT("Should have value."), Result.HasValue());
				if (Result.HasValue())
				{
					FTakeMetadata TakeMetadata = Result.StealValue();
					TestEqual(TEXT("Calibration array is expected to have one element."), TakeMetadata.Calibration.Num(), 1);
					if (TakeMetadata.Calibration.Num() == 1)
					{
						TestEqual(TEXT("Calibration[0].Name has unexpected value."), TakeMetadata.Calibration[0].Name, TEXT("undefined"));
						TestEqual(TEXT("Calibration[0].Path has unexpected value."), TakeMetadata.Calibration[0].Path, TEXT("calib.json"));
					}
				}
			});

		TArray<TestSpecification> Tests =
		{
			{ TEXT("non_existing_take_metadata"), FTakeMetadataParserError::EOrigin::Reader, { TEXT("Json file not found.") }},
			{ TEXT("take_broken_json"), FTakeMetadataParserError::EOrigin::Parser, { TEXT("Json file is not valid.") }},
			{ TEXT("take_date_time_wrong_format"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("pattern"), TEXT("#/DateTime"), TEXT("#/properties/DateTime") }},
			{ TEXT("take_take_number_is_a_string"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("type"), TEXT("#/TakeNumber"), TEXT("#/properties/TakeNumber") }},
			{ TEXT("take_take_number_is_negative"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("minimum"), TEXT("#/TakeNumber"), TEXT("#/properties/TakeNumber") }},
			{ TEXT("take_unique_id_not_uid"), FTakeMetadataParserError::EOrigin::Validator,{ TEXT("pattern"), TEXT("#/UniqueId"), TEXT("#/definitions/UniqueIdFormat") }},
			{ TEXT("take_v1_0_calibration_in_v2_0_format"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("type"), TEXT("#/Calibration"), TEXT("#/properties/Calibration") }},
			{ TEXT("take_v1_0_format_with_v2_0_calibration"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("type"), TEXT("#/Calibration"), TEXT("#/properties/Calibration") }},
			{ TEXT("take_version_additional_property"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("additionalProperties"), TEXT("#/Version/AdditionalProp"), TEXT("#/definitions/SchemaVersionFormat") }},
			{ TEXT("take_version_major_negative_value"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("minimum"), TEXT("#/Version/Major"), TEXT("#/definitions/SchemaVersionFormat/properties/Major") }},
			{ TEXT("take_version_missing_minor_property"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("required"), TEXT("#/Version"), TEXT("#/definitions/SchemaVersionFormat") }},
			{ TEXT("take_missing_mandatory_field"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("required"), TEXT("#"), TEXT("#") }},
			{ TEXT("take_video_item_dropped_frames_index_negative"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("minimum"), TEXT("#/Video/0/DroppedFrames/0"), TEXT("#/definitions/VideoOrImageSequence/items/properties/DroppedFrames/items") }},
			{ TEXT("take_video_item_dropped_frames_not_an_array"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("type"), TEXT("#/Video/0/DroppedFrames"), TEXT("#/definitions/VideoOrImageSequence/items/properties/DroppedFrames") }},
			{ TEXT("take_video_item_frame_height_negative"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("minimum"), TEXT("#/Video/0/FrameHeight"), TEXT("#/definitions/VideoOrImageSequence/items/properties/FrameHeight") }},
			{ TEXT("take_video_item_frame_rate_negative"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("minimum"), TEXT("#/Video/0/FrameRate"), TEXT("#/definitions/VideoOrImageSequence/items/properties/FrameRate") }},
			{ TEXT("take_video_item_frame_width_negative"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("minimum"), TEXT("#/Video/0/FrameWidth"), TEXT("#/definitions/VideoOrImageSequence/items/properties/FrameWidth") }},
			{ TEXT("take_video_item_frames_count_negative"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("minimum"), TEXT("#/Video/0/FramesCount"), TEXT("#/definitions/VideoOrImageSequence/items/properties/FramesCount") }},
			{ TEXT("take_video_item_invalid_orientation"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("enum"), TEXT("#/Video/0/Orientation"), TEXT("#/definitions/VideoOrImageSequence/items/properties/Orientation") }},
			{ TEXT("take_video_item_timecode_start_invalid_format"), FTakeMetadataParserError::EOrigin::Validator, { TEXT("pattern"), TEXT("#/Video/0/TimecodeStart"), TEXT("#/definitions/TimecodeFormat") }},
			{ TEXT("take_metadata_with_not_yet_existing_schema"), FTakeMetadataParserError::EOrigin::Reader, { TEXT("Cannot read version schema file:"), TEXT("v999999.999999.json")}},
			{ TEXT("take_metadata_references_invalid_schema"), FTakeMetadataParserError::EOrigin::Parser, { TEXT("Schema content is not a valid.") }},
		};

		for (const TestSpecification& Test : Tests)
		{
			It(Test.Name, [this, Test]
				{
					TValueOrError<FTakeMetadata, FTakeMetadataParserError> Result = GetTakeMetadataParser().Parse(GetJsonTestsDir() / Test.Name + ".cptake");
					if (!TestTrue(TEXT("Should have error."), Result.HasError()))
					{
						return;
					}
					TestEqual(TEXT("Error origin not as expected."), Result.GetError().Origin, Test.ExpectedErrorOrigin);

					bool ContainsKeywordsInOrder = true;
					int32 LastFoundIndex = 0;
					FString ErrorMessage = Result.GetError().Message.ToString();
					for (const FString& Keyword : Test.ExpectedMessageKeywords)
					{
						LastFoundIndex = ErrorMessage.Find(Keyword, ESearchCase::IgnoreCase, ESearchDir::FromStart, LastFoundIndex);
						if (LastFoundIndex == INDEX_NONE)
						{
							ContainsKeywordsInOrder = false;
							break;
						}
					}

					if (!ContainsKeywordsInOrder)
					{
						FString TestOutputErrorMessage = TEXT("Unexpected error message. Got: \n\t");
						TestOutputErrorMessage += *ErrorMessage;
						TestOutputErrorMessage += "\nbut expected to find these in order:\n\t";
						for (FString Keyword : Test.ExpectedMessageKeywords)
						{
							TestOutputErrorMessage += "'" + Keyword + "', ";
						}
						TestTrue(*TestOutputErrorMessage, false);
					}
				});
		}

	});
}

#endif
