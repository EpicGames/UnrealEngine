// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/Optional.h"
#include "Misc/DateTime.h"
#include "Templates/ValueOrError.h"
#include "Internationalization/Text.h"

#include "Math/MathFwd.h"
#include "ImageCore.h"

#define UE_API CAPTUREMANAGERTAKEMETADATA_API

/** Class for constructing take thumbnails and accessing thumbnail data. */
class FTakeThumbnailData
{
public:

	struct FRawImage
	{
		TArray<FColor> DecompressedImageData;
		uint32 Width;
		uint32 Height;
		ERawImageFormat::Type Format;
	};

	/** Default thumbnail constructor. */
	UE_API FTakeThumbnailData();

	/** Construct thumbnail from file path. */
	UE_API FTakeThumbnailData(FString InImagePath);

	/** Construct thumbnail from file data array. */
	UE_API FTakeThumbnailData(TArray<uint8> InCompressedImageData);

	/** Construct thumbnail from raw data array. */
	UE_API FTakeThumbnailData(TArray<FColor> InDecompressedImageData, uint32 InWidth, uint32 InHeight, ERawImageFormat::Type InFormat);

	/** Construct thumbnail from file path. */
	UE_API void operator=(FString InImagePath);

	/** Construct thumbnail from file data array. */
	UE_API void operator=(TArray<uint8> InCompressedImageData);

	/** Assign thumbnail from raw data array. */
	UE_API void operator=(FRawImage InRawImage);

	/** Get thumbnail. */
	UE_API TOptional<TArray<uint8>> GetThumbnailData() const;
	
	/** Get thumbnail file path. */
	UE_API TOptional<FString> GetThumbnailPath() const;

	/** Get decompressed image. */
	UE_API TOptional<FRawImage> GetRawImage() const;

private:

	using FThumbnail = TVariant<FEmptyVariantState, FString, TArray<uint8>, FRawImage>;
	FThumbnail Thumbnail;
};

/** Data assosciated with a take. */
class FTakeMetadata
{
public:
	/** File extension for take metadata files. */
	static UE_API const FString FileExtension;

	/** Version of the take metadata schema. */
	struct FSchemaVersion
	{
		uint32 Major = 0;
		uint32 Minor = 0;

		bool operator==(const FSchemaVersion& rhs) const = default;

		bool operator<(const FSchemaVersion& rhs) const
		{
			return Major < rhs.Major || (Major == rhs.Major && Minor < rhs.Minor);
		}

		friend uint32 GetTypeHash(const FSchemaVersion& InVersion)
		{
			return HashCombine(GetTypeHash(InVersion.Major), GetTypeHash(InVersion.Minor));
		}
	};

	/** Device information. */
	struct FDevice
	{
		struct FPlatform
		{
			FString Name;
			TOptional<FString> Version;
		};

		struct FSoftware
		{
			FString Name;
			TOptional<FString> Version;
		};

		FString Name;
		FString Type;
		FString Model;
		

		TOptional<FDevice::FPlatform> Platform;
		TArray<FDevice::FSoftware> Software;
	};

	/** Video information. */
	struct FVideo
	{
		enum class EPathType
		{
			Folder,
			File
		};

		enum class EOrientation
		{
			Original,
			CW90,
			CW180,
			CW270
		};

		FString Name;
		FString Path;

		/** Path type (e.g. Folder or File). */
		TOptional<EPathType> PathType;

		/** Format e.g. mov, png. */
		FString Format;

		/** Orientation (e.g. Original, CW90, CW180 or CW270). */
		TOptional<EOrientation> Orientation;

		/** Number of frames. */
		TOptional<uint32> FramesCount;

		/** List of dropped frames. */
		TOptional<TArray<uint32>> DroppedFrames;

		TOptional<uint32> FrameHeight;
		TOptional<uint32> FrameWidth;

		/** Frame rate. */
		float FrameRate = 0.0f;

		/** Timecode of the first frame. */
		TOptional<FString> TimecodeStart;
	};

	/** Calibration information. */
	struct FCalibration
	{
		FString Name;
		FString Path;

		/** File format (e.g. mhaical, unreal). */
		FString Format;
	};

	/** Audio information. */
	struct FAudio
	{
		FString Name;
		FString Path;

		/** Duration in seconds. */
		TOptional<float> Duration;

		/** Timecode of the first sample. */
		TOptional<FString> TimecodeStart;

		/** Timecode rate. */
		TOptional<float> TimecodeRate;
	};

	/** Frame log information. */
	struct FFrameLog
	{
		FString Path;
	};

public:
	/** Schema version of the take metadata file. */
	FSchemaVersion Version;

	TOptional<FDateTime> DateTime;

	/** Thumbnail. */
	FTakeThumbnailData Thumbnail;

	/** Unique identifier (GUID). */
	FString UniqueId;

	uint32 TakeNumber = 0;
	FString Slate;

	/** Schema version (Major.Minor). */
	FDevice Device;

	/** Video list. */
	TArray<FVideo> Video;

	/** Depth list. */
	TArray<FVideo> Depth;

	/** Calibration list. */
	TArray<FCalibration> Calibration;

	/** Audio list. */
	TArray<FAudio> Audio;
};

struct FTakeMetadataParserError
{
	enum EOrigin
	{
		Reader, /** Error reading the file. */
		Validator, /** Error validating the file. */
		Parser /** Error parsing the file. */
	};

	EOrigin Origin;
	FText Message;
};

struct FTakeMetadataSerializerError
{
	FText Message;
};

class FTakeMetadataParser
{
public:
	UE_API FTakeMetadataParser();
	UE_API ~FTakeMetadataParser();

	FTakeMetadataParser(const FTakeMetadataParser& other) = delete;
	UE_API FTakeMetadataParser(FTakeMetadataParser&& other);

	FTakeMetadataParser& operator=(const FTakeMetadataParser& other) = delete;
	UE_API FTakeMetadataParser& operator=(FTakeMetadataParser&& other);

	/** Tries to parse file into a take metadata object. */
	UE_API TValueOrError<FTakeMetadata, FTakeMetadataParserError> Parse(const FString& InJsonFile);

private:
	struct FImpl;
	TUniquePtr<FImpl> ImplPtr;

};

/** Utility functions for handling paths within take metadata files. */
class FTakeMetadataPathUtils
{
public:
	/** Detects path type from the input path (Folder or File). */
	static UE_API FTakeMetadata::FVideo::EPathType DetectPathType(const FString& InPath);

	/** Checks the path matches the path type. */
	static UE_API bool ValidatePathType(const FString& InPath, FTakeMetadata::FVideo::EPathType InPathType);

	/** Converts path type to a string. */
	static UE_API FString PathTypeToString(FTakeMetadata::FVideo::EPathType InPathType);
};

/** Writes a take metadata object to the specified path. */
CAPTUREMANAGERTAKEMETADATA_API TOptional<FTakeMetadataSerializerError> SerializeTakeMetadata(const FString& InFilePath, const FTakeMetadata& InMetadata);


#undef UE_API
