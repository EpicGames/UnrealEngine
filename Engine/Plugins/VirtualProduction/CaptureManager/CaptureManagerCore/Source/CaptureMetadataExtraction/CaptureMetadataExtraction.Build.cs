// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using UnrealBuildTool.Rules;

public class CaptureMetadataExtraction : ModuleRules
{
	public CaptureMetadataExtraction(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
			"CaptureManagerTakeMetadata"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CaptureUtils",
			"Media",
			"CaptureDataCore",
			"CaptureManagerMediaRW",
			"DataIngestCore",
			"ElectraPlayerPlugin",
			"CaptureDataUtils",
			"Json",
			"MediaAssets"
		});

		ShortName = "CapMetExt";
	}
}
