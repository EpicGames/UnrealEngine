// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerCalibrationUtils.h"

#include "Misc/FileHelper.h"
#include "Serialization/JsonSerializer.h"

namespace UE::CaptureManager
{

FString DetectCalibrationFormat(const FString& InFilePath)
{
	FString JsonContent;
	if (!FFileHelper::LoadFileToString(JsonContent, *InFilePath))
	{
		return FString();
	}

	TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<>::Create(JsonContent);

	// Try parsing as object (Unreal format)
	TSharedPtr<FJsonObject> JsonObject;
	if (FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid())
	{
		if (JsonObject->HasField(TEXT("Calibrations")))
		{
			return TEXT("unreal");
		}
	}

	// Try parsing as array (OpenCV format)
	JsonReader = TJsonReaderFactory<>::Create(JsonContent);
	TArray<TSharedPtr<FJsonValue>> JsonArray;
	if (FJsonSerializer::Deserialize(JsonReader, JsonArray))
	{
		for (const TSharedPtr<FJsonValue>& Element : JsonArray)
		{
			const TSharedPtr<FJsonObject>* ElementObject = nullptr;
			if (Element->TryGetObject(ElementObject) && (*ElementObject)->HasField(TEXT("fx")))
			{
				return TEXT("opencv");
			}
		}
	}

	return FString();
}

} // namespace UE::CaptureManager
