// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

#define UE_API CAPTUREMETADATAEXTRACTION_API

namespace UE::CaptureManager
{

// Inspects a JSON calibration file to determine whether it is OpenCV or Unreal format.
// Returns "opencv", "unreal", or empty string if the format cannot be determined.
// Detection mirrors the entry checks in the actual parsers:
//   OpenCV (OpenCVCalibrationReader.cpp): root array, elements have "fx"
//   Unreal (UnrealCalibrationParser.cpp): root object with "Calibrations" key
UE_API FString DetectCalibrationFormat(const FString& InFilePath);

} // namespace UE::CaptureManager

#undef UE_API
