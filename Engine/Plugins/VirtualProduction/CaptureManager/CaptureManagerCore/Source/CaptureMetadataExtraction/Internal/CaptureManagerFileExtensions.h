// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Misc/CString.h"

namespace UE::CaptureManager
{

/** Returns true if the extension (without leading dot) is a recognized video container format. Case-insensitive. */
inline bool IsVideoExtension(FStringView Ext)
{
	return Ext.Equals(TEXTVIEW("mp4"), ESearchCase::IgnoreCase)
		|| Ext.Equals(TEXTVIEW("mov"), ESearchCase::IgnoreCase)
		|| Ext.Equals(TEXTVIEW("avi"), ESearchCase::IgnoreCase)
		|| Ext.Equals(TEXTVIEW("mkv"), ESearchCase::IgnoreCase)
		|| Ext.Equals(TEXTVIEW("webm"), ESearchCase::IgnoreCase);
}

/** Returns true if the extension (without leading dot) is a recognized audio format. Case-insensitive. */
inline bool IsAudioExtension(FStringView Ext)
{
	return Ext.Equals(TEXTVIEW("wav"), ESearchCase::IgnoreCase)
		|| Ext.Equals(TEXTVIEW("mp3"), ESearchCase::IgnoreCase)
		|| Ext.Equals(TEXTVIEW("flac"), ESearchCase::IgnoreCase)
		|| Ext.Equals(TEXTVIEW("m4a"), ESearchCase::IgnoreCase)
		|| Ext.Equals(TEXTVIEW("aac"), ESearchCase::IgnoreCase);
}

/** Returns true if the extension (without leading dot) is a supported image-sequence format. Case-insensitive. */
inline bool IsImageExtension(FStringView Ext)
{
	return Ext.Equals(TEXTVIEW("jpg"), ESearchCase::IgnoreCase)
		|| Ext.Equals(TEXTVIEW("jpeg"), ESearchCase::IgnoreCase)
		|| Ext.Equals(TEXTVIEW("png"), ESearchCase::IgnoreCase);
}

} // namespace UE::CaptureManager
