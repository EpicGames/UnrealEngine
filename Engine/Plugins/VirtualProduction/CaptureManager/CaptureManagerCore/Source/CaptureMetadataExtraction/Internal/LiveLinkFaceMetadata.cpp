// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkFaceMetadata.h"

#include "Utils/AppleDeviceList.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonSerializer.h"

#include "Logging/LogMacros.h"

#define LOCTEXT_NAMESPACE "LiveLinkFaceMetadata"

DEFINE_LOG_CATEGORY_STATIC(LogLiveLinkFaceMetadata, Log, All);

namespace UE::CaptureManager 
{

enum class ELiveLinkTakeMetadataError
{
	InternalError,
	InvalidArgument,
	AbortedByUser,
};

class FLiveLinkTakeMetadataError
{
public:

	FLiveLinkTakeMetadataError(FText InMessage,
		ELiveLinkTakeMetadataError InCode = ELiveLinkTakeMetadataError::InternalError);

	const FString GetMessageString() const;

private:
	const FText& GetMessage() const;
	ELiveLinkTakeMetadataError GetCode() const;

	FText Message;
	ELiveLinkTakeMetadataError Code;
};

template <typename T>
using FLiveLinkTakeMetadataResult = TValueOrError<T, FLiveLinkTakeMetadataError>;

using FLiveLinkTakeMetadataVoidResult = TValueOrError<void, FLiveLinkTakeMetadataError>;

struct FLiveLinkFaceStaticFileNames
{
	inline static const FString AudioMetadata = TEXT("audio_metadata.json");
	inline static const FString DepthData = TEXT("depth_data.bin");
	inline static const FString DepthMetadata = TEXT("depth_metadata.mhaical");
	inline static const FString FrameLog = TEXT("frame_log.csv");
	inline static const FString TakeMetadata = TEXT("take.json");
	inline static const FString Thumbnail = TEXT("thumbnail.jpg");
	inline static const FString VideoMetadata = TEXT("video_metadata.json");
	inline static const FString VideoExt = TEXT(".mov");
};

struct FLiveLinkFaceTakeMetadata
{
	int32 Version = 0;
	FString SlateName;
	FString AppVersion;
	FString DeviceModel;
	FString DeviceClass;
	FString Subject;
	FString Identifier;
	FDateTime Date;
	int32 TakeNumber = 0;
	int32 NumFrames = 0;
	bool bIsCalibrated = false;

	FString MOVFileName() const;

private:
	FString CommonFileNamePrefix() const;
	TArray<FString> GetCalibratedBlendshapeFileNames() const;
	TArray<FString> GetMHAFileNames() const;
	TArray<FString> GetCommonFileNames() const;
	TArray<FString> GetARKitFileNames() const;
};

struct FLiveLinkFaceVideoMetadata
{
	FIntPoint Resolution;
	float JpegCompressionQuality;
	FString Quality;
	EMediaOrientation Orientation;
	double FrameRate;
};

struct FLiveLinkFaceOodleMetadata
{
	FString Compressor;
	FString CompressionLevel;
	FString Version;
};

struct FLiveLinkFaceDepthMetadata
{
	FString Build;
	FIntPoint Resolution;
	FString Compression;
	FString DeviceModel;
	FString DeviceClass;
	FLiveLinkFaceOodleMetadata OodleInfo; // If Compression == "Oodle"
	EMediaOrientation Orientation;
	double FrameRate;
	float PixelSize;
	bool bShouldCompressFiles = false;

	// Lens distortion info
	TArray<float> LensDistortionLookupTable;
	TArray<float> InverseLensDistortionLookupTable;
	TArray<float> IntrinsicMatrix;
	FVector2D LensDistortionCenter;
	FVector2D IntrinsicMatrixReferenceDimensions;
};

struct FLiveLinkFaceAudioMetadata
{
	int32 BitsPerChannel = 0;
	int32 SampleRate = 0;
	int32 ChannelsPerFrame = 0;
	int32 FormatFlags = 0;
};

struct FLiveLinkFaceTakeInfo
{
	// The path to the folder where the files for this take are
	FString TakeOriginDirectory;

	TArray<uint8> RawThumbnailData;
	FLiveLinkFaceTakeMetadata TakeMetadata;
	FLiveLinkFaceVideoMetadata VideoMetadata;
	FLiveLinkFaceDepthMetadata DepthMetadata;
	FLiveLinkFaceAudioMetadata AudioMetadata;

	// Set by TryResolveVideoFilePath when the metadata-derived filename does not match the actual file on disk.
	TOptional<FString> VideoFilePathOverride;

	FString GetTakeName() const;
	FString GetVideoFilePath() const;
	FString GetDepthFilePath() const;
	FString GetFrameLogFilePath() const;
	FString GetCameraCalibrationFilePath() const;

private:
	FString GetTakePath() const;
	FString GetOutputDirectory(const FString& InTakesOriginDirectory) const;
};

class FLiveLinkFaceMetadataParser
{
public:

	static EMediaOrientation ParseOrientation(int32 InOrientation = 4);
	static bool ParseVideoMetadata(const FString& InTakeDirectory, FLiveLinkFaceVideoMetadata& OutVideoMetadata);
	static bool ParseVideoMetadataFromString(const FString& InJsonString, FLiveLinkFaceVideoMetadata& OutVideoMetadata);
	static bool ParseDepthMetadata(const FString& InTakeDirectory, FLiveLinkFaceDepthMetadata& OutDepthMetadata);
	static bool ParseAudioMetadata(const FString& InTakeDirectory, FLiveLinkFaceAudioMetadata& OutAudioMetadata);
	static bool ParseTakeInfo(const FString& InTakeDirectory, FLiveLinkFaceTakeInfo& OutTakeInfo);
	static bool ParseThumbnail(const FString& InTakeDirectory, FLiveLinkFaceTakeInfo& OutTakeInfo);
	static void ParseiOSDeviceModel(const FString& InDeviceModel, FString& OutiOSDeviceClass);

private:

	static FLiveLinkTakeMetadataResult<FLiveLinkFaceTakeMetadata> ParseTakeMetadata(const TSharedPtr<FJsonObject> JsonObject);
	static FLiveLinkTakeMetadataVoidResult ParseString(const TSharedPtr<FJsonObject> JsonObject, const FString& Key, FString& OutString);
	static FLiveLinkTakeMetadataVoidResult ParseNumber(const TSharedPtr<FJsonObject> JsonObject, const FString& Key, int32& OutNumber);
	static FLiveLinkTakeMetadataVoidResult ParseBool(const TSharedPtr<FJsonObject> JsonObject, const FString& Key, bool& OutBool);
	static FLiveLinkTakeMetadataError CreateErrorForMissingJsonKey(const FString& Key);
	static TArray<TSharedPtr<class FJsonValue>> ParseJsonArrayFromFile(const FString& InFilePath);
	static const FString ParseJsonStringFromFile(const FString& InFilePath);
	static TSharedPtr<class FJsonObject> ParseJsonObjectFromString(const FString& InJsonString);
};

static bool ContainsWhitespace(const FString& InStr)
{
	for (const FString::ElementType& Character : InStr)
	{
		if (FChar::IsWhitespace(Character))
		{
			return true;
		}
	}
	return false;
}

FLiveLinkTakeMetadataError::FLiveLinkTakeMetadataError(FText InMessage,
													   ELiveLinkTakeMetadataError InCode)
	: Message(MoveTemp(InMessage))
	, Code(InCode)
{
}

const FText& FLiveLinkTakeMetadataError::GetMessage() const
{
	return Message;
}

const FString FLiveLinkTakeMetadataError::GetMessageString() const
{
	return Message.ToString();
}

ELiveLinkTakeMetadataError FLiveLinkTakeMetadataError::GetCode() const
{
	return Code;
}

TArray<FString> FLiveLinkFaceTakeMetadata::GetCommonFileNames() const
{
	return {
		FLiveLinkFaceStaticFileNames::AudioMetadata,
		FLiveLinkFaceStaticFileNames::FrameLog,
		FLiveLinkFaceStaticFileNames::TakeMetadata,
		FLiveLinkFaceStaticFileNames::Thumbnail,
		FLiveLinkFaceStaticFileNames::VideoMetadata,
		MOVFileName()
	};
}

TArray<FString> FLiveLinkFaceTakeMetadata::GetMHAFileNames() const
{
	TArray FileNames = GetCommonFileNames();
	FileNames.Append({
		FLiveLinkFaceStaticFileNames::DepthData,
		FLiveLinkFaceStaticFileNames::DepthMetadata
	});
	return FileNames;
}

TArray<FString> FLiveLinkFaceTakeMetadata::GetARKitFileNames() const
{
	TArray FileNames = GetCommonFileNames();
	if (bIsCalibrated)
	{
		FileNames.Append(GetCalibratedBlendshapeFileNames());
	}
	else
	{
		const FString BlendshapeFileName = FString::Format(TEXT("{0}.csv"), { CommonFileNamePrefix() });
		FileNames.Add(BlendshapeFileName);
	}
	return FileNames;
}

FString FLiveLinkFaceTakeMetadata::CommonFileNamePrefix() const
{
	return FString::Format(TEXT("{0}_{1}_{2}"), { SlateName, TakeNumber, Subject });
}

TArray<FString> FLiveLinkFaceTakeMetadata::GetCalibratedBlendshapeFileNames() const
{
	FString Prefix = CommonFileNamePrefix();
	return {
		FString::Format(TEXT("{0}_cal.csv"), { Prefix }),
		FString::Format(TEXT("{0}_neutral.csv"), { Prefix }),
		FString::Format(TEXT("{0}_raw.csv"), { Prefix })
	};
}

FString FLiveLinkFaceTakeMetadata::MOVFileName() const
{
	return FString::Format(TEXT("{0}.mov"), { CommonFileNamePrefix() });
}

FString FLiveLinkFaceTakeInfo::GetTakeName() const
{	
	return FString::Format(TEXT("{0}_{1}"), { TakeMetadata.SlateName, TakeMetadata.TakeNumber });
}

FString FLiveLinkFaceTakeInfo::GetTakePath() const
{
	return FString::Format(TEXT("{0}_{1}_{2}"), { TakeMetadata.SlateName, TakeMetadata.TakeNumber, TakeMetadata.Subject });
}

FString FLiveLinkFaceTakeInfo::GetVideoFilePath() const
{
	if (VideoFilePathOverride.IsSet())
	{
		return VideoFilePathOverride.GetValue();
	}
	return FString::Format(TEXT("{0}/{1}"), { TakeOriginDirectory, TakeMetadata.MOVFileName() });
}

FString FLiveLinkFaceTakeInfo::GetDepthFilePath() const
{
	return FString::Format(TEXT("{0}/{1}"), { TakeOriginDirectory, FLiveLinkFaceStaticFileNames::DepthData });
}

FString FLiveLinkFaceTakeInfo::GetFrameLogFilePath() const
{
	return FString::Format(TEXT("{0}/{1}"), { TakeOriginDirectory, FLiveLinkFaceStaticFileNames::FrameLog });
}

FString FLiveLinkFaceTakeInfo::GetCameraCalibrationFilePath() const
{
	return FString::Format(TEXT("{0}/{1}"), { TakeOriginDirectory, FLiveLinkFaceStaticFileNames::DepthMetadata });
}

FString FLiveLinkFaceTakeInfo::GetOutputDirectory(const FString& InTakesOriginDirectory) const
{
	return TakeOriginDirectory.Mid(InTakesOriginDirectory.Len());
}

EMediaOrientation FLiveLinkFaceMetadataParser::ParseOrientation(int32 InOrientation)
{
	// 0: Portrait, 1: Landscape, 2: Portrait, 3: Landscape
	switch (InOrientation)
	{
		case 1: // Portrait
			return EMediaOrientation::Original;
		case 2:	// PortraitUpsideDown
			return EMediaOrientation::CW180;
		case 3: // landscapeLeft
			return EMediaOrientation::CW90;
		case 4: // LandscapeRight
		default:
			return EMediaOrientation::CW270;
	}
}

bool FLiveLinkFaceMetadataParser::ParseVideoMetadata(const FString& InTakeDirectory, FLiveLinkFaceVideoMetadata& OutVideoMetadata)
{
	const FString& VideoMetadataFile = InTakeDirectory / FLiveLinkFaceStaticFileNames::VideoMetadata;
	const FString& VideoMetadataString = ParseJsonStringFromFile(VideoMetadataFile);

	return !VideoMetadataString.IsEmpty() && ParseVideoMetadataFromString(VideoMetadataString, OutVideoMetadata);
}

bool FLiveLinkFaceMetadataParser::ParseVideoMetadataFromString(const FString& InJsonString, FLiveLinkFaceVideoMetadata& OutVideoMetadata)
{
	if (TSharedPtr<FJsonObject> VideoMetadataJson = ParseJsonObjectFromString(InJsonString))
	{
		// Sample video_metadata.json
		// {
		// 	"FrameRate" : 60,
		// 	"Dimensions" : {
		// 		"width" : 1280,
		// 		"height" : 720
		// 	},
		// 	"Quality" : "high",
		// 	"JpegCompressionQuality" : 0.90000000000000002
		// }

		VideoMetadataJson->TryGetNumberField(TEXT("FrameRate"), OutVideoMetadata.FrameRate);

		const TSharedPtr<FJsonObject>* DimensionsJson;
		if (VideoMetadataJson->TryGetObjectField(TEXT("Dimensions"), DimensionsJson))
		{
			DimensionsJson->ToSharedRef()->TryGetNumberField(TEXT("width"), OutVideoMetadata.Resolution.X);
			DimensionsJson->ToSharedRef()->TryGetNumberField(TEXT("height"), OutVideoMetadata.Resolution.Y);
		}

		VideoMetadataJson->TryGetStringField(TEXT("Quality"), OutVideoMetadata.Quality);
		VideoMetadataJson->TryGetNumberField(TEXT("JpegCompressionQuality"), OutVideoMetadata.JpegCompressionQuality);

		int32 Orientation = 4;
		VideoMetadataJson->TryGetNumberField(TEXT("Orientation"), Orientation);
		OutVideoMetadata.Orientation = ParseOrientation(Orientation);

		return true;
	}

	return false;
}

bool FLiveLinkFaceMetadataParser::ParseDepthMetadata(const FString& InTakeDirectory, FLiveLinkFaceDepthMetadata& OutDepthMetadata)
{
	const FString& DepthMetadataFile = InTakeDirectory / FLiveLinkFaceStaticFileNames::DepthMetadata;

	const FString& DepthMetadataString = ParseJsonStringFromFile(DepthMetadataFile);

	if (TSharedPtr<FJsonObject> DepthMetadataJson = ParseJsonObjectFromString(DepthMetadataString))
	{
		DepthMetadataJson->TryGetStringField(TEXT("Build"), OutDepthMetadata.Build);
		DepthMetadataJson->TryGetStringField(TEXT("Compression"), OutDepthMetadata.Compression);
		DepthMetadataJson->TryGetNumberField(TEXT("PixelSize"), OutDepthMetadata.PixelSize);
		DepthMetadataJson->TryGetNumberField(TEXT("DepthFrameRate"), OutDepthMetadata.FrameRate);
		DepthMetadataJson->TryGetStringField(TEXT("DeviceModel"), OutDepthMetadata.DeviceModel);
		ParseiOSDeviceModel(OutDepthMetadata.DeviceModel, OutDepthMetadata.DeviceClass);

		int32 Orientation = 4;
		DepthMetadataJson->TryGetNumberField(TEXT("Orientation"), Orientation);
		OutDepthMetadata.Orientation = ParseOrientation(Orientation);

		if (OutDepthMetadata.Compression == TEXT("Oodle"))
		{
			const TSharedPtr<FJsonObject>* OodleCompression;
			if (DepthMetadataJson->TryGetObjectField(TEXT("Oodle"), OodleCompression))
			{
				OodleCompression->ToSharedRef()->TryGetStringField(TEXT("CompressionLevel"), OutDepthMetadata.OodleInfo.CompressionLevel);
				OodleCompression->ToSharedRef()->TryGetStringField(TEXT("Compressor"), OutDepthMetadata.OodleInfo.Compressor);
				OodleCompression->ToSharedRef()->TryGetStringField(TEXT("Version"), OutDepthMetadata.OodleInfo.Version);
			}
		}

		const TArray<TSharedPtr<FJsonValue>>* LensDistortionLUTArray;
		if (DepthMetadataJson->TryGetArrayField(TEXT("LensDistortionLookupTable"), LensDistortionLUTArray))
		{
			for (const TSharedPtr<FJsonValue>& Value : *LensDistortionLUTArray)
			{
				OutDepthMetadata.LensDistortionLookupTable.Add(Value->AsNumber());
			}
		}

		const TArray<TSharedPtr<FJsonValue>>* InverseLensDistortionLUTArray;
		if (DepthMetadataJson->TryGetArrayField(TEXT("InverseLensDistortionLookupTable"), InverseLensDistortionLUTArray))
		{
			for (const TSharedPtr<FJsonValue>& Value : *InverseLensDistortionLUTArray)
			{
				OutDepthMetadata.InverseLensDistortionLookupTable.Add(Value->AsNumber());
			}
		}

		const TSharedPtr<FJsonObject>* IntrinsicMatrixReferenceDimensions;
		if (DepthMetadataJson->TryGetObjectField(TEXT("IntrinsicMatrixReferenceDimensions"), IntrinsicMatrixReferenceDimensions))
		{
			IntrinsicMatrixReferenceDimensions->ToSharedRef()->TryGetNumberField(TEXT("Width"), OutDepthMetadata.IntrinsicMatrixReferenceDimensions.X);
			IntrinsicMatrixReferenceDimensions->ToSharedRef()->TryGetNumberField(TEXT("Height"), OutDepthMetadata.IntrinsicMatrixReferenceDimensions.Y);
		}

		const TSharedPtr<FJsonObject>* DepthDimensions;
		if (DepthMetadataJson->TryGetObjectField(TEXT("DepthDimensions"), DepthDimensions))
		{
			DepthDimensions->ToSharedRef()->TryGetNumberField(TEXT("Width"), OutDepthMetadata.Resolution.X);
			DepthDimensions->ToSharedRef()->TryGetNumberField(TEXT("Height"), OutDepthMetadata.Resolution.Y);
		}

		const TSharedPtr<FJsonObject>* LensDistorionCenter;
		if (DepthMetadataJson->TryGetObjectField(TEXT("LensDistortionCenter"), LensDistorionCenter))
		{
			LensDistorionCenter->ToSharedRef()->TryGetNumberField(TEXT("X"), OutDepthMetadata.LensDistortionCenter.X);
			LensDistorionCenter->ToSharedRef()->TryGetNumberField(TEXT("Y"), OutDepthMetadata.LensDistortionCenter.Y);
		}

		const TArray<TSharedPtr<FJsonValue>>* IntrinsicMatrix;
		if (DepthMetadataJson->TryGetArrayField(TEXT("IntrinsicMatrix"), IntrinsicMatrix))
		{
			for (const TSharedPtr<FJsonValue>& Value : *IntrinsicMatrix)
			{
				OutDepthMetadata.IntrinsicMatrix.Add(Value->AsNumber());
			}
		}

		return true;
	}

	return false;
}

bool FLiveLinkFaceMetadataParser::ParseAudioMetadata(const FString& InTakeDirectory, FLiveLinkFaceAudioMetadata& OutAudioMetadata)
{
	const FString& AudioMetadataFile = InTakeDirectory / FLiveLinkFaceStaticFileNames::AudioMetadata;

	const FString& AudioMetadataString = ParseJsonStringFromFile(AudioMetadataFile);

	if (TSharedPtr<FJsonObject> AudioMetadataJson = ParseJsonObjectFromString(AudioMetadataString))
	{
		// Sample audio_metadata.json
		// {
		// 	"BitsPerChannel" : 16,
		// 	"SampleRate" : 44100,
		// 	"ChannelsPerFrame" : 1,
		// 	"FormatFlags" : 12
		// }

		AudioMetadataJson->TryGetNumberField(TEXT("BitsPerChannel"), OutAudioMetadata.BitsPerChannel);
		AudioMetadataJson->TryGetNumberField(TEXT("SampleRate"), OutAudioMetadata.SampleRate);
		AudioMetadataJson->TryGetNumberField(TEXT("ChannelsPerFrame"), OutAudioMetadata.ChannelsPerFrame);
		AudioMetadataJson->TryGetNumberField(TEXT("FormatFlags"), OutAudioMetadata.FormatFlags);

		return true;
	}

	return false;
}

bool FLiveLinkFaceMetadataParser::ParseTakeInfo(const FString& InTakeDirectory, FLiveLinkFaceTakeInfo& OutTakeInfo)
{
	const FString TakeMetadataFile = InTakeDirectory / FLiveLinkFaceStaticFileNames::TakeMetadata;

	const FString& TakeMetadataString = ParseJsonStringFromFile(TakeMetadataFile);
	const TSharedPtr TakeMetadataJson = ParseJsonObjectFromString(TakeMetadataString);

	if (!TakeMetadataJson.IsValid())
	{
		return false;
	}
	
	// Sample take.json
	// {
	// 	"frames" : 1058,
	// 	"appVersion" : "v0.1.0 (build 17)",
	// 	"slate" : "HH_Neutral_Rotation",
	// 	"calibrated" : false,
	// 	"subject" : "iPhone12Pro",
	// 	"identifier" : "20211006_HH_Neutral_Rotation_1",
	// 	"date" : "2021-10-06T11:31:46Z",
	// 	"version" : 1,
	// 	"take" : 1,
	// 	"deviceModel" : "iPhone13,3"
	// }

	FLiveLinkTakeMetadataResult<FLiveLinkFaceTakeMetadata> ParseResult = ParseTakeMetadata(TakeMetadataJson);

	if (ParseResult.HasError())
	{
		const FLiveLinkTakeMetadataError& Error = ParseResult.StealError();
		UE_LOGF(LogLiveLinkFaceMetadata, Warning, "Failed to parse take metadata: %ls", *Error.GetMessageString());
		return false;
	}

	if (!ParseResult.HasValue())
	{
		UE_LOGF(LogLiveLinkFaceMetadata, Error, "Take metadata parse result did not contain an error but no value was provided");
		return false;
	}

	OutTakeInfo.TakeMetadata = ParseResult.StealValue();
	OutTakeInfo.TakeOriginDirectory = InTakeDirectory;
	
	return true;
}

bool FLiveLinkFaceMetadataParser::ParseThumbnail(const FString& InTakeDirectory, FLiveLinkFaceTakeInfo& OutTakeInfo)
{
	const FString& ThumbnailFile = InTakeDirectory / FLiveLinkFaceStaticFileNames::Thumbnail;

	return FFileHelper::LoadFileToArray(OutTakeInfo.RawThumbnailData, *ThumbnailFile);
}

void FLiveLinkFaceMetadataParser::ParseiOSDeviceModel(const FString& InDeviceModel, FString& OutiOSDeviceClass)
{
	if (InDeviceModel.IsEmpty())
	{
		OutiOSDeviceClass = TEXT("Unspecified");
		return;
	}

	// Device model is in format [Name][FirstNumber],[SecondNumber]
	TArray<FString> ParsedStringArray;
	InDeviceModel.ParseIntoArray(ParsedStringArray, TEXT(","));

	if (ParsedStringArray[0].Contains(TEXT("iPhone")))
	{
		if (FAppleDeviceList::DeviceMap.Contains(InDeviceModel))
		{
			OutiOSDeviceClass = FAppleDeviceList::DeviceMap[InDeviceModel];
		}
		else
		{
			OutiOSDeviceClass = TEXT("iPhone");
		}
	}
	else if (ParsedStringArray[0].Contains(TEXT("iPad")))
	{
		OutiOSDeviceClass = TEXT("iPad");
	}
	else
	{
		OutiOSDeviceClass = TEXT("Unspecified");
	}
}

FLiveLinkTakeMetadataResult<FLiveLinkFaceTakeMetadata> FLiveLinkFaceMetadataParser::ParseTakeMetadata(const TSharedPtr<FJsonObject> JsonObject)
{
	FLiveLinkFaceTakeMetadata TakeMetadata;

	FString DateString;
	const TArray ParseErrors = {
		ParseString(JsonObject, TEXT("slate"), TakeMetadata.SlateName),
		ParseString(JsonObject, TEXT("appVersion"), TakeMetadata.AppVersion),
		ParseString(JsonObject, TEXT("deviceModel"), TakeMetadata.DeviceModel),
		ParseString(JsonObject, TEXT("subject"), TakeMetadata.Subject),
		ParseString(JsonObject, TEXT("identifier"), TakeMetadata.Identifier),
		ParseString(JsonObject, TEXT("date"), DateString),
		ParseNumber(JsonObject, TEXT("version"), TakeMetadata.Version),
		ParseNumber(JsonObject, TEXT("take"), TakeMetadata.TakeNumber),
		ParseNumber(JsonObject, TEXT("frames"), TakeMetadata.NumFrames),
		ParseBool(JsonObject, TEXT("calibrated"), TakeMetadata.bIsCalibrated)
	};

	TArray Errors = ParseErrors.FilterByPredicate([](const FLiveLinkTakeMetadataVoidResult& Error) { return Error.HasError(); });

	if (!Errors.IsEmpty())
	{			
		return MakeError(Errors[0].StealError());
	}
	
	ParseiOSDeviceModel(TakeMetadata.DeviceModel, TakeMetadata.DeviceClass);
	
	if (!FDateTime::ParseIso8601(*DateString, TakeMetadata.Date))
	{
		return MakeError(FLiveLinkTakeMetadataError(FText::Format(LOCTEXT("LiveLinkFaceMetadata_DateError", "Failed to parse DateString '{0}' as an Iso8601 date"), FText::FromString(DateString))));
	}

	return MakeValue(TakeMetadata);
}

FLiveLinkTakeMetadataVoidResult FLiveLinkFaceMetadataParser::ParseString(const TSharedPtr<FJsonObject> JsonObject, const FString& Key, FString &OutString)
{
	if (!JsonObject->TryGetStringField(Key, OutString))
	{
		return MakeError(CreateErrorForMissingJsonKey(Key));
	}

	return MakeValue();
}

FLiveLinkTakeMetadataVoidResult FLiveLinkFaceMetadataParser::ParseNumber(const TSharedPtr<FJsonObject> JsonObject, const FString& Key, int32 &OutNumber)
{
	if (!JsonObject->TryGetNumberField(Key, OutNumber))
	{
		return MakeError(CreateErrorForMissingJsonKey(Key));
	}

	return MakeValue();
}

FLiveLinkTakeMetadataVoidResult FLiveLinkFaceMetadataParser::ParseBool(const TSharedPtr<FJsonObject> JsonObject, const FString& Key, bool &OutBool)
{
	if (!JsonObject->TryGetBoolField(Key, OutBool))
	{
		return MakeError(CreateErrorForMissingJsonKey(Key));
	}

	return MakeValue();
}

FLiveLinkTakeMetadataError FLiveLinkFaceMetadataParser::CreateErrorForMissingJsonKey(const FString& Key)
{
	return FLiveLinkTakeMetadataError(FText::Format(LOCTEXT("LiveLinkFaceMetadata_MissingValue", "Missing value for key '{0}' in Live Link Face Metadata"), FText::FromString(Key)));
}

TArray<TSharedPtr<FJsonValue>> FLiveLinkFaceMetadataParser::ParseJsonArrayFromFile(const FString& InFilePath)
{
	FString JsonStringBuffer;
	TArray<TSharedPtr<FJsonValue>> Result;

	if (FFileHelper::LoadFileToString(JsonStringBuffer, *InFilePath) &&
		FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(JsonStringBuffer), Result))
	{
		return Result;
	}

	return TArray<TSharedPtr<FJsonValue>>();
}

const FString FLiveLinkFaceMetadataParser::ParseJsonStringFromFile(const FString& InFilePath)
{
	FString JsonStringBuffer;

	if (!IFileManager::Get().FileExists(*InFilePath))
	{
		UE_LOGF(LogLiveLinkFaceMetadata, Error, "File not found: %ls", *InFilePath);
		return {};
	}

	if (!FFileHelper::LoadFileToString(JsonStringBuffer, *InFilePath))
	{
		UE_LOGF(LogLiveLinkFaceMetadata, Error, "Failed to load file (check permissions): %ls", *InFilePath);
		return {};
	}

	return JsonStringBuffer;
}

TSharedPtr<FJsonObject> FLiveLinkFaceMetadataParser::ParseJsonObjectFromString(const FString& InJsonString)
{
	TSharedPtr<FJsonObject> Result;

	if (!FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(InJsonString), Result))
	{
		UE_LOGF(LogLiveLinkFaceMetadata, Error, "Failed to parse json string (check for syntax errors): %ls", *InJsonString);
		return nullptr;
	}

	return Result;
}

class FrameLogEntry
{
public:
	static const char VideoType = 'V';
	static const char DepthType = 'D';
	static const char AudioType = 'A';
	static const char InvalidType = '\0';

	FrameLogEntry() = default;

	static bool Parse(const FString& InLogLine, FrameLogEntry& OutLogEntry)
	{
		TArray<FString> Tokens;
		InLogLine.ParseIntoArray(Tokens, TEXT(","));

		if (Tokens.Num() < 5 || Tokens.Num() > 6)
		{
			return false;
		}

		OutLogEntry = FrameLogEntry{ MoveTemp(Tokens) };
		return true;
	}

	char EntryType()
	{
		return !Tokens[0].IsEmpty() ? Tokens[0][0] : InvalidType;
	}

	int64 FrameIndex()
	{
		return FCString::Atoi64(*Tokens[1]);
	}

	double Time()
	{
		const int64 Numerator = FCString::Atoi64(*Tokens[2]);
		const double Denominator = FCString::Atod(*Tokens[3]);
		return Numerator / Denominator;
	}

	bool Timecode(FTimecode& OutTimecode)
	{
		TArray<FString> TimecodeTokens;
		Tokens[4].ParseIntoArray(TimecodeTokens, TEXT(":"));
		if (TimecodeTokens.Num() != 4 && TimecodeTokens.Num() != 3)
		{
			return false;
		}

		// Limit hours to 0-23 else we can't accurately
		// show the clip in sequencer.
		const int32 Hours = FCString::Atoi(*TimecodeTokens[0]) % 24;
		const int32 Mins = FCString::Atoi(*TimecodeTokens[1]);

		int32 Secs = 0;
		int32 Frames = 0;
		bool bIsDropFrame = TimecodeTokens[2].Contains(TEXT(";"));
		if (bIsDropFrame)
		{
			// TimecodeTokens[2] == 00;00
			TArray<FString> SecondsAndFrames;
			TimecodeTokens[2].ParseIntoArray(SecondsAndFrames, TEXT(";"));

			if (SecondsAndFrames.Num() != 2)
			{
				return false;
			}

			Secs = FCString::Atoi(*SecondsAndFrames[0]);
			Frames = FMath::RoundHalfFromZero(FCString::Atof(*SecondsAndFrames[1]));
		}
		else
		{
			Secs = FCString::Atoi(*TimecodeTokens[2]);
			Frames = FMath::RoundHalfFromZero(FCString::Atof(*TimecodeTokens[3]));
		}

		// iPhone timecode is never drop frame - always either 30 or 60 fps
		OutTimecode = FTimecode(Hours, Mins, Secs, Frames, bIsDropFrame);
		return true;
	}

	bool IsDroppedFrame()
	{
		// If it's the old log format that didn't include dropped frame info, assume it wasn't dopped
		if (Tokens.Num() == 5)
		{
			return false;
		}

		return Tokens[5] != TEXT("0");
	}

private:

	FrameLogEntry(TArray<FString>&& InTokens) : Tokens(MoveTemp(InTokens))
	{
	}

	TArray<FString> Tokens;
};

class FFrameLogParser
{
public:

	void ParseFrameLog(const FString& InFrameLogPath, float InVideoFrameRate, float InDepthFrameRate)
	{
		TArray<FString> FrameLogLines;
		FFileHelper::LoadFileToStringArray(FrameLogLines, *InFrameLogPath);

		bool bAudioTimecodeFound = false;

		FFrameRate OriginalVideoFrameRate;
		uint32 RoundedVideoFrameRate = static_cast<uint32>(FMath::RoundToInt(InVideoFrameRate));
		if (RoundedVideoFrameRate != 0)
		{
			OriginalVideoFrameRate = FFrameRate{ RoundedVideoFrameRate, 1 };
		}

		for (const FString& Line : FrameLogLines)
		{
			FrameLogEntry LogEntry;
			if (!FrameLogEntry::Parse(Line, LogEntry))
			{
				continue;
			}

			if (LogEntry.EntryType() == FrameLogEntry::VideoType)
			{
				if (LogEntry.FrameIndex() == 0) // Take timecode from first video frame
				{
					FTimecode Timecode;
					if (!LogEntry.Timecode(Timecode))
					{
						continue;
					}

					// If frame rate isn't known, there's no sense to parse timecode
					if (OriginalVideoFrameRate == FFrameRate{})
					{
						continue;
					}

					// We re-construct the timecode using the frame rate to resolve any invalid frame number rounding
					// which may have occurred while parsing the frame log. As an example: a fractional frame number of
					// 59.780 would round to 60, which is an invalid frame number at 60FPS. Going to/from a frame number
					// fixes the problem.
					VideoTimecode = FTimecode::FromFrameNumber(Timecode.ToFrameNumber(OriginalVideoFrameRate), OriginalVideoFrameRate);
					VideoTimecodeRate = OriginalVideoFrameRate;
				}
			}
			else if (LogEntry.EntryType() == FrameLogEntry::DepthType)
			{
				if (LogEntry.FrameIndex() == 0) // Take timecode from first depth frame
				{
					FTimecode Timecode;
					if (!LogEntry.Timecode(Timecode))
					{
						continue;
					}

					uint32 RoundedDepthFrameRate = static_cast<uint32>(FMath::RoundToInt(InDepthFrameRate));
					if (RoundedDepthFrameRate == 0)
					{
						continue;
					}

					FFrameRate OriginalDepthFrameRate = FFrameRate{ RoundedDepthFrameRate, 1 };

					// This step is also needed to resolve any invalid frame number rounding which may have occurred
					// during the parsing of the frame log (as it implicitly does a conversion to/from frame number).
					// The depth timecode in the frame log matches the frame rate of the video frame rate so we use the video frame rate when creating the timespan
					DepthTimecode = FTimecode::FromTimespan(Timecode.ToTimespan(OriginalVideoFrameRate), OriginalDepthFrameRate, true);
					DepthTimecodeRate = OriginalDepthFrameRate;
				}

				++DepthFrameCount;
			}
			else if (LogEntry.EntryType() == FrameLogEntry::AudioType)
			{
				if (!bAudioTimecodeFound)
				{
					FTimecode Timecode;
					if (!LogEntry.Timecode(Timecode))
					{
						continue;
					}

					// If frame rate isn't known, there's no sense to parse timecode
					if (OriginalVideoFrameRate == FFrameRate{})
					{
						continue;
					}

					// Originally we intentionally used the same frame rate here as for depth (assuming depth at 30 FPS), but now depth frame rate can vary.
					// In the near future we could consider matching the video frame rate here instead, but for the sake of
					// reducing risk for 5.7 we will maintain the current behavior.
					FFrameRate TargetFrameRate{ 30, 1 };

					// The audio timecode is expressed at the video frame rate 
					// This step is also needed to resolve any invalid frame number rounding which may have occurred
					// during the parsing of the frame log (as it implicitly does a conversion to/from frame number).
					AudioTimecode = FTimecode::FromTimespan(Timecode.ToTimespan(OriginalVideoFrameRate), TargetFrameRate, true);
					AudioTimecodeRate = TargetFrameRate;
					bAudioTimecodeFound = true;
				}
			}
		}

		// If no audio timecode was specified, assume it's the same as video
		if (!bAudioTimecodeFound)
		{
			AudioTimecode = VideoTimecode;
		}
	}

	FTimecode VideoTimecode;
	FTimecode DepthTimecode;
	FTimecode AudioTimecode;

	FFrameRate VideoTimecodeRate;
	FFrameRate DepthTimecodeRate;
	FFrameRate AudioTimecodeRate;

	int32 DepthFrameCount = 0;
};

static FTakeMetadata::FVideo::EOrientation ConvertOldOrientation(EMediaOrientation InOrientation)
{
	switch (InOrientation)
	{
		case EMediaOrientation::Original:
			return FTakeMetadata::FVideo::EOrientation::Original;
		case EMediaOrientation::CW90:
			return FTakeMetadata::FVideo::EOrientation::CW90;
		case EMediaOrientation::CW180:
			return FTakeMetadata::FVideo::EOrientation::CW180;
		case EMediaOrientation::CW270:
		default:
			return FTakeMetadata::FVideo::EOrientation::CW270;
	}
}

bool HasDepthMetadataFile(const FString& InTakeDirectory)
{
	const FString& DepthMetadataFile = InTakeDirectory / FLiveLinkFaceStaticFileNames::DepthMetadata;

	return IFileManager::Get().FileExists(*DepthMetadataFile);
}

FTakeMetadata ConvertOldToNewTakeMetadata(const FLiveLinkFaceTakeInfo& InTakeInfo)
{
	FTakeMetadata NewTakeInfo;

	NewTakeInfo.Version.Major = 3;
	NewTakeInfo.Version.Minor = 0;

	// Create new guid for old takes
	NewTakeInfo.UniqueId = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensLower);
	NewTakeInfo.DateTime = InTakeInfo.TakeMetadata.Date;
	NewTakeInfo.TakeNumber = InTakeInfo.TakeMetadata.TakeNumber;
	NewTakeInfo.Slate = InTakeInfo.TakeMetadata.SlateName;

	NewTakeInfo.Thumbnail = FTakeThumbnailData(InTakeInfo.TakeOriginDirectory / FLiveLinkFaceStaticFileNames::Thumbnail);

	NewTakeInfo.Device.Model = InTakeInfo.TakeMetadata.DeviceModel;

	FTakeMetadata::FDevice::FSoftware Software;
	Software.Version = InTakeInfo.TakeMetadata.AppVersion;
	NewTakeInfo.Device.Software.Add(MoveTemp(Software));

	FTakeMetadata::FDevice::FPlatform Platform;
	Platform.Name = TEXT("iOS");
	Platform.Version = FString();
	NewTakeInfo.Device.Platform = MoveTemp(Platform);

	FFrameLogParser Parser;
	Parser.ParseFrameLog(InTakeInfo.GetFrameLogFilePath(), InTakeInfo.VideoMetadata.FrameRate, InTakeInfo.DepthMetadata.FrameRate);

	FTakeMetadata::FVideo Video;
	Video.Name = "Video";
	Video.FrameWidth = InTakeInfo.VideoMetadata.Resolution.X;
	Video.FrameHeight = InTakeInfo.VideoMetadata.Resolution.Y;
	Video.Orientation = ConvertOldOrientation(InTakeInfo.VideoMetadata.Orientation);
	Video.FrameRate = InTakeInfo.VideoMetadata.FrameRate;
	Video.FramesCount = static_cast<uint32>(InTakeInfo.TakeMetadata.NumFrames);
	Video.DroppedFrames = TArray<uint32>{};
	Video.Format = TEXT("mov");
	Video.Path = InTakeInfo.GetVideoFilePath();
	Video.PathType = FTakeMetadata::FVideo::EPathType::File;
	Video.TimecodeStart = Parser.VideoTimecode.ToString();

	NewTakeInfo.Video.Add(MoveTemp(Video));

	if (HasDepthMetadataFile(InTakeInfo.TakeOriginDirectory))
	{
		FTakeMetadata::FVideo Depth;
		Depth.Name = "Depth";
		Depth.FramesCount = Parser.DepthFrameCount;
		Depth.FrameRate = InTakeInfo.DepthMetadata.FrameRate;
		Depth.FrameWidth = InTakeInfo.DepthMetadata.Resolution.X;
		Depth.FrameHeight = InTakeInfo.DepthMetadata.Resolution.Y;
		Depth.Orientation = ConvertOldOrientation(InTakeInfo.DepthMetadata.Orientation);
		Depth.Format = TEXT("mha_depth");
		Depth.Path = InTakeInfo.GetDepthFilePath();
		Depth.PathType = FTakeMetadata::FVideo::EPathType::File;
		Depth.TimecodeStart = Parser.DepthTimecode.ToString();

		NewTakeInfo.Depth.Add(MoveTemp(Depth));
		NewTakeInfo.Calibration.Push(FTakeMetadata::FCalibration{ TEXT("undefined"), InTakeInfo.GetCameraCalibrationFilePath(), TEXT("mhaical") });
	}

	FTakeMetadata::FAudio Audio;
	Audio.Name = "Audio";
	Audio.Path = InTakeInfo.GetVideoFilePath();

	// We do not know the audio duration and so we must estimate it from the video. This apporach has been deemed 
	// acceptable for the moment, based on how this duration value gets used.
	Audio.Duration = static_cast<float>(InTakeInfo.TakeMetadata.NumFrames) / InTakeInfo.VideoMetadata.FrameRate;
	Audio.TimecodeRate = Parser.AudioTimecodeRate.AsDecimal();
	Audio.TimecodeStart = Parser.AudioTimecode.ToString();

	TArray< FTakeMetadata::FAudio> Audios = { Audio };
	NewTakeInfo.Audio = Audios;

	return NewTakeInfo;
}

TOptional<FText> TakeDurationExceedsLimit(const float InDurationInSeconds)
{
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("au.SoundWaveImportLengthLimitInSeconds"));
	if (!CVar)
	{
		return {};
	}

	static constexpr float Unlimited = -1.f;
	float Limit = CVar->GetFloat();

	if (FMath::IsNearlyEqual(Limit, Unlimited) && FMath::IsNegativeOrNegativeZero(Limit - InDurationInSeconds))
	{
		return {};
	}

	const FText Message = LOCTEXT("TakeDurationExceedsLimit", "Take duration ({0} seconds) exceeds allowed limit ({1} seconds).");

	FNumberFormattingOptions Options;
	Options.MaximumFractionalDigits = 2;
	Options.MinimumFractionalDigits = 2;

	return FText::Format(Message, FText::AsNumber(InDurationInSeconds, &Options), FText::AsNumber(Limit, &Options));
}

// If the video file derived from take.json metadata does not exist on disk, scan the directory
// for a single .mov file and use it instead. Returns false only when multiple .mov files are
// found, as the correct one cannot be determined.
static bool TryResolveVideoFilePath(const FString& InTakeDirectory, FLiveLinkFaceTakeInfo& OutTakeInfo)
{
	const FString ExpectedVideoPath = OutTakeInfo.GetVideoFilePath();
	if (IFileManager::Get().FileExists(*ExpectedVideoPath))
	{
		return true;
	}

	TArray<FString> MovFiles;
	IFileManager::Get().FindFiles(MovFiles, *(InTakeDirectory / (TEXT("*") + FLiveLinkFaceStaticFileNames::VideoExt)), true, false);

	if (MovFiles.Num() > 1)
	{
		UE_LOGF(LogLiveLinkFaceMetadata, Warning,
			"Multiple .mov files found in take directory '%ls'; cannot determine correct video file for ingest",
			*InTakeDirectory);
		return false;
	}

	if (MovFiles.Num() == 1)
	{
		UE_LOGF(LogLiveLinkFaceMetadata, Log,
			"Expected video file '%ls' not found; using '%ls' instead",
			*ExpectedVideoPath, *(InTakeDirectory / MovFiles[0]));
		OutTakeInfo.VideoFilePathOverride = InTakeDirectory / MovFiles[0];
	}

	// If no .mov found at all, leave the path as-is. Downstream ingest will surface the missing file error.
	return true;
}

#define REPORT_IF_ERROR_AND_RETURN(Result, Text) if (!Result) { OutValidationErrors.Add(Text); return {};}
#define REPORT_IF_ERROR(Result, Text) if (!Result) { OutValidationErrors.Add(Text); }

namespace LiveLinkMetadata {

TOptional<FTakeMetadata> ParseOldLiveLinkTakeMetadata(const FString& InTakeDirectory, TArray<FText>& OutValidationErrors)
{
	FLiveLinkFaceTakeInfo TakeInfo;

	REPORT_IF_ERROR_AND_RETURN(FLiveLinkFaceMetadataParser::ParseTakeInfo(InTakeDirectory, TakeInfo), LOCTEXT("ParseOldTakeMetadata_ParseTakeInfoFailed", "Failed to parse take metadata"));
	REPORT_IF_ERROR_AND_RETURN(FLiveLinkFaceMetadataParser::ParseVideoMetadata(InTakeDirectory, TakeInfo.VideoMetadata), LOCTEXT("ParseOldTakeMetadata_ParseVideoInfoFailed", "Failed to parse take video metadata"));
	REPORT_IF_ERROR_AND_RETURN(FLiveLinkFaceMetadataParser::ParseAudioMetadata(InTakeDirectory, TakeInfo.AudioMetadata), LOCTEXT("ParseOldTakeMetadata_ParseAudioInfoFailed", "Failed to parse take audio metadata"));

	// Shouldn't block the ingest process if this fails
	FLiveLinkFaceMetadataParser::ParseThumbnail(InTakeDirectory, TakeInfo);

	REPORT_IF_ERROR_AND_RETURN(TryResolveVideoFilePath(InTakeDirectory, TakeInfo), LOCTEXT("ParseOldTakeMetadata_MultipleMovFiles", "Multiple .mov files found in take directory; cannot determine correct video file"));

	REPORT_IF_ERROR(!ContainsWhitespace(TakeInfo.GetTakeName()), LOCTEXT("ParseOldTakeMetadata_TakeNameContainsWhiteSpace", "Take name contains white space character(s)"));
	REPORT_IF_ERROR(!ContainsWhitespace(TakeInfo.TakeMetadata.Subject), LOCTEXT("ParseOldTakeMetadata_SubjectContainsWhiteSpace", "Subject contains white space character(s)"));
	REPORT_IF_ERROR(!ContainsWhitespace(TakeInfo.TakeMetadata.SlateName), LOCTEXT("ParseOldTakeMetadata_SlateNameContainsWhiteSpace", "Slate name contains white space character(s)"));

	REPORT_IF_ERROR(FCString::IsPureAnsi(*InTakeDirectory), LOCTEXT("ParseOldTakeMetadata_UnsupportedCharactersInTakeDirectoryPath", "Take path contains unsupported text characters"));

	const FString& SlateName = TakeInfo.TakeMetadata.SlateName;
	REPORT_IF_ERROR(FCString::IsPureAnsi(*SlateName), FText::Format(LOCTEXT("ParseOldTakeMetadata_UnsupportedCharactersInSlateName", "Slate name '{0}' contains unsupported text characters"), FText::FromString(SlateName)));

	const FString& Subject = TakeInfo.TakeMetadata.Subject;
	REPORT_IF_ERROR(FCString::IsPureAnsi(*Subject), FText::Format(LOCTEXT("ParseOldTakeMetadata_UnsupportedCharactersInSubjectName", "Subject name '{0}' contains unsupported text characters"), FText::FromString(Subject)));

	REPORT_IF_ERROR(FLiveLinkFaceMetadataParser::ParseDepthMetadata(InTakeDirectory, TakeInfo.DepthMetadata), LOCTEXT("ParseOldTakeMetadata_ParseDepthInfoFailed", "Failed to parse take depth metadata"));

	TOptional<FText> TakeDurationResult = TakeDurationExceedsLimit(static_cast<float>(TakeInfo.TakeMetadata.NumFrames) / TakeInfo.VideoMetadata.FrameRate);
	if (TakeDurationResult.IsSet())
	{
		OutValidationErrors.Add(TakeDurationResult.GetValue());
	}

	return ConvertOldToNewTakeMetadata(TakeInfo);
}

TArray<FTakeMetadata::FVideo> ParseOldLiveLinkVideoMetadataFromString(const FString& InJsonString, TArray<FText>& OutValidationErrors)
{
	FLiveLinkFaceTakeInfo TakeInfo;
	REPORT_IF_ERROR_AND_RETURN(FLiveLinkFaceMetadataParser::ParseVideoMetadataFromString(InJsonString, TakeInfo.VideoMetadata), LOCTEXT("ParseOldTakeMetadata_ParseVideoInfoFailed", "Failed to parse take video metadata"));

	return ConvertOldToNewTakeMetadata(TakeInfo).Video;
}

}

}

#undef LOCTEXT_NAMESPACE
