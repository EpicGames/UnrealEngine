// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dom/JsonObject.h"
#include "IMediaTextureSample.h"

#include "Containers/UnrealString.h"

#include "CaptureManagerTakeMetadata.h"

#define UE_API CAPTUREMETADATAEXTRACTION_API

namespace UE::CaptureManager::LiveLinkMetadata
{

UE_API TOptional<FTakeMetadata> ParseOldLiveLinkTakeMetadata(const FString& InJsonFile, TArray<FText>& OutValidationError);
UE_API TArray<FTakeMetadata::FVideo> ParseOldLiveLinkVideoMetadataFromString(const FString& InJsonString, TArray<FText>& OutValidationError);

}
#undef UE_API
