// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkFaceMetadataExtractor.h"

#include "TakeArchiveMetadataExtractor.h"
#include "LiveLinkFaceMetadata.h"
#include "StereoCameraMetadataParseUtils.h"

#include "HAL/FileManager.h"
#include "Misc/Paths.h"

namespace UE::CaptureManager
{

TValueOrError<FTakeMetadata, ELiveLinkFaceExtractionError>
ExtractLiveLinkFaceMetadata(const FString& InTakeDirectoryPath, const FExtractionConfig& InConfig)
{
	if (InTakeDirectoryPath.IsEmpty())
	{
		return MakeError(ELiveLinkFaceExtractionError::DirectoryNotFound);
	}

	const FString TakeDirectoryPath = FPaths::ConvertRelativePathToFull(InTakeDirectoryPath);

	if (!FPaths::DirectoryExists(TakeDirectoryPath))
	{
		return MakeError(ELiveLinkFaceExtractionError::DirectoryNotFound);
	}

	constexpr bool bFindFiles = true;
	constexpr bool bFindDirectories = false;

	TArray<FString> TakeFiles;
	IFileManager::Get().FindFiles(
		TakeFiles,
		*(FPaths::Combine(TakeDirectoryPath, TEXT("*.") + FTakeMetadata::FileExtension)),
		bFindFiles,
		bFindDirectories
	);

	if (TakeFiles.Num() > 1)
	{
		return MakeError(ELiveLinkFaceExtractionError::MultipleMetadataFilesFound);
	}

	if (TakeFiles.Num() == 1)
	{
		TValueOrError<FTakeMetadata, ETakeArchiveExtractionError> Result = ExtractTakeArchiveMetadata(
			FPaths::Combine(TakeDirectoryPath, TakeFiles[0]),
			InConfig
		);

		if (Result.HasValue())
		{
			return MakeValue(Result.StealValue());
		}

		return MakeError(ELiveLinkFaceExtractionError::MetadataFormatNotRecognized);
	}

	// Fall back to legacy formats. Try LiveLink first (take.json), then HMC/StereoCamera.
	// The stereo fallback is needed because this extractor serves as a base for CPS devices
	// including HMC, which used the old StereoCamera take.json format.
	TArray<FText> ValidationFailures;
	TOptional<FTakeMetadata> OldFormatResult = LiveLinkMetadata::ParseOldLiveLinkTakeMetadata(TakeDirectoryPath, ValidationFailures);

	if (OldFormatResult.IsSet())
	{
		return MakeValue(MoveTemp(OldFormatResult.GetValue()));
	}

	OldFormatResult = StereoCameraMetadata::ParseOldStereoCameraMetadata(TakeDirectoryPath, ValidationFailures);

	if (OldFormatResult.IsSet())
	{
		return MakeValue(MoveTemp(OldFormatResult.GetValue()));
	}

	return MakeError(ELiveLinkFaceExtractionError::MetadataFileNotFound);
}

} // namespace UE::CaptureManager
