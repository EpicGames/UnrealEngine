// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerExtractionConfig.h"
#include "CaptureManagerTakeMetadata.h"

#define UE_API CAPTUREMETADATAEXTRACTION_API

namespace UE::CaptureManager
{

enum class ELiveLinkFaceExtractionError : uint8
{
	DirectoryNotFound,
	MetadataFileNotFound,        // directory exists but contains no .cptake or legacy metadata
	MetadataFormatNotRecognized, // metadata file found but could not be parsed
	MultipleMetadataFilesFound,  // directory contains more than one .cptake file; legacy JSON is not multiplicity-checked
};

// Extracts take metadata from a Live Link Face capture directory.
// Supports both .cptake and legacy directory layouts.
UE_API TValueOrError<FTakeMetadata, ELiveLinkFaceExtractionError>
ExtractLiveLinkFaceMetadata(const FString& InTakeDirectoryPath, const FExtractionConfig& InConfig);

} // namespace UE::CaptureManager

#undef UE_API
