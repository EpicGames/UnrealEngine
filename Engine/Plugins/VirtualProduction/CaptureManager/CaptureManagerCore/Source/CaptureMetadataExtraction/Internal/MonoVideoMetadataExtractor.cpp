// Copyright Epic Games, Inc. All Rights Reserved.

#include "MonoVideoMetadataExtractor.h"

#include "Asset/CaptureAssetSanitization.h"
#include "CaptureManagerFileExtensions.h"
#include "Utils/CaptureExtractTimecode.h"
#include "Utils/ParseTakeUtils.h"

#include "HAL/FileManager.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"

DEFINE_LOG_CATEGORY_STATIC(LogMonoVideoExtractor, Log, All);

namespace UE::CaptureManager
{

TValueOrError<FTakeMetadata, EMonoVideoExtractionError>
ExtractMonoVideoMetadata(FMonoVideoDescriptor InDescriptor, const FExtractionConfig& InConfig)
{
	// Resolve absolute paths

	InDescriptor.VideoFilePath = FPaths::ConvertRelativePathToFull(InDescriptor.VideoFilePath);
	for (FString& AudioPath : InDescriptor.AudioFilePaths)
	{
		AudioPath = FPaths::ConvertRelativePathToFull(AudioPath);
	}

	if (!FPaths::FileExists(InDescriptor.VideoFilePath))
	{
		return MakeError(EMonoVideoExtractionError::VideoFileNotFound);
	}

	const FString Extension = FPaths::GetExtension(InDescriptor.VideoFilePath);
	if (!IsVideoExtension(Extension))
	{
		return MakeError(EMonoVideoExtractionError::UnsupportedVideoFormat);
	}

	for (const FString& AudioPath : InDescriptor.AudioFilePaths)
	{
		if (!FPaths::FileExists(AudioPath))
		{
			return MakeError(EMonoVideoExtractionError::AudioFileNotFound);
		}
	}

	// Build metadata from known inputs before attempting FFprobe enrichment
	FTakeMetadata Metadata;
	Metadata.Version.Major = 4;
	Metadata.Version.Minor = 1;
	FString Slate = InDescriptor.Slate.IsEmpty() ? FPaths::GetBaseFilename(InDescriptor.VideoFilePath) : InDescriptor.Slate;
	SanitizePackagePath(Slate, TEXT('_'));
	Metadata.Slate = MoveTemp(Slate);
	Metadata.TakeNumber = (InDescriptor.TakeNumber >= 1) ? static_cast<uint32>(InDescriptor.TakeNumber) : 1u;
	Metadata.UniqueId = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensLower);
	Metadata.DateTime = IFileManager::Get().GetStatData(*InDescriptor.VideoFilePath).CreationTime;
	Metadata.Device.Model = TEXT("MonoVideo");

	FTakeMetadata::FVideo& Video = Metadata.Video.AddDefaulted_GetRef();
	Video.Name = FPaths::GetBaseFilename(InDescriptor.VideoFilePath);
	Video.Format = Extension.ToLower();
	Video.Path = InDescriptor.VideoFilePath;
	Video.PathType = FTakeMetadata::FVideo::EPathType::File;

	// Attempt to enrich the metadata using ffprobe/electra

	TOptional<Private::FFProbeCommand> ProbeCommand;

	if (InConfig.bUseFFprobe && !InConfig.FFmpegPath.IsEmpty())
	{
		ProbeCommand = Private::FFProbeCommand(InConfig.FFmpegPath);
	}

	FCaptureExtractVideoInfo::FResult VideoInfoResult = FCaptureExtractVideoInfo::Create(InDescriptor.VideoFilePath, ProbeCommand);

	if (VideoInfoResult.IsValid())
	{
		FCaptureExtractVideoInfo VideoInfo = VideoInfoResult.StealValue();

		const FString TimecodeStartString = VideoInfo.GetTimecode().ToString();
		Video.TimecodeStart = TimecodeStartString;
		Video.FrameRate = static_cast<float>(VideoInfo.GetFrameRate().AsDecimal());

		switch (VideoInfo.GetVideoOrientation())
		{
		case EMediaOrientation::CW90:
			Video.Orientation = FTakeMetadata::FVideo::EOrientation::CW90;
			break;
		case EMediaOrientation::CW180:
			Video.Orientation = FTakeMetadata::FVideo::EOrientation::CW180;
			break;
		case EMediaOrientation::CW270:
			Video.Orientation = FTakeMetadata::FVideo::EOrientation::CW270;
			break;
		default:
			Video.Orientation = FTakeMetadata::FVideo::EOrientation::Original;
		}

		if (InDescriptor.AudioFilePaths.IsEmpty() && VideoInfo.ContainsAudio())
		{
			FTakeMetadata::FAudio& Audio = Metadata.Audio.AddDefaulted_GetRef();
			Audio.Name = FPaths::GetBaseFilename(InDescriptor.VideoFilePath);
			Audio.Path = InDescriptor.VideoFilePath;
			Audio.Duration = VideoInfo.GetAudioDurationSeconds();
			Audio.TimecodeStart = TimecodeStartString;
			Audio.TimecodeRate = Video.FrameRate;
		}
	}
	else
	{
		UE_LOGF(
			LogMonoVideoExtractor,
			Warning,
			"Failed to extract video info from '%ls'. Timecode, frame rate, orientation, and embedded audio detection unavailable.",
			*InDescriptor.VideoFilePath
		);
	}

	// External audio - probe independently for timecodes
	const FFrameRate VideoFrameRate = Video.FrameRate > 0.0f ? UE::CaptureManager::ParseFrameRate(Video.FrameRate) : FFrameRate();

	for (const FString& AudioPath : InDescriptor.AudioFilePaths)
	{
		FTakeMetadata::FAudio& Audio = Metadata.Audio.AddDefaulted_GetRef();
		Audio.Name = FPaths::GetBaseFilename(AudioPath);
		Audio.Path = AudioPath;

		TSharedPtr<FCaptureExtractAudioTimecode> AudioExtractor = MakeShareable(new FCaptureExtractAudioTimecode(AudioPath));
		FCaptureExtractAudioTimecode::FTimecodeInfoResult TimecodeResult = AudioExtractor->Extract(VideoFrameRate);

		if (TimecodeResult.IsValid())
		{
			FTimecodeInfo TimecodeInfo = TimecodeResult.GetValue();
			Audio.TimecodeStart = TimecodeInfo.Timecode.ToString();
			Audio.TimecodeRate = static_cast<float>(TimecodeInfo.TimecodeRate.AsDecimal());
		}
	}

	return MakeValue(MoveTemp(Metadata));
}

} // namespace UE::CaptureManager
