// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerExtractionConfig.h"
#include "CaptureManagerTakeMetadata.h"

#define UE_API CAPTUREMETADATAEXTRACTION_API

namespace UE::CaptureManager
{

enum class EMonoVideoExtractionError : uint8
{
	VideoFileNotFound,
	UnsupportedVideoFormat,
	AudioFileNotFound,
};

struct FMonoVideoDescriptor
{
	FString VideoFilePath;
	TArray<FString> AudioFilePaths; // empty = use embedded audio if present
	FString Slate;                  // empty = derived from video filename stem
	int32 TakeNumber = 1;           // clamped to 1 if < 1
};

// Media probing (timecodes, frame rate, orientation) is best-effort.
// Probe failure is non-fatal - extraction succeeds with missing values.
UE_API TValueOrError<FTakeMetadata, EMonoVideoExtractionError>
ExtractMonoVideoMetadata(FMonoVideoDescriptor InDescriptor, const FExtractionConfig& InConfig);

} // namespace UE::CaptureManager

#undef UE_API
