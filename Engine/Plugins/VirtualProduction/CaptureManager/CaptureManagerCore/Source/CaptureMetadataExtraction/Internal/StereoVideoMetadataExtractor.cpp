// Copyright Epic Games, Inc. All Rights Reserved.

#include "StereoVideoMetadataExtractor.h"
#include "CaptureManagerCalibrationUtils.h"
#include "CaptureManagerFileExtensions.h"

#include "Utils/CaptureExtractTimecode.h"
#include "Asset/CaptureAssetSanitization.h"

#include "HAL/FileManager.h"
#include "ImageUtils.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"

DEFINE_LOG_CATEGORY_STATIC(LogStereoVideoExtractor, Log, All);

namespace UE::CaptureManager
{

namespace Private
{

enum class EVideoInputType
{
	File,
	ImageSequence,
};

static TValueOrError<EVideoInputType, EStereoVideoExtractionError> ResolveVideoInputType(
	const FString& InPath,
	EStereoVideoExtractionError NotFoundError,
	EStereoVideoExtractionError UnsupportedFormatError
)
{
	if (InPath.IsEmpty())
	{
		return MakeError(NotFoundError);
	}

	if (FPaths::DirectoryExists(InPath))
	{
		return MakeValue(EVideoInputType::ImageSequence);
	}

	if (FPaths::FileExists(InPath))
	{
		if (IsVideoExtension(FPaths::GetExtension(InPath)))
		{
			return MakeValue(EVideoInputType::File);
		}
		return MakeError(UnsupportedFormatError);
	}

	return MakeError(NotFoundError);
}

} // namespace Private

TValueOrError<FTakeMetadata, EStereoVideoExtractionError>
ExtractStereoVideoMetadata(FStereoVideoDescriptor InDescriptor, const FExtractionConfig& InConfig)
{
	// Resolve absolute paths

	if (!InDescriptor.VideoPathA.IsEmpty())
	{
		InDescriptor.VideoPathA = FPaths::ConvertRelativePathToFull(InDescriptor.VideoPathA);
	}
	if (!InDescriptor.VideoPathB.IsEmpty())
	{
		InDescriptor.VideoPathB = FPaths::ConvertRelativePathToFull(InDescriptor.VideoPathB);
	}
	for (FString& AudioPath : InDescriptor.AudioFilePaths)
	{
		AudioPath = FPaths::ConvertRelativePathToFull(AudioPath);
	}
	if (!InDescriptor.CalibrationFilePath.IsEmpty())
	{
		InDescriptor.CalibrationFilePath = FPaths::ConvertRelativePathToFull(InDescriptor.CalibrationFilePath);
	}

	// Validate and classify inputs

	TValueOrError<Private::EVideoInputType, EStereoVideoExtractionError> TypeA = Private::ResolveVideoInputType(
		InDescriptor.VideoPathA,
		EStereoVideoExtractionError::VideoANotFound,
		EStereoVideoExtractionError::UnsupportedVideoFormatA
	);

	if (TypeA.HasError())
	{
		return MakeError(TypeA.GetError());
	}

	TValueOrError<Private::EVideoInputType, EStereoVideoExtractionError> TypeB = Private::ResolveVideoInputType(
		InDescriptor.VideoPathB,
		EStereoVideoExtractionError::VideoBNotFound,
		EStereoVideoExtractionError::UnsupportedVideoFormatB
	);

	if (TypeB.HasError())
	{
		return MakeError(TypeB.GetError());
	}

	if (TypeA.GetValue() != TypeB.GetValue())
	{
		return MakeError(EStereoVideoExtractionError::VideoTypeMismatch);
	}

	for (const FString& AudioPath : InDescriptor.AudioFilePaths)
	{
		if (!FPaths::FileExists(AudioPath))
		{
			return MakeError(EStereoVideoExtractionError::AudioFileNotFound);
		}
	}

	if (!InDescriptor.CalibrationFilePath.IsEmpty() && !FPaths::FileExists(InDescriptor.CalibrationFilePath))
	{
		return MakeError(EStereoVideoExtractionError::CalibrationFileNotFound);
	}

	// Resolve calibration format

	FString ResolvedCalibrationFormat;
	if (!InDescriptor.CalibrationFilePath.IsEmpty())
	{
		ResolvedCalibrationFormat = DetectCalibrationFormat(InDescriptor.CalibrationFilePath);

		UE_LOGF(
			LogStereoVideoExtractor,
			Verbose,
			"Detected calibration format '%ls' for: %ls",
			*ResolvedCalibrationFormat, *InDescriptor.CalibrationFilePath
		);

		if (ResolvedCalibrationFormat.IsEmpty())
		{
			return MakeError(EStereoVideoExtractionError::CalibrationFormatUnrecognized);
		}
	}

	// Build metadata

	TOptional<Private::FFProbeCommand> ProbeCommand;
	if (InConfig.bUseFFprobe && !InConfig.FFmpegPath.IsEmpty())
	{
		ProbeCommand = Private::FFProbeCommand(InConfig.FFmpegPath);
	}

	FTakeMetadata Metadata;
	Metadata.Version.Major = 4;
	Metadata.Version.Minor = 1;
	Metadata.TakeNumber = (InDescriptor.TakeNumber >= 1) ? static_cast<uint32>(InDescriptor.TakeNumber) : 1u;
	Metadata.UniqueId = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensLower);
	Metadata.Device.Model = TEXT("StereoHMC");

	const Private::EVideoInputType VideoType = TypeA.GetValue();

	bool bVideoFrameRateSet = false;
	FFrameRate VideoFrameRate;

	const FString VideoPaths[] = { InDescriptor.VideoPathA, InDescriptor.VideoPathB };

	for (int32 VideoIndex = 0; VideoIndex < 2; ++VideoIndex)
	{
		const FString& VideoPath = VideoPaths[VideoIndex];
		FTakeMetadata::FVideo Video;

		// Derive name from filename stem (video file) or folder name (image sequence)
		FString Name;
		// FPaths::GetBaseFilename strips extension for files, returns leaf component for directories
		Name = FPaths::GetBaseFilename(VideoPath);
		SanitizePackagePath(Name, TEXT('_'));
		Video.Name = MoveTemp(Name);

		if (VideoType == Private::EVideoInputType::File)
		{
			Video.Format = FPaths::GetExtension(VideoPath).ToLower();
			Video.Path = VideoPath;
			Video.PathType = FTakeMetadata::FVideo::EPathType::File;

			FCaptureExtractVideoInfo::FResult ExtractResult = FCaptureExtractVideoInfo::Create(VideoPath, ProbeCommand);
			if (ExtractResult.IsValid())
			{
				FCaptureExtractVideoInfo VideoInfo = ExtractResult.StealValue();
				Video.TimecodeStart = VideoInfo.GetTimecode().ToString();
				Video.FrameRate = static_cast<float>(VideoInfo.GetFrameRate().AsDecimal());

				if (!bVideoFrameRateSet)
				{
					VideoFrameRate = VideoInfo.GetFrameRate();
					bVideoFrameRateSet = true;
				}

				switch (VideoInfo.GetVideoOrientation())
				{
				case EMediaOrientation::CW90:
					Video.Orientation = FTakeMetadata::FVideo::EOrientation::CW90;
					break;
				case EMediaOrientation::CW180:
					Video.Orientation = FTakeMetadata::FVideo::EOrientation::CW180;
					break;
				case EMediaOrientation::CW270:
					Video.Orientation = FTakeMetadata::FVideo::EOrientation::CW270;
					break;
				default:
					Video.Orientation = FTakeMetadata::FVideo::EOrientation::Original;
				}
			}
			else
			{
				UE_LOGF(
					LogStereoVideoExtractor,
					Warning,
					"Failed to extract video info from '%ls'. Timecode, frame rate, and orientation unavailable for this video.",
					*VideoPath
				);
			}

			// Use creation time of video A for the take timestamp
			if (VideoIndex == 0)
			{
				Metadata.DateTime = IFileManager::Get().GetStatData(*VideoPath).CreationTime;
			}
		}
		else // IMAGE_SEQUENCE
		{
			Video.Path = VideoPath;
			Video.PathType = FTakeMetadata::FVideo::EPathType::Folder;

			int32 FramesCount = 0;
			bool bFirstFrame = true;

			IFileManager::Get().IterateDirectoryRecursively(*VideoPath,
				[&](const TCHAR* InPath, bool bIsDirectory) -> bool
				{
					if (!bIsDirectory && IsImageExtension(FPaths::GetExtension(InPath)))
					{
						if (bFirstFrame)
						{
							bFirstFrame = false;

							FString FilePath, Filename, Ext;
							FPaths::Split(InPath, FilePath, Filename, Ext);
							Video.Format = Ext.ToLower();
							Video.Path = VideoPath;

							FImage Image;
							FImageUtils::LoadImage(InPath, Image);
							if (Image.IsImageInfoValid())
							{
								Video.FrameWidth = static_cast<uint32>(Image.SizeX);
								Video.FrameHeight = static_cast<uint32>(Image.SizeY);
							}

							if (VideoIndex == 0)
							{
								Metadata.DateTime = IFileManager::Get().GetStatData(InPath).CreationTime;
							}
						}
						FramesCount++;
					}
					return true;
				});

			Video.FramesCount = FramesCount;
		}

		Metadata.Video.Add(MoveTemp(Video));
	}

	Metadata.Slate = InDescriptor.Slate.IsEmpty() ? Metadata.Video[0].Name : InDescriptor.Slate;

	// Audio

	if (InDescriptor.AudioFilePaths.Num() > 0 && !bVideoFrameRateSet)
	{
		UE_LOGF(LogStereoVideoExtractor, Warning, "Video frame rate unavailable; audio timecode may be set incorrectly.");
	}

	for (const FString& AudioPath : InDescriptor.AudioFilePaths)
	{
		FTakeMetadata::FAudio Audio;
		Audio.Name = FPaths::GetBaseFilename(AudioPath);
		Audio.Path = AudioPath;
		Audio.Duration = 0;

		TSharedPtr<FCaptureExtractAudioTimecode> Extractor = MakeShareable(new FCaptureExtractAudioTimecode(AudioPath));
		FCaptureExtractAudioTimecode::FTimecodeInfoResult TimecodeResult = Extractor->Extract(VideoFrameRate);
		if (TimecodeResult.IsValid())
		{
			FTimecodeInfo TimecodeInfo = TimecodeResult.GetValue();
			Audio.TimecodeStart = TimecodeInfo.Timecode.ToString();
			Audio.TimecodeRate = static_cast<float>(TimecodeInfo.TimecodeRate.AsDecimal());
		}

		Metadata.Audio.Add(MoveTemp(Audio));
	}

	// Calibration

	if (!InDescriptor.CalibrationFilePath.IsEmpty())
	{
		FTakeMetadata::FCalibration Calibration;
		Calibration.Name = TEXT("calibration");
		Calibration.Path = InDescriptor.CalibrationFilePath;
		Calibration.Format = ResolvedCalibrationFormat;
		Metadata.Calibration.Add(MoveTemp(Calibration));
	}

	return MakeValue(MoveTemp(Metadata));
}

} // namespace UE::CaptureManager
