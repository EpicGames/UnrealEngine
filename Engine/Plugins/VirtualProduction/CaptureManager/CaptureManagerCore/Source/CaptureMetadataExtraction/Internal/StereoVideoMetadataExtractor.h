// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerExtractionConfig.h"
#include "CaptureManagerTakeMetadata.h"

#define UE_API CAPTUREMETADATAEXTRACTION_API

namespace UE::CaptureManager
{

enum class EStereoVideoExtractionError : uint8
{
	VideoANotFound,
	VideoBNotFound,
	UnsupportedVideoFormatA,     // video A: not a supported video file or image sequence directory
	UnsupportedVideoFormatB,     // video B: not a supported video file or image sequence directory
	VideoTypeMismatch,           // video A is a file, video B is a folder, or vice versa
	AudioFileNotFound,
	CalibrationFileNotFound,
	CalibrationFormatUnrecognized, // calibration file provided but format could not be detected
};

struct FStereoVideoDescriptor
{
	FString VideoPathA;              // video file (.mp4/.mov) or image sequence folder
	FString VideoPathB;              // same type as video A
	TArray<FString> AudioFilePaths;  // empty = no audio
	FString CalibrationFilePath;     // empty = no calibration; format auto-detected from JSON structure
	FString Slate;                   // empty = derived from video A stem/folder name
	int32 TakeNumber = 1;            // clamped to 1 if < 1
};

// Media probing (timecodes, frame rate, orientation) is best-effort.
// Probe failure is non-fatal - extraction succeeds with missing values.
UE_API TValueOrError<FTakeMetadata, EStereoVideoExtractionError>
ExtractStereoVideoMetadata(FStereoVideoDescriptor InDescriptor, const FExtractionConfig& InConfig);

} // namespace UE::CaptureManager

#undef UE_API
