// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeArchiveMetadataExtractor.h"

#include "LiveLinkFaceMetadata.h"
#include "StereoCameraMetadataParseUtils.h"

#include "Utils/CaptureExtractTimecode.h"
#include "Utils/ParseTakeUtils.h"
#include "Misc/Paths.h"

DEFINE_LOG_CATEGORY_STATIC(LogTakeArchiveExtractor, Log, All);

namespace UE::CaptureManager
{
static FString ErrorOriginToString(FTakeMetadataParserError::EOrigin InOrigin)
{
	switch (InOrigin)
	{
	case FTakeMetadataParserError::Reader:
		return TEXT("Reader");
	case FTakeMetadataParserError::Validator:
		return TEXT("Validator");
	case FTakeMetadataParserError::Parser:
	default:
		return TEXT("Parser");
	}
}

TValueOrError<FTakeMetadata, ETakeArchiveExtractionError> ExtractTakeArchiveMetadata(const FString& InMetadataFilePath, const FExtractionConfig& InConfig)
{
	using namespace Private;

	const FString MetadataFilePath = FPaths::ConvertRelativePathToFull(InMetadataFilePath);

	if (!FPaths::FileExists(MetadataFilePath))
	{
		UE_LOGF(LogTakeArchiveExtractor, Error, "Metadata file does not exist: %ls", *MetadataFilePath);
		return MakeError(ETakeArchiveExtractionError::MetadataFileNotFound);
	}

	FTakeMetadataParser TakeMetadataParser;
	TValueOrError<FTakeMetadata, FTakeMetadataParserError> TakeMetadataResult = TakeMetadataParser.Parse(MetadataFilePath);
	const FString CurrentDirectory = FPaths::GetPath(MetadataFilePath);

	if (TakeMetadataResult.HasError())
	{
		const FTakeMetadataParserError ParserError = TakeMetadataResult.StealError();
		const FString OriginName = UE::CaptureManager::ErrorOriginToString(ParserError.Origin);

		const FString Message = FString::Format(
			TEXT("Unable to parse take metadata file - {0} (Error origin: {1}): {2}"),
			{
				MetadataFilePath,
				OriginName,
				ParserError.Message.ToString()
			}
		);

		UE_LOGF(LogTakeArchiveExtractor, Warning, "%ls", *Message);
		UE_LOGF(LogTakeArchiveExtractor, Display, "Checking backwards compatible take metadata formats");

		TArray<FText> ValidationFailures;

		UE_LOGF(LogTakeArchiveExtractor, Display, "Checking directory (%ls) for legacy LiveLink take metadata format", *CurrentDirectory);
		TOptional<FTakeMetadata> ParseOldMetadataResult = LiveLinkMetadata::ParseOldLiveLinkTakeMetadata(CurrentDirectory, ValidationFailures);

		if (ParseOldMetadataResult.IsSet())
		{
			return MakeValue(MoveTemp(ParseOldMetadataResult.GetValue()));
		}

		UE_LOGF(LogTakeArchiveExtractor, Display, "Checking directory (%ls) for legacy StereoCamera take metadata format", *CurrentDirectory);
		ParseOldMetadataResult = StereoCameraMetadata::ParseOldStereoCameraMetadata(CurrentDirectory, ValidationFailures);

		if (ParseOldMetadataResult.IsSet())
		{
			return MakeValue(MoveTemp(ParseOldMetadataResult.GetValue()));
		}

		UE_LOGF(LogTakeArchiveExtractor, Error, "Failed to parse take metadata file: %ls", *CurrentDirectory);
		return MakeError(ETakeArchiveExtractionError::MetadataFormatNotRecognized);
	}

	FTakeMetadata& TakeMetadata = TakeMetadataResult.GetValue();

	const TOptional<FString> ThumbnailPath = TakeMetadata.Thumbnail.GetThumbnailPath();
	if (ThumbnailPath.IsSet())
	{
		if (FPaths::IsRelative(*ThumbnailPath))
		{
			TakeMetadata.Thumbnail = FTakeThumbnailData(FPaths::ConvertRelativePathToFull(CurrentDirectory, *ThumbnailPath));
		}
	}

	TOptional<FFProbeCommand> ProbeCommandBuilder;

	if (InConfig.bUseFFprobe && !InConfig.FFmpegPath.IsEmpty())
	{
		ProbeCommandBuilder = FFProbeCommand(InConfig.FFmpegPath);
	}

	bool bVideoFrameRateSet = false;
	FFrameRate VideoFrameRate;

	for (FTakeMetadata::FVideo& Video : TakeMetadata.Video)
	{
		if (FPaths::IsRelative(*Video.Path))
		{
			Video.Path = FPaths::ConvertRelativePathToFull(CurrentDirectory, *Video.Path);
		}

		// Check if this is a file (and not an image sequence directory)
		if (FPaths::FileExists(Video.Path) && !Video.TimecodeStart.IsSet())
		{
			FCaptureExtractVideoInfo::FResult ExtractorOpt = FCaptureExtractVideoInfo::Create(Video.Path, ProbeCommandBuilder);

			if (ExtractorOpt.IsValid())
			{
				FCaptureExtractVideoInfo Extractor = ExtractorOpt.StealValue();
				Video.TimecodeStart = Extractor.GetTimecode().ToString();
				if (FMath::IsNearlyZero(Video.FrameRate))
				{
					Video.FrameRate = static_cast<float>(Extractor.GetFrameRate().AsDecimal());
				}
			}
		}

		if (!bVideoFrameRateSet)
		{
			VideoFrameRate = UE::CaptureManager::ParseFrameRate(Video.FrameRate);
			bVideoFrameRateSet = true;
		}
	}

	for (FTakeMetadata::FAudio& Audio : TakeMetadata.Audio)
	{
		if (FPaths::IsRelative(*Audio.Path))
		{
			Audio.Path = FPaths::ConvertRelativePathToFull(CurrentDirectory, *Audio.Path);
		}

		// Only probe when the .cptake file did not provide timecodes. The metadata file is authoritative.
		if (!Audio.TimecodeStart.IsSet() && !Audio.TimecodeRate.IsSet())
		{
			TSharedPtr<FCaptureExtractAudioTimecode> Extractor = MakeShareable(new FCaptureExtractAudioTimecode(Audio.Path));

			// The video frame rate will be used to calculate the timecode rate if the timecode rate cannot be extracted from the audio file
			// If the VideoFrameRate is not set, the timecode rate may be set to the audio sample rate
			FCaptureExtractAudioTimecode::FTimecodeInfoResult TimecodeInfoResult = Extractor->Extract(VideoFrameRate);

			if (TimecodeInfoResult.IsValid())
			{
				const FTimecodeInfo TimecodeInfo = TimecodeInfoResult.GetValue();

				Audio.TimecodeStart = TimecodeInfo.Timecode.ToString();
				Audio.TimecodeRate = static_cast<float>(TimecodeInfo.TimecodeRate.AsDecimal());
			}
		}
	}

	for (FTakeMetadata::FCalibration& Calibration : TakeMetadata.Calibration)
	{
		if (FPaths::IsRelative(*Calibration.Path))
		{
			Calibration.Path = FPaths::ConvertRelativePathToFull(CurrentDirectory, *Calibration.Path);
		}
	}

	return MakeValue(TakeMetadataResult.StealValue());
}

}
