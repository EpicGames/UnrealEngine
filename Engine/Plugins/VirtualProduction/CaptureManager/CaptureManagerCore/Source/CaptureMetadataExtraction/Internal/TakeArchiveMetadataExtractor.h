// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerExtractionConfig.h"
#include "CaptureManagerTakeMetadata.h"

#define UE_API CAPTUREMETADATAEXTRACTION_API

namespace UE::CaptureManager
{

enum class ETakeArchiveExtractionError : uint8
{
	MetadataFileNotFound,
	MetadataFormatNotRecognized,
};

// Media probing (timecodes, frame rate, orientation) is best-effort.
// Probe failure is non-fatal - extraction succeeds with missing values.
UE_API TValueOrError<FTakeMetadata, ETakeArchiveExtractionError> ExtractTakeArchiveMetadata(const FString& InMetadataFilePath, const FExtractionConfig& InConfig);

} // namespace UE::CaptureManager

#undef UE_API
