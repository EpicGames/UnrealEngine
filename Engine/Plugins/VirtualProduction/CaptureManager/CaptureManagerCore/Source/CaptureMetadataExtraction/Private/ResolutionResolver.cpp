// Copyright Epic Games, Inc. All Rights Reserved.

#include "ResolutionResolver.h"

#include "IImageWrapper.h"
#include "IImageWrapperModule.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

namespace UE::CaptureManager
{

TValueOrError<FIntPoint, FResolutionResolver::EAddError> FResolutionResolver::Add(const FStereoCameraTakeInfo::FCamera& InCamera)
{
	TValueOrError<FIntPoint, EAddError> Resolution = GetCameraResolution(InCamera);

	if (Resolution.HasValue())
	{
		if (CommonResolution == FIntPoint::NoneValue)
		{
			CommonResolution = Resolution.GetValue();
		}

		if (CommonResolution != FIntPoint::NoneValue)
		{
			bAllEqual &= Resolution.GetValue() == CommonResolution;
		}
		else
		{
			bAllEqual = false;
		}
	}

	return Resolution;
}

TValueOrError<FIntPoint, FResolutionResolver::EResolveError> FResolutionResolver::Resolve() const
{
	if (bAllEqual)
	{
		return MakeValue(CommonResolution);
	}
	else
	{
		return MakeError(EResolveError::Mismatched);
	}
}

TValueOrError<FIntPoint, FResolutionResolver::EAddError> FResolutionResolver::GetCameraResolution(const FStereoCameraTakeInfo::FCamera& InCamera) const
{
	if (InCamera.Resolution == FIntPoint::NoneValue)
	{
		if (FPaths::DirectoryExists(InCamera.FramesPath))
		{
			return GetResolutionFromSingleImage(InCamera.FramesPath);
		}
		else
		{
			return MakeError(EAddError::FramesPathDoesNotExist);
		}
	}
	else
	{
		// Property has already been set on the camera so just use it. 
		// 
		// This is a little weird, as it's effectively passing responsibility for determining the resolution value out of this function/class. 
		// However, this functionality has been brought across from the previous implementation and might need some more consideration to change.

		return MakeValue(InCamera.Resolution);
	}
}

TValueOrError<FIntPoint, FResolutionResolver::EAddError> FResolutionResolver::GetResolutionFromSingleImage(const FString& InDirectoryPath) const
{
	FString FirstFilePath;
	TArray<FString> SupportedExtensions = { TEXT("png"), TEXT("jpg"), TEXT("jpeg") };

	IFileManager::Get().IterateDirectory(*InDirectoryPath, [&FirstFilePath, &SupportedExtensions](const TCHAR* InFilenameOrDirectory, bool bInIsDirectory) mutable
	{
		if (!bInIsDirectory &&
			SupportedExtensions.Contains(FPaths::GetExtension(InFilenameOrDirectory)))
		{
			FString FileNamePath = InFilenameOrDirectory;
			FPaths::NormalizeFilename(FileNamePath);
			FirstFilePath = MoveTemp(FileNamePath);

			// Returning false because we only need the first file
			return false;
		}

		return true;
	});

	if (FirstFilePath.IsEmpty())
	{
		return MakeError(EAddError::NoImagesFound);
	}

	TArray<uint8> RawFileData;
	if (!FFileHelper::LoadFileToArray(RawFileData, *FirstFilePath))
	{
		return MakeError(EAddError::ImageLoadFailed);
	}

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
	EImageFormat ImageFormat = ImageWrapperModule.DetectImageFormat(RawFileData.GetData(), RawFileData.Num());
	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(ImageFormat);

	if (!ImageWrapper.IsValid() || !ImageWrapper->SetCompressed(RawFileData.GetData(), RawFileData.Num()))
	{
		return MakeError(EAddError::InvalidImageWrapper);
	}

	return MakeValue(static_cast<int>(ImageWrapper->GetWidth()), static_cast<int>(ImageWrapper->GetHeight()));
}

}
