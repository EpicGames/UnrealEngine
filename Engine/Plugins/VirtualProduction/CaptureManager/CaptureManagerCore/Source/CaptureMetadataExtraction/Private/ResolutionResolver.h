// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StereoCameraTakeMetadata.h"

#include "Containers/UnrealString.h"
#include "Math/IntPoint.h"
#include "Misc/Optional.h"

namespace UE::CaptureManager
{

/**
 * Class used to determine and validate the image resolution for a set of stereo camera data.
 *
 * Each time a camera is added to the resolver, the internal state is updated and will influence the result of the final Resolve() call. A collective
 * value for the image resolution will only be returned if all of the cameras share the same image resolution.
 */
class FResolutionResolver
{
public:
	enum class EAddError
	{
		FramesPathDoesNotExist,
		NoImagesFound,
		ImageLoadFailed,
		InvalidImageWrapper
	};

	enum class EResolveError
	{
		Mismatched
	};

	TValueOrError<FIntPoint, EAddError> Add(const FStereoCameraTakeInfo::FCamera& Camera);
	TValueOrError<FIntPoint, EResolveError> Resolve() const;

private:
	TValueOrError<FIntPoint, EAddError> GetCameraResolution(const FStereoCameraTakeInfo::FCamera& Camera) const;
	TValueOrError<FIntPoint, EAddError> GetResolutionFromSingleImage(const FString& InDirectoryPath) const;

	FIntPoint CommonResolution = FIntPoint::NoneValue;
	bool bAllEqual = true;
};

}