// Copyright Epic Games, Inc. All Rights Reserved.

#include "StereoCameraMetadataParseUtils.h"

#include "StereoCameraTakeMetadata.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonSerializer.h"

#include "Modules/ModuleManager.h"
#include "IImageWrapperModule.h"

#define LOCTEXT_NAMESPACE "StereoCameraTakeMetadata"

TOptional<FText> TakeDurationExceedsLimit(const float InDurationInSeconds)
{
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("au.SoundWaveImportLengthLimitInSeconds"));
	if (!CVar)
	{
		return {};
	}

	static constexpr float Unlimited = -1.f;
	float Limit = CVar->GetFloat();

	if (FMath::IsNearlyEqual(Limit, Unlimited) && FMath::IsNegativeOrNegativeZero(Limit - InDurationInSeconds))
	{
		return {};
	}

	const FText Message = LOCTEXT("TakeDurationExceedsLimit", "Take duration ({0} seconds) exceeds allowed limit ({1} seconds).");

	FNumberFormattingOptions Options;
	Options.MaximumFractionalDigits = 2;
	Options.MinimumFractionalDigits = 2;

	return FText::Format(Message, FText::AsNumber(InDurationInSeconds, &Options), FText::AsNumber(Limit, &Options));
}

FString DetermineImageFormat(const FString& InImageFolderPath)
{
	FString Extension;
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));

	IFileManager::Get().IterateDirectory(*InImageFolderPath, [&Extension, &ImageWrapperModule](const TCHAR* InFileName, bool bIsDirectory)
		{
			if (bIsDirectory)
			{
				return true;
			}

			Extension = FPaths::GetExtension(InFileName);
			EImageFormat Format = ImageWrapperModule.GetImageFormatFromExtension(*Extension);

			if (Format != EImageFormat::Invalid)
			{
				return true;
			}

			return false; // Returns false because we are looking only for the first image in the given directory
		});

	return Extension;
}

FTakeMetadata ConvertOldToNewTakeMetadata(const FString& InTakeFolder, const FStereoCameraTakeInfo& InStereoCameraInfo)
{
	FString CalibrationPath = InTakeFolder / TEXT("calib.json");

	FTakeMetadata::FCalibration Calibration;
	Calibration.Name = TEXT("undefined");
	Calibration.Format = TEXT("opencv");
	Calibration.Path = InStereoCameraInfo.CalibrationFilePath;

	FTakeMetadata::FDevice::FPlatform Platform;
	Platform.Name = TEXT("Windows");

	FTakeMetadata Metadata;
	Metadata.Device.Platform = MoveTemp(Platform);

	Metadata.Calibration.Push(MoveTemp(Calibration));
	Metadata.Device.Type = InStereoCameraInfo.DeviceInfo.Type;
	Metadata.Device.Model = InStereoCameraInfo.DeviceInfo.Model;
	Metadata.Device.Name = InStereoCameraInfo.DeviceInfo.Id;

	Metadata.UniqueId = InStereoCameraInfo.Id;
	Metadata.Slate = InStereoCameraInfo.Slate;
	Metadata.TakeNumber = InStereoCameraInfo.Take;
	Metadata.DateTime = InStereoCameraInfo.Date;

	Metadata.Thumbnail = FTakeThumbnailData(InStereoCameraInfo.ThumbnailPath);

	Metadata.Version.Major = 3;
	Metadata.Version.Minor = 0;

	TOptional<float> AudioDuration = 0.0f;

	TArray<FTakeMetadata::FVideo> VideoArray;
	for (const TPair<FString, FStereoCameraTakeInfo::FCamera>& Camera : InStereoCameraInfo.CameraMap)
	{
		FTakeMetadata::FVideo Video;
		Video.Name = Camera.Value.UserId;
		Video.Path = Camera.Value.FramesPath;
		Video.PathType = FTakeMetadata::FVideo::EPathType::Folder;
		Video.Format = DetermineImageFormat(Camera.Value.FramesPath);
		Video.Orientation = FTakeMetadata::FVideo::EOrientation::Original;
		Video.FramesCount = Camera.Value.FrameRange.Value - Camera.Value.FrameRange.Key + 1;

		Video.FrameHeight = Camera.Value.Resolution.Y;
		Video.FrameWidth = Camera.Value.Resolution.X;

		Video.FrameRate = Camera.Value.FrameRate;

		Video.TimecodeStart = Camera.Value.StartTimecode;

		for (const FFrameRange& FrameRange : Camera.Value.CaptureExcludedFrames)
		{
			TArray<uint32> DroppedFrames;
			for (int32 Frame = FrameRange.StartFrame; Frame <= FrameRange.EndFrame; ++Frame)
			{
				DroppedFrames.Add(Frame);
			}
			Video.DroppedFrames = MoveTemp(DroppedFrames);
		}

		if (Video.FramesCount.IsSet())
		{
			// We do not know the audio duration and so we must estimate it from the video. This apporach has been deemed 
			// acceptable for the moment, based on how this duration value gets used.
			// If the video duration has not been set, then we cannot estimate the audio duration
			const float VideoDuration = static_cast<float>(Video.FramesCount.GetValue()) / Video.FrameRate;
			AudioDuration = FMath::Max(0, VideoDuration);
		}

		VideoArray.Add(MoveTemp(Video));
	}

	Metadata.Video = MoveTemp(VideoArray);

	TArray<FTakeMetadata::FAudio> AudioArray;
	AudioArray.Reserve(InStereoCameraInfo.AudioArray.Num());

	for (const FStereoCameraTakeInfo::FAudio& AudioEntry : InStereoCameraInfo.AudioArray)
	{
		FTakeMetadata::FAudio Audio;
		Audio.Duration = AudioDuration;
		Audio.Path = AudioEntry.StreamPath;
		Audio.TimecodeStart = AudioEntry.StartTimecode;
		Audio.TimecodeRate = AudioEntry.TimecodeRate;

		AudioArray.Add(MoveTemp(Audio));
	}

	Metadata.Audio = MoveTemp(AudioArray);

	return Metadata;
}

namespace UE::CaptureManager::StereoCameraMetadata
{

TOptional<FTakeMetadata> ParseOldStereoCameraMetadata(const FString& InTakeFolder, TArray<FText>& OutValidationError)
{
	const FString MetadataFilePath = InTakeFolder / TEXT("take.json");
	TOptional<FStereoCameraTakeInfo> StereoCameraInfoResult = FStereoCameraSystemTakeParser::ParseTakeMetadataFile(MetadataFilePath);

	if (!StereoCameraInfoResult.IsSet())
	{
		OutValidationError.Add(LOCTEXT("ParseOldStereoCameraTakeMetadata_ParseTakeInfoFailed", "Failed to parse old take json file"));
		return {};
	}

	FStereoCameraTakeInfo StereoCameraInfo = StereoCameraInfoResult.GetValue();

	TArray<FText> Issues = FStereoCameraSystemTakeParser::CheckStereoCameraTakeInfo(InTakeFolder, StereoCameraInfo, 2, TEXT("HMC"));

	if (!Issues.IsEmpty())
	{
		OutValidationError.Append(Issues);
	}

	TArray<FText> ResolveIssues = FStereoCameraSystemTakeParser::ResolveResolution(StereoCameraInfo);
	if (!ResolveIssues.IsEmpty())
	{
		OutValidationError.Append(ResolveIssues);
	}

	FStereoCameraTakeInfo::FCamera CameraInfo = StereoCameraInfo.CameraMap.CreateConstIterator()->Value;
	uint32 NumOfTakes = (CameraInfo.FrameRange.Value - CameraInfo.FrameRange.Key) + 1;

	TOptional<FText> DurationLimit = TakeDurationExceedsLimit(static_cast<float>(NumOfTakes) / CameraInfo.FrameRate);
	if (DurationLimit.IsSet())
	{
		OutValidationError.Add(DurationLimit.GetValue());
	}

	return ConvertOldToNewTakeMetadata(InTakeFolder, StereoCameraInfo);
}

}

#undef LOCTEXT_NAMESPACE
