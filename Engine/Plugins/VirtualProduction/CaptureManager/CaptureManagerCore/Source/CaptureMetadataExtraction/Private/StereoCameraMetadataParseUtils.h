// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerTakeMetadata.h"

#define UE_API CAPTUREMETADATAEXTRACTION_API

namespace UE::CaptureManager::StereoCameraMetadata
{

UE_API TOptional<FTakeMetadata> ParseOldStereoCameraMetadata(const FString& InTakeFolder, TArray<FText>& OutValidationError);

}

#undef UE_API
