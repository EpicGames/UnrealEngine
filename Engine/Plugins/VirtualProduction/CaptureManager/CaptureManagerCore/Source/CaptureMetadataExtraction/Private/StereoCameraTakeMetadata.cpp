// Copyright Epic Games, Inc. All Rights Reserved.

#include "StereoCameraTakeMetadata.h"

#include "ResolutionResolver.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonSerializer.h"

DEFINE_LOG_CATEGORY_STATIC(LogStereoCameraMetadata, Log, All);

#define LOCTEXT_NAMESPACE "StereoCameraTakeMetadata"

#define CHECK_RESULT_AND_RETURN(Fun) if (bool Result = Fun; !Result) {return TOptional<FStereoCameraTakeInfo>{};}

static const FText FailedToDetermineCameraResolution = LOCTEXT("FailedToDetermineCameraResolution", "Failed to determine camera resolution");
static const FText NoImagesFound = LOCTEXT("NoImagesFound", "No images found");
static const FText ImageLoadFailed = LOCTEXT("ImageLoadFailed", "Failed to load an image");
static const FText CameraResolutionMismatch = LOCTEXT("CameraResolutionMismatch", "Camera resolutions do not match");

static FString ConvertPathToAbsolute(const FString& InBasePath, const FString& InPath)
{
	// Convert paths to absolute
	if (FPaths::IsRelative(InPath))
	{
		const FString Directory = FPaths::GetPath(InBasePath);
		return FPaths::Combine(Directory, InPath);
	}

	return InPath;
}

static FString WrapLogMessage(const FString InMessage, const FStereoCameraTakeInfo& InTakeInfo)
{
	return FString::Format(TEXT("{0} ({1} #{2})"), { InMessage, InTakeInfo.Slate, InTakeInfo.Take });
}

static FString WrapLogMessage(const FString InMessage, const FTakeMetadata& InTakeInfo)
{
	return FString::Format(TEXT("{0} ({1} #{2})"), { InMessage, InTakeInfo.Slate, InTakeInfo.TakeNumber });
}

template<typename FTakeInfoType>
static void ReportIssue(FText InMessage, const FTakeInfoType& InTakeInfo, TArray<FText>& OutIssues)
{
	// Log and register as an issue, it's important to do both so the user has a reference log that can be sent, rather than just screenshots.
	UE_LOGF(LogStereoCameraMetadata, Warning, "%ls", *WrapLogMessage(InMessage.ToString(), InTakeInfo));
	OutIssues.Emplace(MoveTemp(InMessage));
}

static void ReportResolutionAddError(
	const UE::CaptureManager::FResolutionResolver::EAddError AddError,
	const FString& CameraId,
	const FStereoCameraTakeInfo& InStereoCameraTakeInfo,
	TArray<FText>& OutIssues
)
{
	using namespace UE::CaptureManager;

	FText Message;

	switch (AddError)
	{
		case FResolutionResolver::EAddError::FramesPathDoesNotExist:
		{
			// Manually format the message in this case, as we need a little more control
			const FStereoCameraTakeInfo::FCamera* Camera = InStereoCameraTakeInfo.CameraMap.Find(CameraId);

			if (Camera)
			{
				// We can provide more information in this case and we want to put the "for camera" part before the path, which can be very long
				ReportIssue(
					FText::Format(
						LOCTEXT("FramesPathDoesNotExistForCamera", "Frames path does not exist for camera: {0} ({1})"),
						FText::FromString(CameraId),
						FText::FromString(Camera->FramesPath)
					),
					InStereoCameraTakeInfo,
					OutIssues
				);
			}
			else
			{
				ReportIssue(
					FText::Format(
						LOCTEXT("FramesPathDoesNotExist", "Frames path does not exist: {0}"),
						FText::FromString(CameraId)
					),
					InStereoCameraTakeInfo,
					OutIssues
				);
			}
			break;
		}

		case FResolutionResolver::EAddError::NoImagesFound:
			Message = NoImagesFound;
			break;

		case FResolutionResolver::EAddError::ImageLoadFailed:
			Message = ImageLoadFailed;
			break;

		case FResolutionResolver::EAddError::InvalidImageWrapper:
			// Log detailed message but don't display that to the user (they don't know what the image wrapper is)
			UE_LOGF(LogStereoCameraMetadata, Warning, "%ls",
				   *WrapLogMessage(FString::Format(TEXT("Image wrapper is in an invalid state for camera: {0}"), { CameraId, }), InStereoCameraTakeInfo)
			);
			Message = FailedToDetermineCameraResolution;
			break;

		default:
			Message = FailedToDetermineCameraResolution;
			break;
	}

	if (!Message.IsEmpty())
	{
		FText IssueMessage = FText::Format(LOCTEXT("ResolutionAddError", "{0} for camera: {1}"), Message, FText::FromString(CameraId));
		ReportIssue(MoveTemp(IssueMessage), InStereoCameraTakeInfo, OutIssues);
	}
}

static void ReportResolutionResolveError(
	const UE::CaptureManager::FResolutionResolver::EResolveError ResolutionError,
	const FStereoCameraTakeInfo& InStereoCameraTakeInfo,
	TArray<FText>& OutIssues
)
{
	using namespace UE::CaptureManager;

	FText Message;

	switch (ResolutionError)
	{
		case FResolutionResolver::EResolveError::Mismatched:
			Message = CameraResolutionMismatch;
			break;
		default:
			Message = FailedToDetermineCameraResolution;
			break;
	}

	ReportIssue(MoveTemp(Message), InStereoCameraTakeInfo, OutIssues);
}

static TArray<TSharedPtr<FJsonValue>> ParseJsonArrayFromFile(const FString& InFilePath)
{
	FString JsonStringBuffer;
	TArray<TSharedPtr<FJsonValue>> Result;

	if (FFileHelper::LoadFileToString(JsonStringBuffer, *InFilePath) &&
		FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(JsonStringBuffer), Result))
	{
		return Result;
	}

	return TArray<TSharedPtr<FJsonValue>>();
}

static TSharedPtr<FJsonObject> ParseJsonObjectFromFile(const FString& InFilePath)
{
	FString JsonStringBuffer;

	if (!IFileManager::Get().FileExists(*InFilePath))
	{
		UE_LOGF(LogStereoCameraMetadata, Error, "File not found: %ls", *InFilePath);
		return nullptr;
	}

	if (!FFileHelper::LoadFileToString(JsonStringBuffer, *InFilePath))
	{
		UE_LOGF(LogStereoCameraMetadata, Error, "Failed to load file (check permissions): %ls", *InFilePath);
		return nullptr;
	}

	TSharedPtr<FJsonObject> Result;

	if (!FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(JsonStringBuffer), Result))
	{
		UE_LOGF(LogStereoCameraMetadata, Error, "Failed to load json file (check for syntax errors): %ls", *InFilePath);
		return nullptr;
	}

	return Result;
}

static void CheckCameraIds(const FTakeMetadata& InStereoCameraTakeInfo, const TMap<FString, FStereoCameraInfo>& OutTakeCameras, TArray<FText>& OutIssues)
{
	for (const FTakeMetadata::FVideo& ExpectedCamera : InStereoCameraTakeInfo.Video)
	{
		const FString& ExpectedId = ExpectedCamera.Name;

		if (!OutTakeCameras.Contains(ExpectedId))
		{
			ReportIssue(
				FText::Format(
					LOCTEXT("CameraUserIdMismatch", "Camera ID present in take metadata but not present in the calibration: {0}"),
					FText::FromString(ExpectedId)
				),
				InStereoCameraTakeInfo,
				OutIssues
			);
		}
	}
}

static void CheckAudio(const FStereoCameraTakeInfo& InStereoCameraTakeInfo, TArray<FText>& OutIssues)
{
	for (const FStereoCameraTakeInfo::FAudio& Audio : InStereoCameraTakeInfo.AudioArray)
	{
		if (!FPaths::FileExists(Audio.StreamPath))
		{
			ReportIssue(
				FText::Format(LOCTEXT("MissingAudio", "Audio file not found: {0}"), FText::FromString(Audio.StreamPath)),
				InStereoCameraTakeInfo,
				OutIssues
			);
		}
	}
}

static bool ParseCalibrationFile(const FString& InFileName, TMap<FString, FStereoCameraInfo>& OutCameras)
{
	const TArray<TSharedPtr<FJsonValue>> CalibrationJson = ParseJsonArrayFromFile(InFileName);

	if (CalibrationJson.IsEmpty())
	{
		UE_LOGF(LogStereoCameraMetadata, Warning, "Calibration file is empty: %ls", *InFileName);
		return false;
	}

	for (const TSharedPtr<FJsonValue>& Value : CalibrationJson)
	{
		const TSharedPtr<FJsonObject>* MaybeObject = nullptr;
		if (Value->TryGetObject(MaybeObject))
		{
			const TSharedRef<FJsonObject> Object = MaybeObject->ToSharedRef();

			const TSharedPtr<FJsonObject>* MaybeMetaData = nullptr;
			if (Object->TryGetObjectField(TEXT("metadata"), MaybeMetaData))
			{
				const TSharedRef<FJsonObject> MetaData = MaybeMetaData->ToSharedRef();

				FStereoCameraInfo CameraInfo;
				MetaData->TryGetStringField(TEXT("camera"), CameraInfo.Name);

				if (!CameraInfo.Name.IsEmpty())
				{
					bool bOK = true;
					CameraInfo.Calibration.CameraId = CameraInfo.Name;
					bOK &= Object->TryGetNumberField(TEXT("image_size_x"), CameraInfo.Calibration.ImageSize.X);
					bOK &= Object->TryGetNumberField(TEXT("image_size_y"), CameraInfo.Calibration.ImageSize.Y);
					bOK &= Object->TryGetNumberField(TEXT("fx"), CameraInfo.Calibration.FocalLength.X);
					bOK &= Object->TryGetNumberField(TEXT("fy"), CameraInfo.Calibration.FocalLength.Y);
					bOK &= Object->TryGetNumberField(TEXT("cx"), CameraInfo.Calibration.PrincipalPoint.X);
					bOK &= Object->TryGetNumberField(TEXT("cy"), CameraInfo.Calibration.PrincipalPoint.Y);
					bOK &= Object->TryGetNumberField(TEXT("k1"), CameraInfo.Calibration.K1);
					bOK &= Object->TryGetNumberField(TEXT("k2"), CameraInfo.Calibration.K2);
					bOK &= Object->TryGetNumberField(TEXT("k3"), CameraInfo.Calibration.K3);
					bOK &= Object->TryGetNumberField(TEXT("p1"), CameraInfo.Calibration.P1);
					bOK &= Object->TryGetNumberField(TEXT("p2"), CameraInfo.Calibration.P2);

					const TArray<TSharedPtr<FJsonValue>>* Transform = nullptr;
					bOK &= Object->TryGetArrayField(TEXT("transform"), Transform);
					bOK &= Transform && Transform->Num() == 16;

					if (bOK)
					{
						for (int32 I = 0; I < 4; ++I)
						{
							for (int32 J = 0; J < 4; ++J)
							{
								CameraInfo.Calibration.Transform.M[J][I] = (*Transform)[I * 4 + J]->AsNumber();
							}
						}

						FString Name = CameraInfo.Name;
						OutCameras.Add(MoveTemp(Name), MoveTemp(CameraInfo));
					}
				}
			}
		}
	}

	return true;
}

static void LoadCalibration(const FTakeMetadata& InStereoCameraTakeInfo, TMap<FString, FStereoCameraInfo>& OutTakeCameras, TArray<FText>& OutIssues)
{
	if (InStereoCameraTakeInfo.Calibration.IsEmpty())
	{
		ReportIssue(
			LOCTEXT("CalibrationNotFound", "Calibration object not set"),
			InStereoCameraTakeInfo,
			OutIssues
		);

		return;
	}

	FTakeMetadata::FCalibration Calibration = InStereoCameraTakeInfo.Calibration[0];

	if (FPaths::FileExists(Calibration.Path))
	{
		if (!ParseCalibrationFile(Calibration.Path, OutTakeCameras))
		{
			ReportIssue(
				FText::Format(
					LOCTEXT("CalibrationLoadFailed", "Failed to load calibration: {0}"),
					FText::FromString(Calibration.Path)
				),
				InStereoCameraTakeInfo,
				OutIssues
			);
		}
	}
	else
	{
		ReportIssue(
			FText::Format(LOCTEXT("CalibrationFileNotFound", "Calibration file not found: {0}"), FText::FromString(Calibration.Path)),
			InStereoCameraTakeInfo,
			OutIssues
		);
	}
}

static void CheckForEmptyFramesPaths(const FStereoCameraTakeInfo& InStereoCameraTakeInfo, TArray<FText>& OutIssues)
{
	// Work-around for lack of enforcement in take parsing code (which allows this to occur)

	for (const TPair<FString, FStereoCameraTakeInfo::FCamera>& Camera : InStereoCameraTakeInfo.CameraMap)
	{
		if (Camera.Value.FramesPath.IsEmpty())
		{
			ReportIssue(
				FText::Format(LOCTEXT("EmptyFramesPath", "Empty frames path for camera: {0}"), FText::FromString(Camera.Value.UserId)),
				InStereoCameraTakeInfo,
				OutIssues
			);
		}
	}
}

static void CheckForNonANSICharacters(const FStereoCameraTakeInfo& InStereoCameraTakeInfo, TArray<FText>& OutIssues)
{
	TArray<FString> NonANSIContainingProperties;

	if (!FCString::IsPureAnsi(*InStereoCameraTakeInfo.Id))
	{
		NonANSIContainingProperties.Add(TEXT("Id"));
	}

	if (!FCString::IsPureAnsi(*InStereoCameraTakeInfo.Slate))
	{
		NonANSIContainingProperties.Add(TEXT("Slate"));
	}

	if (!FCString::IsPureAnsi(*InStereoCameraTakeInfo.ThumbnailPath))
	{
		NonANSIContainingProperties.Add(TEXT("ThumbnailPath"));
	}

	if (!FCString::IsPureAnsi(*InStereoCameraTakeInfo.CalibrationFilePath))
	{
		NonANSIContainingProperties.Add(TEXT("CalibrationFilePath"));
	}

	for (const TPair<FString, FStereoCameraTakeInfo::FCamera>& Camera : InStereoCameraTakeInfo.CameraMap)
	{
		const FString& CameraUserId = Camera.Value.UserId;
		FString CameraPropertyPrefix = FString::Format(TEXT("Camera({0})."), { CameraUserId });
		if (!FCString::IsPureAnsi(*CameraUserId))
		{
			NonANSIContainingProperties.Add(CameraPropertyPrefix + TEXT("UserId"));
		}

		if (!FCString::IsPureAnsi(*Camera.Value.FramesPath))
		{
			NonANSIContainingProperties.Add(CameraPropertyPrefix + TEXT("FramesPath"));
		}
	}

	for (const FStereoCameraTakeInfo::FAudio& Audio : InStereoCameraTakeInfo.AudioArray)
	{
		const FString& AudioUserId = Audio.UserId;
		FString AudioPropertyPrefix = FString::Format(TEXT("Audio({0})."), { AudioUserId });
		if (!FCString::IsPureAnsi(*AudioUserId))
		{
			NonANSIContainingProperties.Add(AudioPropertyPrefix + TEXT("UserId"));
		}

		if (!FCString::IsPureAnsi(*Audio.StreamPath))
		{
			NonANSIContainingProperties.Add(AudioPropertyPrefix + TEXT("StreamPath"));
		}
	}

	for (const FString& NonANSIContainingProperty : NonANSIContainingProperties)
	{
		const FText Message = FText::Format(LOCTEXT("UnsupportedCharactersWithinTakeInfo", "Take '{0}' contains unsupported text characters"), FText::FromString(NonANSIContainingProperty));
		OutIssues.Add(Message);
		UE_LOGF(LogStereoCameraMetadata, Warning, "InStereoCameraTakeInfo property '%ls' contains unsupported non-ansi text characters.", *NonANSIContainingProperty);
	}
}

FString FStereoCameraTakeInfo::GetName() const
{
	return FString::Format(TEXT("{0}_{1}"), { Slate, Take });
}

FString FStereoCameraTakeInfo::GetFolderName() const
{
	return FPaths::GetPathLeaf(FPaths::GetPath(TakeJsonFilePath));
}

TOptional<FStereoCameraTakeInfo> FStereoCameraSystemTakeParser::ParseTakeMetadataFile(const FString& InFileName)
{
	FStereoCameraTakeInfo CubicTakeInfo;
	const FString CurrentFileName = FPaths::GetCleanFilename(InFileName);

	if (CurrentFileName == TEXT("take.json"))
	{
		if (TSharedPtr<FJsonObject> TakeMetadataJson = ParseJsonObjectFromFile(InFileName))
		{
			CHECK_RESULT_AND_RETURN(TakeMetadataJson->TryGetNumberField(TEXT("Version"), CubicTakeInfo.Version));
			CHECK_RESULT_AND_RETURN(TakeMetadataJson->TryGetStringField(TEXT("Id"), CubicTakeInfo.Id));
			CHECK_RESULT_AND_RETURN(TakeMetadataJson->TryGetNumberField(TEXT("Take"), CubicTakeInfo.Take));
			CHECK_RESULT_AND_RETURN(TakeMetadataJson->TryGetStringField(TEXT("Slate"), CubicTakeInfo.Slate));
			// Optional
			FString Thumbnail;
			if (TakeMetadataJson->TryGetStringField(TEXT("Thumbnail"), Thumbnail))
			{
				CubicTakeInfo.ThumbnailPath = ConvertPathToAbsolute(InFileName, Thumbnail);
			}

			FString Date;
			CHECK_RESULT_AND_RETURN(TakeMetadataJson->TryGetStringField(TEXT("LocalDateTime"), Date));

			if (!FDateTime::ParseIso8601(*Date, CubicTakeInfo.Date))
			{
				UE_LOGF(LogStereoCameraMetadata, Warning, "Failed to parse date string '%ls' as an ISO8601 date", *Date);
			}

			FString CalibrationFilePath;
			CHECK_RESULT_AND_RETURN(TakeMetadataJson->TryGetStringField(TEXT("CalibrationInfo"), CalibrationFilePath));
			CubicTakeInfo.CalibrationFilePath = ConvertPathToAbsolute(InFileName, CalibrationFilePath);

			const TSharedPtr<FJsonObject>* DeviceInfoObject;
			CHECK_RESULT_AND_RETURN(TakeMetadataJson->TryGetObjectField(TEXT("DeviceInfo"), DeviceInfoObject));

			CHECK_RESULT_AND_RETURN((*DeviceInfoObject)->TryGetStringField(TEXT("Model"), CubicTakeInfo.DeviceInfo.Model));
			CHECK_RESULT_AND_RETURN((*DeviceInfoObject)->TryGetStringField(TEXT("Type"), CubicTakeInfo.DeviceInfo.Type));
			CHECK_RESULT_AND_RETURN((*DeviceInfoObject)->TryGetStringField(TEXT("Id"), CubicTakeInfo.DeviceInfo.Id));

			const TArray<TSharedPtr<FJsonValue>>* CamerasJson;
			CHECK_RESULT_AND_RETURN(TakeMetadataJson->TryGetArrayField(TEXT("Cameras"), CamerasJson));

			// Cameras
			for (const TSharedPtr<FJsonValue>& CameraJson : *CamerasJson)
			{
				FStereoCameraTakeInfo::FCamera Camera;
				const TSharedPtr<FJsonObject>& CameraObject = CameraJson->AsObject();

				CHECK_RESULT_AND_RETURN(CameraObject->TryGetStringField(TEXT("UserID"), Camera.UserId));

				// Optional
				const TArray<TSharedPtr<FJsonValue>>* FrameRangeJson = nullptr;
				if (CameraObject->TryGetArrayField(TEXT("FrameRange"), FrameRangeJson))
				{
					CHECK_RESULT_AND_RETURN(((*FrameRangeJson).Num() == 2));
					Camera.FrameRange.Key = (*FrameRangeJson)[0]->AsNumber();
					Camera.FrameRange.Value = (*FrameRangeJson)[1]->AsNumber();
				}

				CHECK_RESULT_AND_RETURN(CameraObject->TryGetNumberField(TEXT("FrameRate"), Camera.FrameRate));

				FString FramesPath;
				CHECK_RESULT_AND_RETURN(CameraObject->TryGetStringField(TEXT("FramesPath"), FramesPath));

				if (!FramesPath.IsEmpty())
				{
					Camera.FramesPath = ConvertPathToAbsolute(InFileName, FramesPath);
				}

				// Optional
				CameraObject->TryGetStringField(TEXT("StartTimecode"), Camera.StartTimecode);

				// Optional
				const TArray<TSharedPtr<FJsonValue>>* FramesDroppedJson;
				if (CameraObject->TryGetArrayField(TEXT("FramesDropped"), FramesDroppedJson))
				{
					for (const TSharedPtr<FJsonValue>& DroppedFrameJson : *FramesDroppedJson)
					{
						FString DroppedFrame;
						CHECK_RESULT_AND_RETURN(DroppedFrameJson->TryGetString(DroppedFrame));

						TArray<FString> FrameTokens;
						DroppedFrame.ParseIntoArray(FrameTokens, TEXT("-"));
						CHECK_RESULT_AND_RETURN(FrameTokens.Num() > 0 && FrameTokens.Num() <= 2);

						FFrameRange FrameRange;
						FrameRange.StartFrame = FCString::Atoi(*FrameTokens[0]);
						FrameRange.EndFrame = FrameTokens.Num() == 2 ? FCString::Atoi(*FrameTokens[1]) : FrameRange.StartFrame;

						CHECK_RESULT_AND_RETURN(FrameRange.StartFrame >= 0);
						CHECK_RESULT_AND_RETURN(FrameRange.EndFrame >= 0);
						CHECK_RESULT_AND_RETURN(FrameRange.EndFrame >= FrameRange.StartFrame);

						if (FrameRangeJson)
						{
							// Dropped frames are specified wrt frame range start
							FrameRange.StartFrame -= Camera.FrameRange.Key;
							FrameRange.EndFrame -= Camera.FrameRange.Key;
						}

						if (FrameRange.StartFrame >= 0)
						{
							Camera.CaptureExcludedFrames.Add(FrameRange);
						}
					}
				}

				FString Name = Camera.UserId;
				CubicTakeInfo.CameraMap.Add(MoveTemp(Name), MoveTemp(Camera));
			}

			const TArray<TSharedPtr<FJsonValue>>* AudioArrayJson;
			// Optional
			bool HasAudio = TakeMetadataJson->TryGetArrayField(TEXT("Audio"), AudioArrayJson);
			if (HasAudio)
			{
				// Audio
				for (const TSharedPtr<FJsonValue>& AudioJson : *AudioArrayJson)
				{
					FStereoCameraTakeInfo::FAudio Audio;
					const TSharedPtr<FJsonObject>& AudioObject = AudioJson->AsObject();

					CHECK_RESULT_AND_RETURN(AudioObject->TryGetStringField(TEXT("UserID"), Audio.UserId));

					FString StreamPath;
					CHECK_RESULT_AND_RETURN(AudioObject->TryGetStringField(TEXT("StreamPath"), StreamPath));
					Audio.StreamPath = ConvertPathToAbsolute(InFileName, StreamPath);

					// Optional
					AudioObject->TryGetNumberField(TEXT("TimecodeRate"), Audio.TimecodeRate);
					AudioObject->TryGetStringField(TEXT("StartTimecode"), Audio.StartTimecode);

					CubicTakeInfo.AudioArray.Add(MoveTemp(Audio));
				}
			}

			CubicTakeInfo.TakeJsonFilePath = InFileName;
			return MoveTemp(CubicTakeInfo);
		}
	}

	return {};
}

TArray<FText> FStereoCameraSystemTakeParser::CheckStereoCameraTakeInfo(
	const FString& InFilePath,
	FStereoCameraTakeInfo& InStereoCameraTakeInfo,
	const int32 InExpectedCameraCount,
	const FString& InDeviceType)
{
	TArray<FText> Issues;

	if (!FCString::IsPureAnsi(*InFilePath))
	{
		ReportIssue(
			LOCTEXT("UnsupportedCharactersWithinFilePath", "Take File Path contains unsupported text characters"),
			InStereoCameraTakeInfo,
			Issues
		);
	}

	if (InStereoCameraTakeInfo.DeviceInfo.Type != InDeviceType)
	{
		ReportIssue(
			FText::Format(
				LOCTEXT("UnexpectedDeviceType", "Unexpected device type: {0} instead of {1}"),
				FText::FromString(InStereoCameraTakeInfo.DeviceInfo.Type),
				FText::FromString(InDeviceType)
			),
			InStereoCameraTakeInfo,
			Issues
		);
	}

	if (InStereoCameraTakeInfo.CameraMap.Num() != InExpectedCameraCount)
	{
		ReportIssue(
			FText::Format(
				LOCTEXT("UnexpectedNumberOfCameras", "Unexpected number of cameras: expected {0}, found {1}"),
				InExpectedCameraCount,
				InStereoCameraTakeInfo.CameraMap.Num()
			),
			InStereoCameraTakeInfo,
			Issues
		);
	}

	CheckForNonANSICharacters(InStereoCameraTakeInfo, Issues);

	CheckAudio(InStereoCameraTakeInfo, Issues);
	CheckForEmptyFramesPaths(InStereoCameraTakeInfo, Issues);

	return Issues;
}


TArray<FText> FStereoCameraSystemTakeParser::ResolveResolution(FStereoCameraTakeInfo& OutStereoCameraTakeInfo)
{
	using namespace UE::CaptureManager;

	FResolutionResolver ResolutionResolver;

	TArray<FText> Issues;

	for (TPair<FString, FStereoCameraTakeInfo::FCamera>& CameraIt : OutStereoCameraTakeInfo.CameraMap)
	{
		const FStereoCameraTakeInfo::FCamera& Camera = CameraIt.Value;

		// Doesn't take into account dropped frames:
		const int32 CurrentCameraFrames = (Camera.FrameRange.Value - Camera.FrameRange.Key) + 1;

		TValueOrError<FIntPoint, FResolutionResolver::EAddError> CameraResolution = ResolutionResolver.Add(Camera);

		if (CameraResolution.HasError())
		{
			ReportResolutionAddError(CameraResolution.GetError(), Camera.UserId, OutStereoCameraTakeInfo, Issues);
			continue;
		}

		CameraIt.Value.Resolution = CameraResolution.StealValue();
	}

	TValueOrError<FIntPoint, FResolutionResolver::EResolveError> Resolution = ResolutionResolver.Resolve();

	if (Resolution.HasError())
	{
		ReportResolutionResolveError(Resolution.GetError(), OutStereoCameraTakeInfo, Issues);
	}

	return Issues;
}

TOptional<FText> CheckResolutions(const FTakeMetadata::FVideo& InCameraTakeInfo, const FCameraCalibration& InCameraCalibration)
{
	FIntPoint CameraCalibrationResolution(InCameraCalibration.ImageSize.X, InCameraCalibration.ImageSize.Y);

	if (InCameraTakeInfo.FrameWidth.IsSet() && InCameraTakeInfo.FrameHeight.IsSet())
	{
		FIntPoint VideoResolution(InCameraTakeInfo.FrameWidth.GetValue(), InCameraTakeInfo.FrameHeight.GetValue());

		if (VideoResolution != CameraCalibrationResolution)
		{
			return LOCTEXT("CheckResolutions_ResolutionValidationFailed", "Calibration and Image resolution differ");
		}
	}
	else
	{
		UE_LOGF(LogStereoCameraMetadata, Warning, "Could not check resolutions as FrameWidth and FrameHeight have not been set for %ls at %ls", *InCameraTakeInfo.Name, *InCameraTakeInfo.Path);
	}

	return {};
}

#undef LOCTEXT_NAMESPACE
