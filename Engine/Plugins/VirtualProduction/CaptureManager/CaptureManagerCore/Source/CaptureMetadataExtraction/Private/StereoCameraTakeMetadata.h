// Copyright Epic Games, Inc. All Rights Reserved.

// TODO: Make Private

#pragma once

#include "CameraCalibration.h"
#include "FrameRange.h"

#include "Async/StopToken.h"

#include "CaptureManagerTakeMetadata.h"


class FStereoCameraInfo
{
public:

	FString Name;
	FCameraCalibration Calibration;
};

class FStereoCameraTakeInfo
{
public:
	struct FDeviceInfo
	{
		FString Model;
		FString Type;
		FString Id;
	};

	struct FCamera
	{
		FString UserId;
		TPair<uint32, uint32> FrameRange;
		float FrameRate = 0.0f;
		FString FramesPath;
		FString StartTimecode;
		FIntPoint Resolution = FIntPoint::NoneValue;
		TArray<FFrameRange> CaptureExcludedFrames;
	};
	using FCameraMap = TMap<FString, FCamera>;

	struct FAudio
	{
		FString UserId;
		FString StreamPath;
		float TimecodeRate = 0.0f;
		FString StartTimecode;
	};
	using FAudioArray = TArray<FAudio>;

	uint32 Version = 0;
	FString Id;
	uint32 Take = 0;
	FString Slate;
	FString ThumbnailPath;
	FDateTime Date;
	FDeviceInfo DeviceInfo;
	FString CalibrationFilePath;

	FCameraMap CameraMap;
	FAudioArray AudioArray;

	FString TakeJsonFilePath;

	FString GetName() const;
	FString GetFolderName() const;
};

class FStereoCameraSystemTakeParser
{
public:

	static TOptional<FStereoCameraTakeInfo> ParseTakeMetadataFile(const FString& InFileName);

	static TArray<FText> CheckStereoCameraTakeInfo(
		const FString& InFilePath,
		FStereoCameraTakeInfo& InStereoCameraTakeInfo,
		int32 InExpectedCameraCount,
		const FString& InDeviceType);

	static TArray<FText> ResolveResolution(FStereoCameraTakeInfo& OutStereoCameraTakeInfo);
};
