// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleDeviceList.h"

namespace UE::CaptureManager
{

const FAppleDeviceList::FDeviceMap FAppleDeviceList::DeviceMap = {
	// iPhone
	{ "iPhone1,1", "iPhone" },
	{ "iPhone1,2", "iPhone 3G" },
	{ "iPhone2,1", "iPhone 3GS" },
	{ "iPhone3,1", "iPhone 4" },
	{ "iPhone3,2", "iPhone 4" },
	{ "iPhone3,3", "iPhone 4" },
	{ "iPhone4,1", "iPhone 4S" },
	{ "iPhone5,1", "iPhone 5" },
	{ "iPhone5,2", "iPhone 5" },
	{ "iPhone5,3", "iPhone 5C" },
	{ "iPhone5,4", "iPhone 5C" },
	{ "iPhone6,1", "iPhone 5S" },
	{ "iPhone6,2", "iPhone 5S" },
	{ "iPhone7,1", "iPhone 6 Plus" },
	{ "iPhone7,2", "iPhone 6" },
	{ "iPhone8,1", "iPhone 6S" },
	{ "iPhone8,2", "iPhone 6S Plus" },
	{ "iPhone8,4", "iPhone SE" },
	{ "iPhone9,1", "iPhone 7" },
	{ "iPhone9,2", "iPhone 7 Plus" },
	{ "iPhone9,3", "iPhone 7"  },
	{ "iPhone9,4", "iPhone 7 Plus" },
	{ "iPhone10,1", "iPhone 8 " },
	{ "iPhone10,2", "iPhone 8 Plus" },
	{ "iPhone10,3", "iPhone X" },
	{ "iPhone10,4", "iPhone 8" },
	{ "iPhone10,5", "iPhone 8 Plus" },
	{ "iPhone10,6", "iPhone X" },
	{ "iPhone11,2", "iPhone XS" },
	{ "iPhone11,4", "iPhone XS Max" },
	{ "iPhone11,6", "iPhone XS Max" },
	{ "iPhone11,8", "iPhone XR" },
	{ "iPhone12,1", "iPhone 11" },
	{ "iPhone12,3", "iPhone 11 Pro" },
	{ "iPhone12,5", "iPhone 11 Pro Max" },
	{ "iPhone12,8", "iPhone SE 2" },
	{ "iPhone13,1", "iPhone 12 Mini" },
	{ "iPhone13,2", "iPhone 12" },
	{ "iPhone13,3", "iPhone 12 Pro" },
	{ "iPhone13,4", "iPhone 12 Pro Max" },
	{ "iPhone14,2", "iPhone 13 Pro" },
	{ "iPhone14,3", "iPhone 13 Pro Max" },
	{ "iPhone14,4", "iPhone 13 Mini" },
	{ "iPhone14,5", "iPhone 13" },
	{ "iPhone14,6", "iPhone SE 3" },
	{ "iPhone14,7", "iPhone 14" },
	{ "iPhone14,8", "iPhone 14 Plus" },
	{ "iPhone15,2", "iPhone 14 Pro" },
	{ "iPhone15,3", "iPhone 14 Pro Max" },

	// iPad
	{ "iPad1,1", "iPad" },
	{ "iPad2,1", "iPad 2" },
	{ "iPad2,2", "iPad 2" },
	{ "iPad2,3", "iPad 2" },
	{ "iPad2,4", "iPad 2" },
	{ "iPad3,1", "iPad 3" },
	{ "iPad3,2", "iPad 3" },
	{ "iPad3,3", "iPad 3" },
	{ "iPad3,4", "iPad 4" },
	{ "iPad3,5", "iPad 4" },
	{ "iPad3,6", "iPad 4" },
	{ "iPad6,11", "iPad 5" },
	{ "iPad6,12", "iPad 5" },
	{ "iPad7,5", "iPad 6" },
	{ "iPad7,6", "iPad 6" },
	{ "iPad7,11", "iPad 7" },
	{ "iPad7,12", "iPad 7" },
	{ "iPad11,6", "iPad 8" },
	{ "iPad11,7", "iPad 8" },
	{ "iPad12,1", "iPad 9" },
	{ "iPad12,2", "iPad 9" },
	{ "iPad4,1", "iPad Air" },
	{ "iPad4,2", "iPad Air" },
	{ "iPad4,3", "iPad Air" },
	{ "iPad5,3", "iPad Air 2" },
	{ "iPad5,4", "iPad Air 2" },
	{ "iPad11,3", "iPad Air 3" },
	{ "iPad11,4", "iPad Air 3" },
	{ "iPad13,1", "iPad Air 4" },
	{ "iPad13,2", "iPad Air 4" },
	{ "iPad13,16", "iPad Air 5" },
	{ "iPad13,17", "iPad Air 5" },
	{ "iPad2,5", "iPad Mini" },
	{ "iPad2,6", "iPad Mini" },
	{ "iPad2,7", "iPad Mini" },
	{ "iPad4,4", "iPad Mini 2" },
	{ "iPad4,5", "iPad Mini 2" },
	{ "iPad4,6", "iPad Mini 2" },
	{ "iPad4,7", "iPad Mini 3" },
	{ "iPad4,8", "iPad Mini 3" },
	{ "iPad4,9", "iPad Mini 3" },
	{ "iPad5,1", "iPad Mini 4" },
	{ "iPad5,2", "iPad Mini 4" },
	{ "iPad11,1", "iPad Mini 5" },
	{ "iPad11,2", "iPad Mini 5" },
	{ "iPad14,1", "iPad Mini 6" },
	{ "iPad14,2", "iPad Mini 6" },
	{ "iPad6,3", "iPad Pro 9.7-inch" },
	{ "iPad6,4", "iPad Pro 9.7-inch" },
	{ "iPad7,3", "iPad Pro 10.5-inch" },
	{ "iPad7,4", "iPad Pro 10.5-inch" },
	{ "iPad8,1", "iPad Pro 11-inch" },
	{ "iPad8,2", "iPad Pro 11-inch" },
	{ "iPad8,3", "iPad Pro 11-inch" },
	{ "iPad8,4", "iPad Pro 11-inch" },
	{ "iPad8,9", "iPad Pro 11-inch 2" },
	{ "iPad8,10", "iPad Pro 11-inch 2" },
	{ "iPad13,4", "iPad Pro 11-inch 3" },
	{ "iPad13,5", "iPad Pro 11-inch 3" },
	{ "iPad13,6", "iPad Pro 11-inch 3" },
	{ "iPad13,7", "iPad Pro 11-inch 3" },
	{ "iPad6,7", "iPad Pro 12.9-inch" },
	{ "iPad6,8", "iPad Pro 12.9-inch" },
	{ "iPad7,1", "iPad Pro 12.9-inch 2" },
	{ "iPad7,2", "iPad Pro 12.9-inch 2" },
	{ "iPad8,5", "iPad Pro 12.9-inch 3" },
	{ "iPad8,6", "iPad Pro 12.9-inch 3" },
	{ "iPad8,7", "iPad Pro 12.9-inch 3" },
	{ "iPad8,8", "iPad Pro 12.9-inch 3" },
	{ "iPad8,11", "iPad Pro 12.9-inch 4" },
	{ "iPad8,12", "iPad Pro 12.9-inch 4" },
	{ "iPad13,8", "iPad Pro 12.9-inch 5" },
	{ "iPad13,9", "iPad Pro 12.9-inch 5" },
	{ "iPad13,10", "iPad Pro 12.9-inch 5" },
	{ "iPad13,11", "iPad Pro 12.9-inch 5" }
};

}