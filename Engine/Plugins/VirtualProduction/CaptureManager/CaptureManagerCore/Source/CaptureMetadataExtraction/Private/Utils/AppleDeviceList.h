// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"

namespace UE::CaptureManager
{

class FAppleDeviceList
{
public:
	using FDeviceMap = TMap<FString, FString>;

	static const FDeviceMap DeviceMap;
};

}