// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/CaptureExtractTimecode.h"

#include "Audio.h"
#include "Async/Async.h"
#include "CaptureManagerMediaRWModule.h"
#include "Misc/FileHelper.h"
#include "MediaPlaylist.h"
#include "Sound/SoundWaveTimecodeInfo.h"
#include "UObject/Package.h"
//#include "Settings/CaptureManagerSettings.h"
#include "Logging/LogMacros.h"
#include "IElectraPlayerPluginModule.h"
#include "IMediaPlayer.h"
#include "IMediaOptions.h"
#include "IMediaEventSink.h"
#include "IMediaTracks.h"
#include "Serialization/JsonSerializer.h"

#include "ParseTakeUtils.h"
#include "SmpteTimecodeUtils.h"

#include "ProcessRunner/ProcessRunner.h"

#define LOCTEXT_NAMESPACE "CaptureExtractTimecode"

DEFINE_LOG_CATEGORY_STATIC(LogCaptureExtractTimecode, Log, All);



static bool IsEncoderFFmpeg(const FString& InEncoderPath)
{
	// Get base filename without extension
	const FString EncoderFileName = FPaths::GetBaseFilename(InEncoderPath);
	return EncoderFileName.Equals(TEXT("ffmpeg"), ESearchCase::IgnoreCase);
}

static FString GetFFprobePath(const FString& InEncoderPath)
{
	if (!IsEncoderFFmpeg(InEncoderPath))
	{
		return FString();
	}

	// Get base filename with extension: e.g ffmpeg.exe
	const FString EncoderFileName = FPaths::GetCleanFilename(InEncoderPath);
	const FString RootPath = FPaths::GetPath(InEncoderPath);
	const FString FFprobeFileName = EncoderFileName.Replace(TEXT("ffmpeg"), TEXT("ffprobe"), ESearchCase::IgnoreCase);
	const FString FFprobePath = RootPath / FFprobeFileName;
	return FFprobePath;
}

FCaptureExtractVideoInfo::FResult FCaptureExtractVideoInfo::Create(const FString& InFilePath, TOptional<UE::CaptureManager::Private::FFProbeCommand> InProbeCommand)
{
	using namespace UE::CaptureManager;

	FCaptureExtractVideoInfo VideoInfoExtractor;

	FExtractResult Result = VideoInfoExtractor.ExtractInfo(InFilePath, InProbeCommand);

	if (Result.HasError())
	{
		return MakeError(Result.StealError());
	}

	UE_LOGF(LogCaptureExtractTimecode, Verbose, "Extracted timecode '%ls' from video file: %ls", *VideoInfoExtractor.GetTimecode().ToString(), *InFilePath);

	return MakeValue(MoveTemp(VideoInfoExtractor));
}

FFrameRate FCaptureExtractVideoInfo::GetFrameRate() const
{
	return FrameRate;
}

FTimecode FCaptureExtractVideoInfo::GetTimecode() const
{
	return TimecodeInfo.Timecode;
}

FFrameRate FCaptureExtractVideoInfo::GetTimecodeRate() const
{
	return TimecodeInfo.TimecodeRate;
}

bool FCaptureExtractVideoInfo::ContainsAudio() const
{
	return bContainsAudio;
}

float FCaptureExtractVideoInfo::GetAudioDurationSeconds() const
{
	return AudioDurationSeconds;
}

EMediaOrientation FCaptureExtractVideoInfo::GetVideoOrientation() const
{
	return VideoOrientation;
}

FCaptureExtractVideoInfo::FCaptureExtractVideoInfo()
	: FrameRate(1, 1)
{
}

FCaptureExtractVideoInfo::FExtractResult FCaptureExtractVideoInfo::ExtractInfo(const FString& InFilePath, TOptional<UE::CaptureManager::Private::FFProbeCommand> InProbeCommand)
{
	using namespace UE::CaptureManager;
	using namespace UE::CaptureManager::Private;

	// Replace any backslashes to avoid escape code issues while logging the file path
	const FString FilePath = InFilePath.Replace(TEXT("\\"), TEXT("/"));
	if (FilePath.IsEmpty())
	{
		return MakeError(ECaptureExtractInfoError::InternalError);
	}

	if (InProbeCommand)
	{
		return ExtractInfoUsingFFProbe(InFilePath, *InProbeCommand);
	}
	else
	{
		// We gather audio duration via an audio reader if available and the format is supported.
		FMediaRWManager& MediaManager = FModuleManager::LoadModuleChecked<FCaptureManagerMediaRWModule>("CaptureManagerMediaRW").Get();
		TValueOrError<TUniquePtr<IAudioReader>, FText> AudioReaderResult = MediaManager.CreateAudioReader(InFilePath);
		if (AudioReaderResult.HasValue())
		{
			bContainsAudio = true;
			AudioDurationSeconds = static_cast<float>(AudioReaderResult.StealValue()->GetDuration().GetTotalSeconds());
		}

		// Frame rate, and timecode data is read from the electra player
		return ExtractInfoUsingElectraPlayer(InFilePath);
	}
}

FCaptureExtractVideoInfo::FExtractResult FCaptureExtractVideoInfo::ExtractInfoUsingElectraPlayer(const FString& InFilePath)
{
	using namespace UE::CaptureManager;

	using FEventSinkResult = FExtractResult;
	TPromise<FExtractResult> EventSinkPromise;

	class FMediaEventSink : public IMediaEventSink
	{
	public:

		FMediaEventSink(TPromise<FEventSinkResult>& InPromise)
			: Promise(InPromise)
			, bReleased(false)
		{
		}

		virtual void ReceiveMediaEvent(EMediaEvent Event) override
		{
			if (bReleased)
			{
				return;
			}

			if (Event == EMediaEvent::MediaOpened)
			{
				Promise.SetValue(MakeValue());
			}
			else if (Event == EMediaEvent::MediaOpenFailed)
			{
				Promise.SetValue(MakeError(ECaptureExtractInfoError::UnableToOpenMedia));
			}
		}

		void Release()
		{
			bReleased = true;
		}

	private:

		TPromise<FEventSinkResult>& Promise;
		std::atomic_bool bReleased;

	} MediaEventSink(EventSinkPromise);

	IElectraPlayerPluginModule& ElectraModule = FModuleManager::LoadModuleChecked<IElectraPlayerPluginModule>("ElectraPlayerPlugin");

	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> MediaPlayer = ElectraModule.CreatePlayer(MediaEventSink);
	if (!MediaPlayer)
	{
		// Broken promises are considered errors
		FExtractResult Error = MakeError(ECaptureExtractInfoError::InternalError);
		EventSinkPromise.SetValue(Error);
		return Error;
	}

	UDesiredPlayerMediaSource* MediaSource = NewObject<UDesiredPlayerMediaSource>();
	MediaSource->SetFilePath(InFilePath);

	FString FileUrl = TEXT("file://") + InFilePath;

	FMediaPlayerOptions PlayerOptions;
	PlayerOptions.SetAllAsOptional();
	PlayerOptions.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ParseTimecodeInfo(), FVariant());

	MediaPlayer->Open(FileUrl, MediaSource, &PlayerOptions);

	FString TimecodeString;
	FString TimecodeRateString;

	FDateTime EventWaitStart = FDateTime::Now();

	TFuture<FEventSinkResult> EventSinkFuture = EventSinkPromise.GetFuture();

	bool bIsTimeout = true;

	while ((FDateTime::Now() - EventWaitStart).GetSeconds() < TimeoutPeriod)
	{
		MediaPlayer->TickInput(0, 0);

		if (EventSinkFuture.WaitFor(FTimespan::FromMilliseconds(100.0f))) // Wait 100 milliseconds before invoking TickInput again
		{
			FEventSinkResult Result = EventSinkFuture.Get();
			if (Result.HasError())
			{
				return Result;
			}

			bIsTimeout = false;

			const FVariant Timecode = MediaPlayer->GetMediaInfo(UMediaPlayer::MediaInfoNameStartTimecodeValue.Resolve());
			if (!Timecode.IsEmpty())
			{
				TimecodeString = Timecode.GetValue<FString>();

				const FVariant TimecodeRate = MediaPlayer->GetMediaInfo(UMediaPlayer::MediaInfoNameStartTimecodeFrameRate.Resolve());
				if (!TimecodeRate.IsEmpty())
				{
					TimecodeRateString = TimecodeRate.GetValue<FString>();
				}
			}

			int32 Track = MediaPlayer->GetTracks().GetSelectedTrack(EMediaTrackType::Video);
			int32 Format = MediaPlayer->GetTracks().GetTrackFormat(EMediaTrackType::Video, Track);

			if (Track != INDEX_NONE && Format != INDEX_NONE)
			{
				FMediaVideoTrackFormat FormatInfo;
				MediaPlayer->GetTracks().GetVideoTrackFormat(Track, Format, FormatInfo);

				FrameRate = ConvertFrameRate(FormatInfo.FrameRate);
			}
			else
			{
				UE_LOGF(LogCaptureExtractTimecode, Warning, "Failed to obtain the frame rate");
			}

			break;
		}
	}

	// Making sure that no additional events will arrive
	MediaEventSink.Release();
	MediaPlayer->Close();

	if (bIsTimeout)
	{
		FExtractResult Error = MakeError(ECaptureExtractInfoError::InternalError);

		if (!EventSinkFuture.IsReady())
		{
			// Broken promises are considered errors
			EventSinkPromise.SetValue(Error);
		}

		return Error;
	}

	if (!TimecodeString.IsEmpty())
	{
		const TOptional<FTimecode> Timecode = FTimecode::ParseTimecode(*TimecodeString);
		if (!Timecode.IsSet())
		{
			UE_LOGF(LogCaptureExtractTimecode, Warning, "Failed to parse the timecode");
		}
		else
		{
			TimecodeInfo.Timecode = Timecode.GetValue();
		}
	}
	else
	{
		UE_LOGF(LogCaptureExtractTimecode, Warning, "Timecode has not been found");
	}

	if (!TimecodeRateString.IsEmpty())
	{
		TValueOrError<FFrameRate, UE::CaptureManager::ECaptureExtractInfoError> Result = ParseTimecodeRate(TimecodeRateString);
		if (Result.IsValid())
		{
			TimecodeInfo.TimecodeRate = Result.GetValue();
		}
	}
	else
	{
		UE_LOGF(LogCaptureExtractTimecode, Warning, "Timecode rate has not been found. Using video Frame rate as timecode rate");
		TimecodeInfo.TimecodeRate = FrameRate;
	}

	return MakeValue();
}

FCaptureExtractVideoInfo::FExtractResult FCaptureExtractVideoInfo::ExtractInfoUsingFFProbe(const FString& InFilePath, const UE::CaptureManager::Private::FFProbeCommand& InProbeCommand)
{
	using namespace UE::CaptureManager;

	FProcessRunnerResult RunnerResult = InProbeCommand.Execute(InFilePath, TimeoutPeriod);

	if (!RunnerResult.HasValue())
	{
		return MakeError(ECaptureExtractInfoError::InternalError);
	}

	TArray<uint8> CommandOutput = RunnerResult.StealValue();
	FString JsonString = FString::ConstructFromPtrSize(reinterpret_cast<const UTF8CHAR*>(CommandOutput.GetData()), CommandOutput.Num());

	TSharedPtr<FJsonObject> JsonObject;
	if (!FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(JsonString), JsonObject))
	{
		UE_LOGF(LogCaptureExtractTimecode, Error, "Failed to parse ffprobe output for file \"%ls\". Ensure ffprobe is up to date.", *FPaths::GetCleanFilename(InFilePath));
		return MakeError(ECaptureExtractInfoError::FailedToParseJson);
	}

	// Streams
	const TArray<TSharedPtr<FJsonValue>>* StreamsArray;
	if (JsonObject->TryGetArrayField(TEXT("streams"), StreamsArray))
	{
		for (const TSharedPtr<FJsonValue>& Value : *StreamsArray)
		{
			TSharedPtr<FJsonObject> StreamObject = Value->AsObject();
			if (StreamObject.IsValid())
			{
				FString CodecType;
				if (!StreamObject->TryGetStringField(TEXT("codec_type"), CodecType)) 
				{
					continue;
				}

				if (CodecType == TEXT("audio"))
				{
					// Audio Duration
					float Duration = 0.0;
					if (StreamObject->TryGetNumberField(TEXT("duration"), Duration))
					{
						bContainsAudio = true;
						AudioDurationSeconds = Duration;
					}
				}
				else if (CodecType == TEXT("video"))
				{
					// Frame Rate
					FString FrameRateString;
					if (StreamObject->TryGetStringField(TEXT("r_frame_rate"), FrameRateString)) 
					{
						TArray<FString> FrameRateComponents;
						FrameRateString.ParseIntoArray(FrameRateComponents, TEXT("/"));
						if (FrameRateComponents.Num() == 2)
						{
							int32 Numerator = FCString::Atoi(*FrameRateComponents[0]);
							int32 Denominator = FCString::Atoi(*FrameRateComponents[1]);
							FrameRate = FFrameRate(Numerator, Denominator);
							TimecodeInfo.TimecodeRate = FrameRate;
						}
					}

					// Video Orientation
					int32 Rotation = 0;
					const TArray<TSharedPtr<FJsonValue>>* SideDataArrayPointer;
					if (StreamObject->TryGetArrayField(TEXT("side_data_list"), SideDataArrayPointer))
					{
						const TArray<TSharedPtr<FJsonValue>> SideDataArray = *SideDataArrayPointer;
						if (SideDataArray.Num() > 0)
						{
							const TSharedPtr<FJsonObject>* SideDataObjectPointer;
							if (SideDataArray[0]->TryGetObject(SideDataObjectPointer))
							{
								(*SideDataObjectPointer)->TryGetNumberField(TEXT("rotation"), Rotation);
							}
						}
					}
					switch (FMath::WrapExclusive(Rotation, 0, 360))
					{
						case 90:
							VideoOrientation = EMediaOrientation::CW90;
							break;
						case 180:
							VideoOrientation = EMediaOrientation::CW180;
							break;
						case 270:
							VideoOrientation = EMediaOrientation::CW270;
							break;
						default:
							VideoOrientation = EMediaOrientation::Original;
					}
					// Timecode
					const TSharedPtr<FJsonObject>* TagsObject;
					if (StreamObject->TryGetObjectField(TEXT("tags"), TagsObject))
					{
						FString TimecodeString;
						if ((*TagsObject)->TryGetStringField(TEXT("timecode"), TimecodeString))
						{
							TOptional<FTimecode> TimecodeOpt = FTimecode::ParseTimecode(TimecodeString);
							if (TimecodeOpt.IsSet())
							{
								TimecodeInfo.Timecode = TimecodeOpt.GetValue();
							}
							else
							{
								UE_LOGF(LogCaptureExtractTimecode, Error, "Failed to parse the timecode string %ls", *TimecodeString);
								return MakeError(ECaptureExtractInfoError::UnableToParseTimecode);
							}
						}
					}
				}
			}
		}
	}

	// If we fail to obtain timecode from the video stream, we are searching in the format object
	if (TimecodeInfo.Timecode == FTimecode())
	{
		// Timecode
		const TSharedPtr<FJsonObject>* FormatObject;
		if (JsonObject->TryGetObjectField(TEXT("format"), FormatObject))
		{
			const TSharedPtr<FJsonObject>* TagsObject;
			if ((*FormatObject)->TryGetObjectField(TEXT("tags"), TagsObject))
			{
				FString TimecodeString;
				if ((*TagsObject)->TryGetStringField(TEXT("timecode"), TimecodeString))
				{
					TOptional<FTimecode> TimecodeOpt = FTimecode::ParseTimecode(TimecodeString);
					if (TimecodeOpt.IsSet())
					{
						TimecodeInfo.Timecode = TimecodeOpt.GetValue();
					}
					else
					{
						UE_LOGF(LogCaptureExtractTimecode, Error, "Failed to parse the timecode string %ls", *TimecodeString);
						return MakeError(ECaptureExtractInfoError::UnableToParseTimecode);
					}
				}
			}
		}
	}

	return MakeValue();
}

TValueOrError<FFrameRate, UE::CaptureManager::ECaptureExtractInfoError> FCaptureExtractVideoInfo::ParseTimecodeRate(FString TimecodeRateString)
{
	using namespace UE::CaptureManager;

	if (TimecodeRateString.IsEmpty())
	{
		return MakeError(ECaptureExtractInfoError::TimecodeRateNotFound);
	}
	else
	{
		// TimecodeRateString is made out of FFrameRate::ToPrettyText()
		// It could be "{0} fps" or "{0} s"

		FFrameRate TimecodeRate;

		FString Left;
		FString Right;
		TimecodeRateString.Split(" ", &Left, &Right);

		double Number = FCString::Atoi(*Left);
		uint32 IntNumber;
		uint32 Multiplier = 1;


		for (; Multiplier <= 10000; Multiplier *= 10)
		{
			double TmpNumber = Number * Multiplier;
			IntNumber = FMath::RoundToInt32(TmpNumber);
			if (FMath::Abs(TmpNumber - IntNumber) < 0.01)
			{
				break;
			}
		}

		uint32 Nominator = 0;
		uint32 Denominator = 0;

		if (Right == "fps")
		{
			Nominator = IntNumber;
			Denominator = Multiplier;
		}
		else if (Right == "s")
		{
			Nominator = Multiplier;
			Denominator = IntNumber;
		}

		TimecodeRate = FFrameRate(Nominator, Denominator);

		return MakeValue(TimecodeRate);
	}
}

FCaptureExtractAudioTimecode::FCaptureExtractAudioTimecode(const FString& InFilePath)
	: FilePath(InFilePath)
{}

FCaptureExtractAudioTimecode::FTimecodeInfoResult FCaptureExtractAudioTimecode::Extract()
{
	return ExtractTimecodeFromBroadcastWaveFormat(FFrameRate());
}

FCaptureExtractAudioTimecode::FTimecodeInfoResult FCaptureExtractAudioTimecode::Extract(FFrameRate InFrameRate)
{
	using namespace UE::CaptureManager;

	check(!FilePath.IsEmpty());

	FTimecodeInfoResult Result = MakeError(ECaptureExtractInfoError::UnhandledMedia);

	FString FileExtension = FPaths::GetExtension(FilePath);
	if (FileExtension == "wav")
	{
		// Convert timecode rate to SMPTE timecode rate
		FFrameRate TimecodeRate = UE::CaptureData::EstimateSmpteTimecodeRate(InFrameRate);
		Result = ExtractTimecodeFromBroadcastWaveFormat(TimecodeRate);
	}

	if (Result.HasValue())
	{
		UE_LOGF(LogCaptureExtractTimecode, Verbose, "Extracted timecode '%ls' from audio file: %ls", *Result.GetValue().Timecode.ToString(), *FilePath);
	}
	else
	{
		UE_LOGF(LogCaptureExtractTimecode, Log, "Timecode not found for audio file: %ls", *FilePath);
	}

	return Result;
}

FCaptureExtractAudioTimecode::FTimecodeInfoResult FCaptureExtractAudioTimecode::ExtractTimecodeFromBroadcastWaveFormat(FFrameRate InTimecodeRate)
{
	using namespace UE::CaptureManager;

	TArray<uint8> WAVData;
	if (FFileHelper::LoadFileToArray(WAVData, *FilePath))
	{
		FWaveModInfo WAVInfo;
		if (WAVInfo.ReadWaveInfo(WAVData.GetData(), WAVData.Num()))
		{
			if (WAVInfo.TimecodeInfo.IsValid())
			{
				FSoundWaveTimecodeInfo TimecodeInfo = *WAVInfo.TimecodeInfo.Get();
				const double NumSecondsSinceMidnight = TimecodeInfo.GetNumSecondsSinceMidnight();

				FFrameRate TimecodeRate = TimecodeInfo.TimecodeRate;
				const bool bTimecodeRateIsSampleRate = TimecodeRate == FFrameRate(TimecodeInfo.NumSamplesPerSecond, 1);
				if (bTimecodeRateIsSampleRate)
				{
					UE_LOGF(
						LogCaptureExtractTimecode,
						Log,
						"Embedded timecode rate is %.2f fps (the sample rate). "
						"This usually indicates there is no timecode rate information in the wav file: %ls",
						TimecodeInfo.TimecodeRate.AsDecimal(),
						*FilePath
					);

					if (InTimecodeRate != FFrameRate())
					{
						// Use the provided timecode rate instead
						TimecodeRate = InTimecodeRate;

						UE_LOGF(
							LogCaptureExtractTimecode,
							Log,
							"Taking the embedded audio timecode but estimating an SMPTE audio timecode rate. "
							"Timecode rate for %ls set to %.2f",
							*FilePath,
							TimecodeRate.AsDecimal()
						);
					}
				}

				FTimecode AudioTimecode = FTimecode(NumSecondsSinceMidnight, TimecodeRate, TimecodeInfo.bTimecodeIsDropFrame, /* InbRollover = */ true);
				FTimecodeInfo TimecodeAndRate
				{ 
					.Timecode = MoveTemp(AudioTimecode), 
					.TimecodeRate = MoveTemp(TimecodeRate)
				};

				return MakeValue(MoveTemp(TimecodeAndRate));
			}
		}
	}

	return MakeError(ECaptureExtractInfoError::TimecodeNotFound);
}

namespace UE::CaptureManager::Private
{

FFProbeCommand::FFProbeCommand(FString InFfmpegPath) :
	ProbePath(GetFFprobePath(InFfmpegPath))
{
}

FProcessRunnerResult FFProbeCommand::Execute(const FString& InFilePath, int32 InTimeoutSeconds, const FStopToken* InStopToken) const
{
	static constexpr TCHAR ArgsFmt[] = TEXT("-v error -show_entries stream=codec_type,r_frame_rate,duration:format_tags=timecode:stream_side_data=rotation:stream_tags=timecode -of json \"{0}\"");

	if (!FPaths::FileExists(ProbePath))
	{
		UE_LOGF(LogCaptureExtractTimecode, Warning, "Could not extract timecode for '%ls' using ffprobe. ffprobe could not be found at path '%ls'. ffprobe is expected to be found in the same directory as ffmpeg.", *InFilePath, *ProbePath);
		return MakeError(EProcessRunnerError::LaunchFailed);
	}

	const FString Args = FString::Format(ArgsFmt, { InFilePath });
	return FProcessRunner::Run(ProbePath, Args, InStopToken, InTimeoutSeconds);
}

}

#undef LOCTEXT_NAMESPACE


