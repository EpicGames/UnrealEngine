// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FileMediaSource.h"

#include "MediaSample.h"
#include "MediaPlayer.h"

#include "ProcessRunner/ProcessRunner.h"
#include "Templates/ValueOrError.h"

#include "IMediaClock.h"
#include "IMediaClockSink.h"
#include "IMediaModule.h"

#include "CaptureExtractTimecode.generated.h"

#define UE_API CAPTUREMETADATAEXTRACTION_API

namespace UE::CaptureManager
{

struct FTimecodeInfo
{
	FTimecode Timecode;
	FFrameRate TimecodeRate;
};

struct FVideoInformation
{
	FFrameRate FrameRate;
	FTimecodeInfo TimecodeInfo;
};

enum ECaptureExtractInfoError : int32
{
	InternalError = -1,
	TimecodeNotFound = 1,
	UnableToParseTimecode = 2,
	UnableToParseTimecodeRate = 3,
	TimecodeRateNotFound = 4,
	UnhandledMedia = 5,
	UnableToOpenMedia = 6,
	FfprobeNotFound = 7,
	FailedToParseJson = 8,
};
}

#if WITH_EDITOR
UCLASS(BlueprintType, MinimalAPI)
class UDesiredPlayerMediaSource : public UFileMediaSource
{
	GENERATED_BODY()
public:
	//~ IMediaOptions interface
	virtual FName GetDesiredPlayerName() const override
	{
		return TEXT("ElectraPlayer");
	}
};
#endif // WITH_EDITOR

namespace UE::CaptureManager::Private
{
class FFProbeCommand
{
public:
	FFProbeCommand(FString InFfmpegPath);

	FProcessRunnerResult Execute(const FString& InFilePath, int32 InTimeoutSeconds, const FStopToken* InStopToken = nullptr) const;

private:
	const FString ProbePath;
};
}

class FCaptureExtractVideoInfo final
{
public:

	using FResult = TValueOrError<FCaptureExtractVideoInfo, UE::CaptureManager::ECaptureExtractInfoError>;

	UE_API static FResult Create(const FString& InFilePath, TOptional<UE::CaptureManager::Private::FFProbeCommand> InProbeCommand /*= {}*/);

	UE_API FFrameRate GetFrameRate() const;
	UE_API FTimecode GetTimecode() const;
	UE_API FFrameRate GetTimecodeRate() const;
	UE_API bool ContainsAudio() const;
	UE_API float GetAudioDurationSeconds() const;
	UE_API EMediaOrientation GetVideoOrientation() const;

private:

	FCaptureExtractVideoInfo();

	using FExtractResult = TValueOrError<void, UE::CaptureManager::ECaptureExtractInfoError>;
	FExtractResult ExtractInfo(const FString& InFilePath, TOptional<UE::CaptureManager::Private::FFProbeCommand> InProbeCommand);
	FExtractResult ExtractInfoUsingElectraPlayer(const FString& InFilePath);
	FExtractResult ExtractInfoUsingFFProbe(const FString& InFilePath, const UE::CaptureManager::Private::FFProbeCommand& InProbeCommand);

	TValueOrError<FFrameRate, UE::CaptureManager::ECaptureExtractInfoError> ParseTimecodeRate(FString TimecodeRateString);

	static constexpr int32 TimeoutPeriod = 3;

	FFrameRate FrameRate;
	UE::CaptureManager::FTimecodeInfo TimecodeInfo;
	bool bContainsAudio = false;
	float AudioDurationSeconds = 0.0f;
	EMediaOrientation VideoOrientation = EMediaOrientation::Original;
};

class FCaptureExtractAudioTimecode final : public TSharedFromThis<FCaptureExtractAudioTimecode>
{
public:

	using FTimecodeInfoResult = TValueOrError<UE::CaptureManager::FTimecodeInfo, UE::CaptureManager::ECaptureExtractInfoError>;

	UE_API FCaptureExtractAudioTimecode(const FString& InFilePath);
	UE_API ~FCaptureExtractAudioTimecode() = default;

	UE_API FTimecodeInfoResult Extract();
	UE_API FTimecodeInfoResult Extract(FFrameRate InFrameRate);

private:

	FString FilePath;

	const int32 TimeoutPeriod = 3;

	FTimecodeInfoResult ExtractTimecodeFromBroadcastWaveFormat(FFrameRate InTimecodeRate);
};

#undef UE_API
