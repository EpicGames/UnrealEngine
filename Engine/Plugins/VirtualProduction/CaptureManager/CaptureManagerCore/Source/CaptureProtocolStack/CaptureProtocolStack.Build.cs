// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CaptureProtocolStack : ModuleRules
{
	public CaptureProtocolStack(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange(new string[]
		{
            "Core",
			"CaptureUtils"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CoreUObject",
			"Engine",
			"Json",
            "JsonUtilities",
			"Networking",
			"Sockets"
		});
	}
}
