// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

namespace UE::CaptureManager
{

using FCaptureProtocolStackModule = IModuleInterface;

}

IMPLEMENT_MODULE(UE::CaptureManager::FCaptureProtocolStackModule, CaptureProtocolStack)
