// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Capabilities/Utils.h"
#include "Control/Messages/ControlJsonUtilities.h"
#include "Control/Messages/Constants.h"
#include "Control/Messages/ProtocolConstants.h"

#include "Dom/JsonValue.h"

namespace UE::CaptureManager
{
TProtocolResult<void> FCapabilityUtilities::TypeStringToEnum(const FString& InTypeString, ECapabilityValue::Type& OutType)
{
	if (InTypeString == CPS::Capabilities::GBoolean)
	{
		OutType = ECapabilityValue::Type::Boolean;
	}
	else if (InTypeString == CPS::Capabilities::GInteger)
	{
		OutType = ECapabilityValue::Type::Integer;
	}
	else if (InTypeString == CPS::Capabilities::GFloat)
	{
		OutType = ECapabilityValue::Type::Float;
	}
	else if (InTypeString == CPS::Capabilities::GString)
	{
		OutType = ECapabilityValue::Type::String;
	}
	else if (InTypeString == CPS::Capabilities::GEnumeration)
	{
		OutType = ECapabilityValue::Type::Enum;
	}
	else
	{
		return FCaptureProtocolError(TEXT("Invalud value type string."));
	}
	return ResultOk;
}

TProtocolResult<void> FCapabilityUtilities::ValueFromJsonObject(const TSharedPtr<FJsonObject>& InJsonObject,
																const FString& InKey,
																const ECapabilityValue::Type InType,
																FCapabilityValue& OutValue)
{
	if (InType == ECapabilityValue::Type::Boolean)
	{
		bool Value;
		CHECK_PARSE(FJsonUtility::ParseBool(InJsonObject, InKey, Value));
		OutValue.Set<bool>(Value);
	}
	else if (InType == ECapabilityValue::Type::Integer)
	{
		int32 Value;
		CHECK_PARSE(FJsonUtility::ParseNumber(InJsonObject, InKey, Value));
		OutValue.Set<int32>(Value);
	}
	else if (InType == ECapabilityValue::Type::Float)
	{
		float Value;
		CHECK_PARSE(FJsonUtility::ParseNumber(InJsonObject, InKey, Value));
		OutValue.Set<float>(Value);
	}
	else if ((InType == ECapabilityValue::Type::String) || (InType == ECapabilityValue::Type::Enum))
	{
		FString Value;
		CHECK_PARSE(FJsonUtility::ParseString(InJsonObject, InKey, Value));
		OutValue.Set<FString>(Value);
	}
	else
	{
		check(false);
	}

	return ResultOk;
}

void FCapabilityUtilities::SetProperJsonFieldType(const FString& InKey,
												  const FCapabilityValue& InValue,
												  TSharedPtr<FJsonObject>& OutJsonObjectPtr)
{
	ECapabilityValue::Type Type = TypeFromVariant(InValue);

	if (Type == ECapabilityValue::Type::Boolean)
	{
		OutJsonObjectPtr->SetBoolField(InKey, InValue.Get<bool>());
	}
	else if (Type == ECapabilityValue::Type::Integer)
	{
		OutJsonObjectPtr->SetNumberField(InKey, InValue.Get<int32>());
	}
	else if (Type == ECapabilityValue::Type::Float)
	{
		OutJsonObjectPtr->SetNumberField(InKey, InValue.Get<float>());
	}
	else if ((Type == ECapabilityValue::Type::String) || (Type == ECapabilityValue::Type::Enum))
	{
		OutJsonObjectPtr->SetStringField(InKey, InValue.Get<FString>());
	}
	else
	{
		check(false);
	}
}

ECapabilityValue::Type FCapabilityUtilities::TypeFromVariant(const FCapabilityValue& InValue)
{
	if (InValue.GetIndex() == 0)
	{
		return ECapabilityValue::Type::Boolean;
	}
	else if (InValue.GetIndex() == 1)
	{
		return ECapabilityValue::Type::Integer;
	}
	else if (InValue.GetIndex() == 2)
	{
		return ECapabilityValue::Type::Float;
	}
	else if (InValue.GetIndex() == 3)
	{
		return ECapabilityValue::Type::String;
	}
	else
	{
		check(false);
	}
	return ECapabilityValue::Type::Boolean;
}

} // namespace UE::CaptureManager
