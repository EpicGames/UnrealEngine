// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Utility/Error.h"
#include "Control/Capabilities/Capability.h"

#include "Dom/JsonObject.h"

class FString;

#define CPS_API CAPTUREPROTOCOLSTACK_API

namespace UE::CaptureManager
{
struct FCapabilityUtilities
{
	static CPS_API TProtocolResult<void> TypeStringToEnum(const FString& InTypeString, ECapabilityValue::Type& OutType);
	static CPS_API TProtocolResult<void> ValueFromJsonObject(const TSharedPtr<FJsonObject>& InJsonObject,
															 const FString& InKey,
															 const ECapabilityValue::Type InType,
															 FCapabilityValue& OutValue);
	static CPS_API void SetProperJsonFieldType(const FString& InKey,
											   const FCapabilityValue& InValue,
											   TSharedPtr<FJsonObject>& OutJsonObjectPtr);

	static CPS_API ECapabilityValue::Type TypeFromVariant(const FCapabilityValue& InValue);
};
} // namespace UE::CaptureManager
