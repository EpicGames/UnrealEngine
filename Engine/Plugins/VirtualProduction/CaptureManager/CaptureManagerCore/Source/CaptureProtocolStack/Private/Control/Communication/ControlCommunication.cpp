// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Communication/ControlCommunication.h"

#include "Utility/Definitions.h"

namespace UE::CaptureManager
{

DEFINE_LOG_CATEGORY(LogCPSControlCommunication)

FControlCommunication::FControlCommunication()
	: ReceiveHandler(nullptr)
	, SynchronizedRunnable(TQueueRunner<TSharedPtr<FRunnable>>::FOnProcess::CreateRaw(this, &FControlCommunication::RunnableHandler))
{
}

TProtocolResult<void> FControlCommunication::Init()
{
	return Client.Init();
}

TProtocolResult<void> FControlCommunication::Start(const FString& InServerIp, const uint16 InServerPort)
{
	TProtocolResult<void> Result = Client.Start(InServerIp + TEXT(":") + FString::FromInt(InServerPort));
	if (Result.HasValue())
	{
		TSharedPtr<FCommunicationRunnable> CommunicationRunnable = MakeShared<FCommunicationRunnable>(*this);
		Runnable = CommunicationRunnable;

		SynchronizedRunnable.Add(CommunicationRunnable);
	}

	return Result;
}

void FControlCommunication::Stop()
{
	Client.Stop();
	
	if (TSharedPtr<FCommunicationRunnable> CommunicationRunnable = Runnable.Pin(); CommunicationRunnable)
	{
		CommunicationRunnable->Stop();
		CommunicationRunnable->Join();
	}
}

void FControlCommunication::CommunicationRunnableStopped()
{
	Runnable.Reset();
	CommunicationStoppedHandler.ExecuteIfBound();
}

bool FControlCommunication::IsRunning() const
{
	return Client.IsRunning();
}

void FControlCommunication::SendMessage(FControlPacket InMessage)
{
	static constexpr int32 MaxNumberOfMessagesInQueue = 20;

	if (SenderQueueCurrentSize < MaxNumberOfMessagesInQueue)
	{
		SenderQueue.Enqueue(MoveTemp(InMessage));
		++SenderQueueCurrentSize;
	}
	else
	{
		UE_LOGF(LogCPSControlCommunication, Error, "Message is not scheduled for sending due to busy senders queue");
	}
}

void FControlCommunication::SetReceiveHandler(FOnPacketReceived InReceiveHandler)
{
	ReceiveHandler = MoveTemp(InReceiveHandler);
}

void FControlCommunication::SetCommunicationStoppedHandler(FCommunicationStoppedHandler InCommunicationStoppedHandler)
{
	CommunicationStoppedHandler = MoveTemp(InCommunicationStoppedHandler);
}

void FControlCommunication::RunnableHandler(TSharedPtr<FRunnable> InRunnable)
{
	InRunnable->Run();
}

void FControlCommunication::SendNextPacket()
{
	FControlPacket Packet;

	if (SenderQueue.Dequeue(Packet))
	{
		--SenderQueueCurrentSize;

		SendPacket(MoveTemp(Packet));
	}
}

void FControlCommunication::SendPacket(FControlPacket InPacket)
{
	FTcpClientWriter Writer(Client);

	TProtocolResult<void> SerializeResult = FControlPacket::Serialize(InPacket, Writer);
	if (SerializeResult.HasError())
	{
		FCaptureProtocolError Error = SerializeResult.StealError();
		UE_LOGF(LogCPSControlCommunication, Error, "Failed to serialize message to tcp writer: '%ls' code: %d", *Error.GetMessage(), Error.GetCode());
		return;
	}
}

void FControlCommunication::ReceiveNextPacket()
{
	FControlPacket Packet;
	if (ReceiverQueue.Dequeue(Packet))
	{
		ReceiveHandler.ExecuteIfBound(MoveTemp(Packet));
	}
}

TProtocolResult<FControlPacketHeader> FControlCommunication::ReceiveControlHeader()
{
	FTcpClientReader Reader(Client);

	static constexpr uint32 HeaderTimeout = 50; // 50 milliseconds as we are doing reading and writing to the socket in a single thread
	return FControlPacketHeader::Deserialize(Reader, HeaderTimeout);
}

TProtocolResult<void> FControlCommunication::ReceiveControlPacket(const FControlPacketHeader& InHeader)
{
	FTcpClientReader Reader(Client);

	TProtocolResult<FControlPacket> PacketDeserialize = FControlPacket::Deserialize(InHeader, Reader);
	if (PacketDeserialize.HasError())
	{
		return PacketDeserialize.StealError();
	}

	ReceiverQueue.Enqueue(PacketDeserialize.StealValue());

	return ResultOk;
}

FControlCommunication::FCommunicationRunnable::FCommunicationRunnable(FControlCommunication& InCommunication)
	: Communication(InCommunication)
	, bIsRunning(true)
{
}

FControlCommunication::FCommunicationRunnable::~FCommunicationRunnable() = default;

uint32 FControlCommunication::FCommunicationRunnable::Run()
{
	while (bIsRunning)
	{
		static constexpr int32 MaxMessagesPerIteration = 10;

		for (int32 Counter = 0; Counter < MaxMessagesPerIteration && !Communication.SenderQueue.IsEmpty(); ++Counter)
		{
			Communication.SendNextPacket();
		}

		for (int32 Counter = 0; Counter < MaxMessagesPerIteration && !Communication.ReceiverQueue.IsEmpty(); ++Counter)
		{
			Communication.ReceiveNextPacket();
		}

		TProtocolResult<FControlPacketHeader> ReceiveHeaderResult = Communication.ReceiveControlHeader();
		if (ReceiveHeaderResult.HasError())
		{
			HandleError(ReceiveHeaderResult.StealError());
			continue;
		}

		TProtocolResult<void> ReceivePacketResult = Communication.ReceiveControlPacket(ReceiveHeaderResult.StealValue());
		if (ReceivePacketResult.HasError())
		{
			HandleError(ReceivePacketResult.StealError());
		}
	}

	DoneEvent->Trigger();

	Communication.CommunicationRunnableStopped();

	return 0;
}

void FControlCommunication::FCommunicationRunnable::Stop()
{
	UE_LOGF(LogCPSControlCommunication, Verbose, "Stopping FCommunicationRunnable")
	bIsRunning = false;
}

void FControlCommunication::FCommunicationRunnable::Join()
{
	DoneEvent->Wait();
}

void FControlCommunication::FCommunicationRunnable::HandleError(const FCaptureProtocolError& Error)
{
	switch (Error.GetCode())
	{
	case FTcpClient::DisconnectedError:
	case FTcpClient::NoPendingDataError:
	case FTcpClient::ReadError:
		UE_LOGF(LogCPSControlCommunication, Verbose, "Unrecoverable FTcpClient error occurred when receiving control packet header: '%ls' Code: %d.", *Error.GetMessage(), Error.GetCode())
		Stop();
		break;
	default:
		UE_LOGF(LogCPSControlCommunication, Verbose, "Unhandled FTcpClient error occurred when receiving control packet header: '%ls' Code: %d", *Error.GetMessage(), Error.GetCode())
		break;
	}
}
	
}
