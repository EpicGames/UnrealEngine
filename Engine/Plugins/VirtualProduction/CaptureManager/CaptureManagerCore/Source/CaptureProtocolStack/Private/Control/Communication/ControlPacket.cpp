// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Communication/ControlPacket.h"

#include "Control/Messages/ControlJsonUtilities.h"

namespace UE::CaptureManager 
{

const TArray<uint8> FControlPacketHeader::Header = { 'C', 'P', 'S', 'C', 'O', 'N', 'T', 'R', 'O', 'L', '\0' };

#define CPS_CHECK_RESULT_RET_ERROR(Result) if (Result.HasError()) { return Result.StealError(); }

TProtocolResult<FControlPacketHeader> FControlPacketHeader::Deserialize(ITcpSocketReader& InReader, uint32 InTimeoutForControlHeader)
{
    TProtocolResult<TArray<uint8>> HeaderResult = InReader.ReceiveMessage(Header.Num(), InTimeoutForControlHeader);
    CPS_CHECK_RESULT_RET_ERROR(HeaderResult);

    TArray<uint8> HeaderData = HeaderResult.StealValue();

    if (FMemory::Memcmp(HeaderData.GetData(), Header.GetData(), Header.Num()) != 0)
    {
        return FCaptureProtocolError(TEXT("Header doesn't match"));
    }

	TProtocolResult<TArray<uint8>> VersionResult = InReader.ReceiveMessage(sizeof(uint16));
	CPS_CHECK_RESULT_RET_ERROR(VersionResult);

	TArray<uint8> VersionData = VersionResult.StealValue();

	TProtocolResult<TArray<uint8>> PayloadSizeResult = InReader.ReceiveMessage(sizeof(uint32));
    CPS_CHECK_RESULT_RET_ERROR(PayloadSizeResult);

    TArray<uint8> PayloadSizeData = PayloadSizeResult.StealValue();

    FControlPacketHeader PacketHeader;
	PacketHeader.Version = *(reinterpret_cast<const uint16*>(VersionData.GetData()));
    PacketHeader.PayloadSize = *(reinterpret_cast<const uint32*>(PayloadSizeData.GetData()));

    return PacketHeader;
}

uint32 FControlPacketHeader::GetHeaderSize()
{
    return Header.Num() + sizeof(uint16) + sizeof(uint32);
}

FControlPacketHeader::FControlPacketHeader()
{
}

FControlPacketHeader::FControlPacketHeader(uint16 InVersion, uint32 InPayloadSize)
	: Version(InVersion)
	, PayloadSize(InPayloadSize)
{
}

uint16 FControlPacketHeader::GetVersion() const
{
	return Version;
}

uint32 FControlPacketHeader::GetPayloadSize() const
{
    return PayloadSize;
}

TProtocolResult<FControlPacket> FControlPacket::Deserialize(const FControlPacketHeader& InPacketHeader, ITcpSocketReader& InReader)
{
    TProtocolResult<TArray<uint8>> PayloadResult = InReader.ReceiveMessage(InPacketHeader.GetPayloadSize());
    CPS_CHECK_RESULT_RET_ERROR(PayloadResult);

    FControlPacket Packet;

	Packet.Header = InPacketHeader;
	Packet.Payload = PayloadResult.StealValue();

    return Packet;
}

TProtocolResult<void> FControlPacket::Serialize(const FControlPacket& InMessage, ITcpSocketWriter& InWriter)
{
    TArray<uint8> Data;
    Data.Append(FControlPacketHeader::Header.GetData(), FControlPacketHeader::Header.Num());

	uint16 Version = InMessage.GetVersion();
	Data.Append(reinterpret_cast<uint8*>(&Version), sizeof(Version));

    uint32 PayloadLength = InMessage.Payload.Num();

    Data.Append(reinterpret_cast<uint8*>(&PayloadLength), sizeof(PayloadLength));
	Data.Append(InMessage.Payload.GetData(), PayloadLength);

	return InWriter.SendMessage(Data);
}

FControlPacket::FControlPacket()
{
}

FControlPacket::FControlPacket(uint16 InVersion, TArray<uint8> InPayload)
    : Header(FControlPacketHeader(InVersion, InPayload.Num()))
	, Payload(MoveTemp(InPayload))
{
}

uint16 FControlPacket::GetVersion() const
{
	return Header.GetVersion();
}

const TArray<uint8>& FControlPacket::GetPayload() const
{
    return Payload;
}

uint32 FControlPacket::GetPayloadSize() const
{
    return Header.GetPayloadSize();
}

}