// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/ControlMessenger.h"

#include "CaptureUtilsModule.h"

#include "Async/Async.h"

namespace UE::CaptureManager
{

DEFINE_LOG_CATEGORY(LogCPSControlMessenger)


FKeepAliveCounter::FKeepAliveCounter() :
	Counter(0)
{
}

void FKeepAliveCounter::Increment()
{
	++Counter;
}

void FKeepAliveCounter::Reset()
{
	Counter.store(0);
}

bool FKeepAliveCounter::HasReached(uint16 InBound)
{
	uint16 CurrentCounter = Counter.exchange(0);

	if (CurrentCounter == InBound)
	{
		return true;
	}

	Counter.exchange(CurrentCounter);

	return false;
}

const TCHAR* FControlMessenger::HandshakeSessionId = TEXT("handshake");
FControlMessenger::FControlMessenger()
	: SessionId(HandshakeSessionId)
	, AsyncRequestRunner(TQueueRunner<FAsyncRequestDelegate>::FOnProcess::CreateRaw(this, &FControlMessenger::OnAsyncRequestProcess))
{
}

FControlMessenger::~FControlMessenger()
{
	Stop();
}

void FControlMessenger::RegisterUpdateHandler(FString InAddressPath, FControlUpdate::FOnUpdateMessage InUpdateHandler)
{
	FScopeLock Lock(&UpdatesMutex);
	UpdateHandlers.Emplace(MoveTemp(InAddressPath), MoveTemp(InUpdateHandler));
}

void FControlMessenger::RegisterDisconnectHandler(FOnDisconnect InOnDisconnectHandler)
{
	OnDisconnectHandler = MoveTemp(InOnDisconnectHandler);
}

TProtocolResult<void> FControlMessenger::Start(const FString& InServerIp, const uint16 InServerPort)
{
	if (!Communication.IsRunning())
	{
		CPS_CHECK_VOID_RESULT(Communication.Init());

		Communication.SetReceiveHandler(FControlCommunication::FOnPacketReceived::CreateRaw(this, &FControlMessenger::MessageHandler));
		Communication.SetCommunicationStoppedHandler(FControlCommunication::FCommunicationStoppedHandler::CreateRaw(this, &FControlMessenger::CommunicationStoppedHandler));

		CPS_CHECK_VOID_RESULT(Communication.Start(InServerIp, InServerPort));
	}

	return ResultOk;
}

void FControlMessenger::Stop()
{
	if (Communication.IsRunning())
	{
		Communication.Stop();
	}
}

TProtocolResult<void> FControlMessenger::StartSession()
{
	TProtocolResult<FStartSessionResponse> Response = SendRequest(FStartSessionRequest());

	if (Response.HasError())
	{
		return FCaptureProtocolError(TEXT("Response for Start Session Request is invalid."));
	}

	FScopeLock Lock(&SessionIdMutex);
	if (SessionId != Response.GetValue().GetSessionId())
	{
		SessionId = Response.GetValue().GetSessionId();

		if (KeepAliveTimer.IsValid())
		{
			StopKeepAliveTimer();
		}

		StartKeepAliveTimer();
	}

	return ResultOk;
}

TProtocolResult<FGetServerInformationResponse> FControlMessenger::GetServerInformation()
{
	return SendRequest(FGetServerInformationRequest());
}

TProtocolResult<FGetTakeListResponse> FControlMessenger::GetTakeList()
{
	TProtocolResult<FGetTakeListResponse> Response =
		SendRequest(FGetTakeListRequest());

	if (Response.HasError())
	{
		return Response;
	}

	TArray<FString> TakeNames = Response.GetValue().GetNames();
	TakeNames.Sort();
	
	for (int32 Index = 0; Index < TakeNames.Num() - 1; ++Index)
	{
		if (TakeNames[Index] == TakeNames[Index + 1])
		{
			FString Message = FString::Format(TEXT("Response for Get Take List Request contains duplicate take name: {0}. Take names are expected to be unique"), { TakeNames[Index] });
			return FCaptureProtocolError(MoveTemp(Message));
		}
	}

	return Response;
}

void FControlMessenger::SendPacket(FControlPacket InPacket)
{
	Communication.SendMessage(MoveTemp(InPacket));
}

void FControlMessenger::KeepAlive()
{
	SendAsyncRequest(FKeepAliveRequest(),
					 FOnControlResponse<FKeepAliveRequest>::CreateLambda([this](TProtocolResult<FKeepAliveResponse> InResult)
	{
		if (InResult.HasError())
		{
			KeepAliveFailures.Increment();

			if (KeepAliveFailures.HasReached(3))
			{
				UE_LOGF(LogCPSControlMessenger, Warning, "Server failed to respond to Keep Alive message")
				Stop();
			}
		}
		else
		{
			KeepAliveFailures.Reset();
		}
	}));
}

// Note: This callback happens on sender and receiver thread
void FControlMessenger::MessageHandler(FControlPacket InPacket)
{
	TProtocolResult<FControlMessage> DeserializeResult = FControlMessage::Deserialize(InPacket);

	if (DeserializeResult.HasError())
	{
		FCaptureProtocolError DeserializeError = DeserializeResult.StealError();
		UE_LOGF(LogCPSControlMessenger, Error, "Failed to parse: %ls", *DeserializeError.GetMessage());
		return;
	}

	FControlMessage Message = DeserializeResult.StealValue();

	if (Message.GetType() == FControlMessage::EType::Request)
	{
		UE_LOGF(LogCPSControlMessenger, Error, "Client currently doesn't support requests.");
		return;
	}
	else if (Message.GetType() == FControlMessage::EType::Response)
	{
		FScopeLock Lock(&RequestsMutex);

		if (const TUniquePtr<FRequestContext>* Iterator = RequestContexts.Find(Message.GetTransactionId()))
		{
			const TUniquePtr<FRequestContext>& RequestContext = *Iterator;

			if (Message.GetAddressPath() != RequestContext->Request.GetAddressPath())
			{
				UE_LOGF(LogCPSControlMessenger, Error, "Invalid response arrived");
				return;
			}

			RequestContext->Promise.SetValue(MoveTemp(Message));
		}
		return;
	}
	else if (Message.GetType() == FControlMessage::EType::Update)
	{
		FScopeLock Lock(&UpdatesMutex);

		if (const FControlUpdate::FOnUpdateMessage* Iterator = UpdateHandlers.Find(Message.GetAddressPath()))
		{
			const FControlUpdate::FOnUpdateMessage& Handler = *Iterator;

			// Using Shared Pointer as ExecuteIfBound can't accept non-copyable type
			TProtocolResult<TSharedRef<FControlUpdate>> UpdateCreateResult = FControlUpdateCreator::Create(Message.GetAddressPath());

			if (UpdateCreateResult.HasError())
			{
				UE_LOGF(LogCPSControlMessenger, Error, "%ls", *(UpdateCreateResult.StealError().GetMessage()));
				return;
			}

			TSharedPtr<FControlUpdate> Update = UpdateCreateResult.StealValue();

			TProtocolResult<void> ParseResult = Update->Parse(Message.GetBody());
			if (ParseResult.HasError())
			{
				UE_LOGF(LogCPSControlMessenger, Error, "Failed to parse update: %ls", *(ParseResult.StealError().GetMessage()));
				return;
			}

			Handler.ExecuteIfBound(MoveTemp(Update));
		}

		return;
	}
	else
	{
		UE_LOGF(LogCPSControlMessenger, Error, "Invalid message arrived");
		return;
	}
}

void FControlMessenger::CommunicationStoppedHandler()
{
	UE_LOGF(LogCPSControlMessenger, Display, "Server disconnected.");
	
	if (KeepAliveTimer.IsValid())
	{
		StopKeepAliveTimer();
	}

	{
		FScopeLock Lock(&SessionIdMutex);
		SessionId = HandshakeSessionId;
	}

	OnDisconnectHandler.ExecuteIfBound(TEXT("Connection ended"));
}

uint32 FControlMessenger::GenerateTransactionId() const
{
	return ++NextTransactionId;
}

uint64 FControlMessenger::GetTimestamp() const
{
	FDateTime Now = FDateTime::UtcNow();
	FDateTime Epoch(1970, 1, 1);

	return static_cast<uint64>((Now - Epoch).GetTotalMilliseconds());
}

void FControlMessenger::StartKeepAliveTimer()
{
	TSharedPtr<FCaptureTimerManager> TimerManager
		= FModuleManager::LoadModuleChecked<FCaptureUtilsModule>(TEXT("CaptureUtils")).GetTimerManager();

	KeepAliveTimer = TimerManager->AddTimer(FTimerDelegate::CreateRaw(this, &FControlMessenger::KeepAlive), KeepAliveInterval, true, KeepAliveInterval);
}

void FControlMessenger::StopKeepAliveTimer()
{
	TSharedPtr<FCaptureTimerManager> TimerManager 
		= FModuleManager::LoadModuleChecked<FCaptureUtilsModule>(TEXT("CaptureUtils")).GetTimerManager();

	TimerManager->RemoveTimer(KeepAliveTimer);
}

void FControlMessenger::OnAsyncRequestProcess(FAsyncRequestDelegate InAsyncDelegate)
{
	InAsyncDelegate.ExecuteIfBound();
}
	
}
