// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Messages/Constants.h"

namespace UE::CaptureManager::CPS::AddressPaths
{
// Requests/Responses
const TCHAR* GKeepAlive = TEXT("/session/keep-alive");
const TCHAR* GStartSession = TEXT("/session/start");
const TCHAR* GStopSession = TEXT("/session/stop");
const TCHAR* GSubscribe = TEXT("/subscribe");
const TCHAR* GUnsubscribe = TEXT("/unsubscribe");
const TCHAR* GGetServerInformation = TEXT("/information");
const TCHAR* GGetState = TEXT("/state");
const TCHAR* GStartRecordingTake = TEXT("/take/record/start");
const TCHAR* GStopRecordingTake = TEXT("/take/record/stop");
const TCHAR* GAbortRecordingTake = TEXT("/take/record/abort");
const TCHAR* GGetTakeList = TEXT("/take/list");
const TCHAR* GGetTakeMetadata = TEXT("/take/metadata");
const TCHAR* GExportTakeData = TEXT("/export/start/take");
const TCHAR* GExportTakeVideoFrame = TEXT("/export/start/take/frame");
const TCHAR* GExportCameraFeedFrame = TEXT("/export/start/camera/frame");
const TCHAR* GPauseExport = TEXT("/export/pause");
const TCHAR* GCancelExport = TEXT("/export/cancel");
const TCHAR* GGetStreamingSubjects = TEXT("/stream/subjects");
const TCHAR* GStartStreaming = TEXT("/stream/start");
const TCHAR* GStopStreaming = TEXT("/stream/stop");

// Updates
const TCHAR* GSessionStopped = TEXT("/event/session-stopped");
const TCHAR* GRecordingStatus = TEXT("/state/recording");
const TCHAR* GTakeAdded = TEXT("/take/added");
const TCHAR* GTakeRemoved = TEXT("/take/removed");
const TCHAR* GTakeUpdated = TEXT("/take/updated");

// Capabilities
const TCHAR* GCapabilitiesRoot = TEXT("/capabilities");
const TCHAR* GCapabilitiesListAll = TEXT("list-all");
const TCHAR* GCapabilityProperties = TEXT("properties");
const TCHAR* GCapabilityPropertySet = TEXT("set");
const TCHAR* GCapabilityPropertyGet = TEXT("get");
const TCHAR* GCapabilityCommands = TEXT("commands");
const TCHAR* GCapabilityCommandExecute = TEXT("execute");
const TCHAR* GCapabilityEvents = TEXT("events");
const TCHAR* GCapabilityEventSubscribe = TEXT("subscribe");
const TCHAR* GCapabilityEventUnsubscribe = TEXT("unsubscribe");

// Platform Specific: IOS
const TCHAR* GDiskCapacity = TEXT("state/ios/disk-capacity");
const TCHAR* GBattery = TEXT("state/ios/battery");
const TCHAR* GThermalState = TEXT("state/ios/thermal");
}

namespace UE::CaptureManager::CPS::Properties
{
// Keys
// Messages
const TCHAR* GSessionId = TEXT("sessionId");
const TCHAR* GAddressPath = TEXT("addressPath");
const TCHAR* GTransactionId = TEXT("transactionId");
const TCHAR* GTimestamp = TEXT("timestamp");
const TCHAR* GType = TEXT("type");
const TCHAR* GBody = TEXT("body");
const TCHAR* GError = TEXT("error");
const TCHAR* GDescription = TEXT("description");

// Requests
const TCHAR* GSlateName = TEXT("slateName");
const TCHAR* GTakeNumber = TEXT("takeNumber");
const TCHAR* GSubject = TEXT("subject");
const TCHAR* GScenario = TEXT("scenario");
const TCHAR* GTags = TEXT("tags");
const TCHAR* GNames = TEXT("names");
const TCHAR* GTakeName = TEXT("takeName");
const TCHAR* GFiles = TEXT("files");
const TCHAR* GName = TEXT("name");
const TCHAR* GOffset = TEXT("offset");
const TCHAR* GStreamPort = TEXT("streamPort");
const TCHAR* GSubjectIds = TEXT("subjectIds");

// Responses
const TCHAR* GId = TEXT("id");
const TCHAR* GModel = TEXT("model");
const TCHAR* GPlatformName = TEXT("platformName");
const TCHAR* GPlatformVersion = TEXT("platformVersion");
const TCHAR* GSoftwareName = TEXT("softwareName");
const TCHAR* GSoftwareVersion = TEXT("softwareVersion");
const TCHAR* GExportPort = TEXT("exportPort");
const TCHAR* GIsRecording = TEXT("isRecording");
const TCHAR* GPlatformState = TEXT("platformState");
const TCHAR* GTakes = TEXT("takes");
const TCHAR* GDateTime = TEXT("dateTime");
const TCHAR* GFrames = TEXT("frames");
const TCHAR* GAppVersion = TEXT("appVersion");
const TCHAR* GLength = TEXT("length");
const TCHAR* GVideo = TEXT("video");
const TCHAR* GFrameRate = TEXT("frameRate");
const TCHAR* GHeight = TEXT("height");
const TCHAR* GWidth = TEXT("width");
const TCHAR* GAudio = TEXT("audio");
const TCHAR* GChannels = TEXT("channels");
const TCHAR* GSampleRate = TEXT("sampleRate");
const TCHAR* GBitsPerChannel = TEXT("bitsPerChannel");
const TCHAR* GSubjects = TEXT("subjects");
const TCHAR* GVersion = TEXT("version");
const TCHAR* GControls = TEXT("controls");
const TCHAR* GAnimationMetadata = TEXT("animationMetadata");
	
// Platform Specific: IOS
const TCHAR* GTotalCapacity = TEXT("totalCapacity");
const TCHAR* GRemainingCapacity = TEXT("remainingCapacity");
const TCHAR* GBatteryLevel = TEXT("batteryLevel");
const TCHAR* GThermalState = TEXT("thermalState");

const TCHAR* GTotal = TEXT("total");
const TCHAR* GRemaining = TEXT("remaining");
const TCHAR* GLevel = TEXT("level");
const TCHAR* GState = TEXT("state");

// Values
// Messages
const TCHAR* GRequest = TEXT("request");
const TCHAR* GResponse = TEXT("response");
const TCHAR* GUpdate = TEXT("update");

const TCHAR* GIOS = TEXT("iOS");
const TCHAR* GAndroid = TEXT("android");
const TCHAR* GWindows = TEXT("windows");
const TCHAR* GWindowsServer = TEXT("windowsServer");
const TCHAR* GLinux = TEXT("linux");
const TCHAR* GMacOS = TEXT("macOS");

// Platform Specific: iOS
const TCHAR* GNominal = TEXT("nominal");
const TCHAR* GFair = TEXT("fair");
const TCHAR* GSerious = TEXT("serious");
const TCHAR* GCritical = TEXT("critical");
}

namespace UE::CaptureManager::CPS::ErrorNames
{
const TCHAR* GUnsupportedProtocolVersion = TEXT("unsupportedProtocolVersion");
const TCHAR* GRequestFailed = TEXT("requestFailed");
const TCHAR* GRequestParsingFailed = TEXT("parsingRequestFailed");
const TCHAR* GNotSupported = TEXT("notSupported");
const TCHAR* GNoSession = TEXT("noSession");
const TCHAR* GMissingBody = TEXT("missingBody");
} // namespace UE::CaptureManager::CPS::ErrorNames

namespace UE::CaptureManager::CPS::Capabilities
{
const TCHAR* GId = TEXT("id");
const TCHAR* GName = TEXT("name");
const TCHAR* GValue = TEXT("value");
const TCHAR* GProperties = TEXT("properties");
const TCHAR* GType = TEXT("type");
const TCHAR* GAccess = TEXT("access");
const TCHAR* GReadOnly = TEXT("readOnly");
const TCHAR* GReadWrite = TEXT("readWrite");
const TCHAR* GEnumOptions = TEXT("enumOptions");
const TCHAR* GCurrentValue = TEXT("currentValue");
const TCHAR* GMin = TEXT("min");
const TCHAR* GMax = TEXT("max");
const TCHAR* GGet = TEXT("get");
const TCHAR* GSet = TEXT("set");
const TCHAR* GCommands = TEXT("commands");
const TCHAR* GParameters = TEXT("parameters");
const TCHAR* GExecute = TEXT("execute");
const TCHAR* GOptional = TEXT("optional");
const TCHAR* GReturnType = TEXT("returnType");
const TCHAR* GDefaultValue = TEXT("defaultValue");
const TCHAR* GEvents = TEXT("events");
const TCHAR* GArguments = TEXT("arguments");
const TCHAR* GSubscribe = TEXT("subscribe");
const TCHAR* GUnsubscribe = TEXT("unsubscribe");
const TCHAR* GBoolean = TEXT("boolean");
const TCHAR* GInteger = TEXT("integer");
const TCHAR* GFloat = TEXT("float");
const TCHAR* GString = TEXT("string");
const TCHAR* GEnumeration = TEXT("enumeration");
} // namespace UE::CaptureManager::CPS::Capabilities
