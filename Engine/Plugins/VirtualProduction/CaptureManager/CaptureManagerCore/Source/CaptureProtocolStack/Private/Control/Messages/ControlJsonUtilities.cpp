// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Messages/ControlJsonUtilities.h"

namespace UE::CaptureManager
{

class FArrayDataWriter final : public FArchive
{
public:

	FArrayDataWriter(TArray<uint8>& InWriterArray)
		: WriterArray(InWriterArray)
	{
	}

	virtual ~FArrayDataWriter() = default;

	virtual void Serialize(void* Data, int64 Num)
	{
		WriterArray.Append(static_cast<uint8*>(Data), Num);
	}

	virtual FString GetArchiveName() const
	{
		return TEXT("FArrayDataWriter");
	}

private:

	TArray<uint8>& WriterArray;
};

class FArrayDataReader final : public FArchive
{
public:

	FArrayDataReader(const TArray<uint8>& InReaderArray)
		: ReaderArray(InReaderArray)
		, ReaderPos(0)
	{
	}

	virtual ~FArrayDataReader() = default;

	virtual void Serialize(void* Data, int64 Num)
	{
		FMemory::Memcpy(Data, ReaderArray.GetData() + ReaderPos, Num);
		ReaderPos += Num;
	}

	virtual int64 Tell()
	{
		return ReaderPos;
	}

	virtual int64 TotalSize()
	{
		return ReaderArray.Num();
	}

	virtual void Seek(int64 InPos)
	{
		ReaderPos = InPos;
	}

	virtual bool AtEnd()
	{
		return ReaderPos >= TotalSize();
	}

	virtual FString GetArchiveName() const
	{
		return TEXT("FArrayDataReader");
	}

private:

	const TArray<uint8>& ReaderArray;
	int64 ReaderPos;
};

bool FJsonUtility::CreateJsonFromUTF8Data(const TArray<uint8>& InData, TSharedPtr<FJsonObject>& OutObject)
{
	FArrayDataReader Reader(InData);

	TSharedRef<TJsonReader<UTF8CHAR>> JsonReader = TJsonReaderFactory<UTF8CHAR>::Create(&Reader);
	return FJsonSerializer::Deserialize(JsonReader, OutObject);
}

bool FJsonUtility::CreateUTF8DataFromJson(TSharedPtr<FJsonObject> InObject, TArray<uint8>& OutData)
{
	FArrayDataWriter Writer(OutData);

	using FJsonWriterCondensedFactory = TJsonWriterFactory<UTF8CHAR, TCondensedJsonPrintPolicy<UTF8CHAR>>;
	using FJsonWriterCondensed = TJsonWriter<UTF8CHAR, TCondensedJsonPrintPolicy<UTF8CHAR>>;

	TSharedRef<FJsonWriterCondensed> JsonWriter = FJsonWriterCondensedFactory::Create(&Writer);
	return FJsonSerializer::Serialize(InObject.ToSharedRef(), JsonWriter);
}

TProtocolResult<void> FJsonUtility::ParseString(const TSharedPtr<FJsonObject>& InBody, const FString& InFieldName, FString& OutFieldValue)
{
	if (!InBody->TryGetStringField(InFieldName, OutFieldValue))
	{
		return FCaptureProtocolError(TEXT("Failed to parse key: ") + InFieldName);
	}

	return ResultOk;
}

TProtocolResult<void> FJsonUtility::ParseBool(const TSharedPtr<FJsonObject>& InBody, const FString& InFieldName, bool& OutFieldValue)
{
	if (!InBody->TryGetBoolField(InFieldName, OutFieldValue))
	{
		return FCaptureProtocolError(TEXT("Failed to parse key: ") + InFieldName);
	}

	return ResultOk;
}

TProtocolResult<void> FJsonUtility::ParseObject(const TSharedPtr<FJsonObject>& InBody, const FString& InFieldName, const TSharedPtr<FJsonObject>*& OutFieldValue)
{
	if (!InBody->TryGetObjectField(InFieldName, OutFieldValue))
	{
		return FCaptureProtocolError(TEXT("Failed to parse key: ") + InFieldName);
	}

	return ResultOk;
}

TProtocolResult<void> FJsonUtility::ParseArray(const TSharedPtr<FJsonObject>& InBody, const FString& InFieldName, const TArray<TSharedPtr<FJsonValue>>*& OutFieldValue)
{
	if (!InBody->TryGetArrayField(InFieldName, OutFieldValue))
	{
		return FCaptureProtocolError(TEXT("Failed to parse key: ") + InFieldName);
	}

	return ResultOk;
}

}