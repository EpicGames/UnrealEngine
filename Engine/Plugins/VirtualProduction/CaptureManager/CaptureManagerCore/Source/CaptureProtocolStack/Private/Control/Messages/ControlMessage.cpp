// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Messages/ControlMessage.h"
#include "Control/Messages/Constants.h"

#include "Control/Messages/ControlJsonUtilities.h"

#include "Utility/Definitions.h"

namespace UE::CaptureManager
{

TProtocolResult<FControlMessage> FControlMessage::Deserialize(const FControlPacket& InPacket)
{
	if (InPacket.GetVersion() != CPS_VERSION)
	{
		return FCaptureProtocolError(TEXT("Version not supported"));
	}

	const TArray<uint8>& PayloadData = InPacket.GetPayload();

	TSharedPtr<FJsonObject> Payload;
	if (!FJsonUtility::CreateJsonFromUTF8Data(PayloadData, Payload))
	{
		return FCaptureProtocolError(TEXT("Failed to parse the data"));
	}

	FString SessionId;
	CHECK_PARSE(FJsonUtility::ParseString(Payload, CPS::Properties::GSessionId, SessionId));

	FString AddressPath;
	CHECK_PARSE(FJsonUtility::ParseString(Payload, CPS::Properties::GAddressPath, AddressPath));

	uint32 TransactionId;
	CHECK_PARSE(FJsonUtility::ParseNumber(Payload, CPS::Properties::GTransactionId, TransactionId));

	uint64 Timestamp;
	CHECK_PARSE(FJsonUtility::ParseNumber(Payload, CPS::Properties::GTimestamp, Timestamp));

	FString MessageTypeStr;
	CHECK_PARSE(FJsonUtility::ParseString(Payload, CPS::Properties::GType, MessageTypeStr));

	FControlMessage::EType MessageType = FControlMessage::DeserializeType(MoveTemp(MessageTypeStr));

	if (MessageType == EType::Invalid)
	{
		return FCaptureProtocolError(TEXT("Invalid message type"));
	}

	// Optionals
	const TSharedPtr<FJsonObject>* Body;
	TProtocolResult<void> ParseBodyResult = FJsonUtility::ParseObject(Payload, CPS::Properties::GBody, Body);

	const TSharedPtr<FJsonObject>* Error;
	TProtocolResult<void> ParseErrorResult = FJsonUtility::ParseObject(Payload, CPS::Properties::GError, Error);

	FErrorResponse ErrorStruct;
	if (ParseErrorResult.HasValue())
	{
		CHECK_PARSE(FJsonUtility::ParseString(*Error, CPS::Properties::GName, ErrorStruct.Name));
		CHECK_PARSE(FJsonUtility::ParseString(*Error, CPS::Properties::GDescription, ErrorStruct.Description));
	}

	return FControlMessage(MoveTemp(SessionId),
						   MoveTemp(AddressPath),
						   TransactionId,
						   Timestamp,
						   MessageType,
						   ParseBodyResult.HasValue() ? *Body : nullptr,
						   MoveTemp(ErrorStruct));
}

TProtocolResult<FControlPacket> FControlMessage::Serialize(const FControlMessage& InMessage)
{
	TSharedPtr<FJsonObject> Payload = MakeShared<FJsonObject>();

	Payload->SetStringField(CPS::Properties::GSessionId, InMessage.SessionId);
	Payload->SetStringField(CPS::Properties::GAddressPath, InMessage.AddressPath);
	Payload->SetNumberField(CPS::Properties::GTransactionId, InMessage.TransactionId);
	Payload->SetNumberField(CPS::Properties::GTimestamp, InMessage.Timestamp);

	FString MessageTypeStr = SerializeType(InMessage.MessageType);
	Payload->SetStringField(CPS::Properties::GType, MessageTypeStr);

	if (InMessage.Body)
	{
		Payload->SetObjectField(CPS::Properties::GBody, InMessage.Body);
	}

	if (!InMessage.Error.Name.IsEmpty())
	{
		TSharedPtr<FJsonObject> Error = MakeShared<FJsonObject>();
		Error->SetStringField(CPS::Properties::GName, InMessage.Error.Name);
		Error->SetStringField(CPS::Properties::GDescription, InMessage.Error.Description);

		Payload->SetObjectField(CPS::Properties::GError, Error);
	}

	TArray<uint8> Data;
	if (!FJsonUtility::CreateUTF8DataFromJson(Payload, Data))
	{
		return FCaptureProtocolError(TEXT("Failed to serialize the payload"));
	}

	return FControlPacket(CPS_VERSION, MoveTemp(Data));
}

FControlMessage::FControlMessage(FString InSessionId,
								 FString InAddressPath,
								 uint32 InTransactionId,
								 uint64 InTimestamp,
								 EType InMessageType,
								 TSharedPtr<FJsonObject> InBody,
								 FErrorResponse InError)
	: SessionId(MoveTemp(InSessionId))
	, AddressPath(MoveTemp(InAddressPath))
	, TransactionId(InTransactionId)
	, Timestamp(InTimestamp)
	, MessageType(InMessageType)
	, Body(MoveTemp(InBody))
	, Error(MoveTemp(InError))
{
}

FControlMessage::FControlMessage(FString InAddressPath,
								 EType InType,
								 TSharedPtr<FJsonObject> InBody)
	: AddressPath(MoveTemp(InAddressPath))
	, MessageType(InType)
	, Body(MoveTemp(InBody))
{
}

void FControlMessage::SetSessionId(FString InSessionId)
{
	SessionId = MoveTemp(InSessionId);
}

void FControlMessage::SetTransactionId(uint32 InTransactionId)
{
	TransactionId = InTransactionId;
}

void FControlMessage::SetTimestamp(uint64 InTimestamp)
{
	Timestamp = InTimestamp;
}

const FString& FControlMessage::GetSessionId() const
{
	return SessionId;
}

const FString& FControlMessage::GetAddressPath() const
{
	return AddressPath;
}

uint32 FControlMessage::GetTransactionId() const
{
	return TransactionId;
}

uint64 FControlMessage::GetTimestamp() const
{
	return Timestamp;
}

FControlMessage::EType FControlMessage::GetType() const
{
	return MessageType;
}

const TSharedPtr<FJsonObject>& FControlMessage::GetBody() const
{
	return Body;
}

TSharedPtr<FJsonObject>& FControlMessage::GetBody()
{
	return Body;
}

const FString& FControlMessage::GetErrorName() const
{
	return Error.Name;
}

const FString& FControlMessage::GetErrorDescription() const
{
	return Error.Description;
}

FControlMessage::EType FControlMessage::DeserializeType(FString InMessageTypeStr)
{
	if (InMessageTypeStr == CPS::Properties::GRequest)
	{
		return EType::Request;
	}
	else if (InMessageTypeStr == CPS::Properties::GResponse)
	{
		return EType::Response;
	}
	else if (InMessageTypeStr == CPS::Properties::GUpdate)
	{
		return EType::Update;
	}
	else
	{
		return EType::Invalid;
	}
}

FString FControlMessage::SerializeType(EType InMessageType)
{
	switch (InMessageType)
	{
		case EType::Request:
			return CPS::Properties::GRequest;
		case EType::Response:
			return CPS::Properties::GResponse;
		case EType::Update:
			return CPS::Properties::GUpdate;
		default:
			return TEXT("invalid");
	}
}

}