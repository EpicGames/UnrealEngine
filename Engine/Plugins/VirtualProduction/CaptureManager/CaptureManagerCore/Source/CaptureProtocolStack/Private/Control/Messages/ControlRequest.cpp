// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Messages/ControlRequest.h"
#include "Control/Messages/Constants.h"
#include "Control/Messages/ProtocolConstants.h"

#include "Misc/Paths.h"
#include "Control/Capabilities/Utils.h"

namespace UE::CaptureManager
{

FControlRequest::FControlRequest(FString InAddressPath)
	: AddressPath(MoveTemp(InAddressPath))
{
}

const FString& FControlRequest::GetAddressPath() const
{
	return AddressPath;
}

TSharedPtr<FJsonObject> FControlRequest::GetBody() const
{
	return nullptr;
}

FKeepAliveRequest::FKeepAliveRequest()
	: FControlRequest(CPS::AddressPaths::GKeepAlive)
{
}

FStartSessionRequest::FStartSessionRequest()
	: FControlRequest(CPS::AddressPaths::GStartSession)
{
}

FStopSessionRequest::FStopSessionRequest()
	: FControlRequest(CPS::AddressPaths::GStopSession)
{
}

FGetServerInformationRequest::FGetServerInformationRequest()
	: FControlRequest(CPS::AddressPaths::GGetServerInformation)
{
}

FSubscribeRequest::FSubscribeRequest()
	: FControlRequest(CPS::AddressPaths::GSubscribe)
{
}

FUnsubscribeRequest::FUnsubscribeRequest()
	: FControlRequest(CPS::AddressPaths::GUnsubscribe)
{
}

FGetStateRequest::FGetStateRequest()
	: FControlRequest(CPS::AddressPaths::GGetState)
{
}

FStartRecordingTakeRequest::FStartRecordingTakeRequest(FString InSlateName,
													   uint16 InTakeNumber,
													   TOptional<FString> InSubject,
													   TOptional<FString> InScenario,
													   TOptional<TArray<FString>> InTags)
	: FControlRequest(CPS::AddressPaths::GStartRecordingTake)
	, SlateName(MoveTemp(InSlateName))
	, TakeNumber(InTakeNumber)
	, Subject(MoveTemp(InSubject))
	, Scenario(MoveTemp(InScenario))
	, Tags(MoveTemp(InTags))
{
}

TSharedPtr<FJsonObject> FStartRecordingTakeRequest::GetBody() const
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();

	Body->SetStringField(CPS::Properties::GSlateName, SlateName);
	Body->SetNumberField(CPS::Properties::GTakeNumber, TakeNumber);
	if (Subject.IsSet())
	{
		Body->SetStringField(CPS::Properties::GSubject, Subject.GetValue());
	}

	if (Scenario.IsSet())
	{
		Body->SetStringField(CPS::Properties::GScenario, Scenario.GetValue());
	}

	if (Tags.IsSet())
	{
		TArray<TSharedPtr<FJsonValue>> TagsJson;
		for (const FString& Tag : Tags.GetValue())
		{
			TagsJson.Add(MakeShared<FJsonValueString>(Tag));
		}

		Body->SetArrayField(CPS::Properties::GTags, TagsJson);
	}

	return Body;
}

FStopRecordingTakeRequest::FStopRecordingTakeRequest()
	: FControlRequest(CPS::AddressPaths::GStopRecordingTake)
{
}

FAbortRecordingTakeRequest::FAbortRecordingTakeRequest()
	: FControlRequest(CPS::AddressPaths::GAbortRecordingTake)
{
}

FGetTakeListRequest::FGetTakeListRequest()
	: FControlRequest(CPS::AddressPaths::GGetTakeList)
{
}

FGetTakeMetadataRequest::FGetTakeMetadataRequest(TArray<FString> InNames)
	: FControlRequest(CPS::AddressPaths::GGetTakeMetadata)
	, Names(MoveTemp(InNames))
{
}

TSharedPtr<FJsonObject> FGetTakeMetadataRequest::GetBody() const
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();

	TArray<TSharedPtr<FJsonValue>> NamesJson;
	for (const FString& Name : Names)
	{
		NamesJson.Add(MakeShared<FJsonValueString>(Name));
	}

	Body->SetArrayField(CPS::Properties::GNames, NamesJson);

	return Body;
}

FGetStreamingSubjectsRequest::FGetStreamingSubjectsRequest()
	: FControlRequest(CPS::AddressPaths::GGetStreamingSubjects)
{
}
	
FStartStreamingRequest::FStartStreamingRequest(uint16 InStreamPort, TArray<FSubject> InSubjects)
	: FControlRequest(CPS::AddressPaths::GStartStreaming)
	, StreamPort(InStreamPort)
	, Subjects(MoveTemp(InSubjects))
{
}

TSharedPtr<FJsonObject> FStartStreamingRequest::GetBody() const
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();
	Body->SetNumberField(CPS::Properties::GStreamPort, StreamPort);

	TArray<TSharedPtr<FJsonValue>> SubjectsJson;
	for (const FSubject& Subject : Subjects)
	{
		TSharedPtr<FJsonObject> SubjectObject = MakeShared<FJsonObject>();
		SubjectObject->SetField(CPS::Properties::GId, MakeShared<FJsonValueString>(Subject.Id));
		if (Subject.Name.IsSet())
		{
			SubjectObject->SetField(CPS::Properties::GName, MakeShared<FJsonValueString>(Subject.Name.GetValue()));
		}
		SubjectsJson.Add(MakeShared<FJsonValueObject>(SubjectObject));
	}
	Body->SetArrayField(CPS::Properties::GSubjects, SubjectsJson);
	
	return Body;
}
	
FStopStreamingRequest::FStopStreamingRequest(TOptional<TArray<FString>> InSubjectIds)
	: FControlRequest(CPS::AddressPaths::GStopStreaming)
	, SubjectIds(MoveTemp(InSubjectIds))
{
}

TSharedPtr<FJsonObject> FStopStreamingRequest::GetBody() const
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();

	if (SubjectIds.IsSet())
	{
		TArray<TSharedPtr<FJsonValue>> SubjectIdsJson;
		const TArray<FString>& SubjectIdsArray = SubjectIds.GetValue();
		for (const FString& SubjectId : SubjectIdsArray)
		{
			SubjectIdsJson.Add(MakeShared<FJsonValueString>(SubjectId));
		}
		Body->SetArrayField(CPS::Properties::GSubjectIds, SubjectIdsJson);
	}
	
	return Body;
}

FListAllCapabilitiesRequest::FListAllCapabilitiesRequest()
	: FControlRequest(FPaths::Combine(CPS::AddressPaths::GCapabilitiesRoot, CPS::AddressPaths::GCapabilitiesListAll))
{
}

FGetCapabilityPropertyRequest::FGetCapabilityPropertyRequest(FString InCapabilityId, FString InPropertyId)
	: FControlRequest(FPaths::Combine(CPS::AddressPaths::GCapabilitiesRoot,
									  MoveTemp(InCapabilityId),
									  CPS::AddressPaths::GCapabilityProperties,
									  MoveTemp(InPropertyId),
									  CPS::AddressPaths::GCapabilityPropertyGet))
{
}

FSetCapabilityPropertyRequest::FSetCapabilityPropertyRequest(FString InCapabilityId,
															 FString InPropertyId,
															 FCapabilityValue InValue)
	: FControlRequest(FPaths::Combine(CPS::AddressPaths::GCapabilitiesRoot,
									  MoveTemp(InCapabilityId),
									  CPS::AddressPaths::GCapabilityProperties,
									  MoveTemp(InPropertyId),
									  CPS::AddressPaths::GCapabilityPropertySet))
{
	Value = MoveTemp(InValue);
}

TSharedPtr<FJsonObject> FSetCapabilityPropertyRequest::GetBody() const
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();
	FCapabilityUtilities::SetProperJsonFieldType(CPS::Capabilities::GValue, Value, Body);
	return Body;
}

FExecuteCapabilityCommandRequest::FExecuteCapabilityCommandRequest(FString InCapabilityId,
																   FString InCommandId,
																   FCapabilityValues InArguments)
	: FControlRequest(FPaths::Combine(CPS::AddressPaths::GCapabilitiesRoot,
									  MoveTemp(InCapabilityId),
									  CPS::AddressPaths::GCapabilityCommands,
									  MoveTemp(InCommandId),
									  CPS::AddressPaths::GCapabilityCommandExecute))
{
	Arguments = MoveTemp(InArguments);
}

TSharedPtr<FJsonObject> FExecuteCapabilityCommandRequest::GetBody() const
{
	if (Arguments.Num() > 0)
	{
		TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();
		TSharedPtr<FJsonObject> JsonArguments = MakeShared<FJsonObject>();
		for (const TPair<FString, FCapabilityValue>& Argument : Arguments)
		{
			FCapabilityUtilities::SetProperJsonFieldType(Argument.Key, Argument.Value, JsonArguments);
		}
		Body->SetObjectField(CPS::Capabilities::GArguments, JsonArguments);
		return Body;
	}
	return nullptr;
}

FCapabilitySubscribeRequest::FCapabilitySubscribeRequest(FString InCapabilityId)
	: FControlRequest(FPaths::Combine(CPS::AddressPaths::GCapabilitiesRoot,
									  InCapabilityId,
									  CPS::AddressPaths::GCapabilityEvents,
									  CPS::AddressPaths::GCapabilityEventSubscribe))
{
}

FCapabilityUnsubscribeRequest::FCapabilityUnsubscribeRequest(FString InCapabilityId)
	: FControlRequest(FPaths::Combine(CPS::AddressPaths::GCapabilitiesRoot,
									  InCapabilityId,
									  CPS::AddressPaths::GCapabilityEvents,
									  CPS::AddressPaths::GCapabilityEventUnsubscribe))
{
}

} // namespace UE::CaptureManager
