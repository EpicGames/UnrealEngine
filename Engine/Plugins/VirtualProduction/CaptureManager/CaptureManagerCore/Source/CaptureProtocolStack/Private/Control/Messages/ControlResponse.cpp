// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Messages/ControlResponse.h"
#include "Control/Messages/Constants.h"
#include "Control/Messages/ProtocolConstants.h"

#include "Control/Messages/ControlJsonUtilities.h"
#include "Control/Capabilities/Utils.h"

#include "Misc/Paths.h"

namespace UE::CaptureManager
{

FControlResponse::FControlResponse(FString InAddressPath)
	: AddressPath(MoveTemp(InAddressPath))
{
}

TProtocolResult<void> FControlResponse::Parse(TSharedPtr<FJsonObject> InBody)
{
	if (InBody.IsValid() && !InBody->Values.IsEmpty())
	{
		return FCaptureProtocolError("Response must NOT have a body");
	}

	return ResultOk;
}

void FControlResponse::SetAddressPath(const FString& InAddressPath)
{
	AddressPath = InAddressPath;
}

const FString& FControlResponse::GetAddressPath() const
{
	return AddressPath;
}

FKeepAliveResponse::FKeepAliveResponse()
	: FControlResponse(CPS::AddressPaths::GKeepAlive)
{
}

FStartSessionResponse::FStartSessionResponse()
	: FControlResponse(CPS::AddressPaths::GStartSession)
{
}

TProtocolResult<void> FStartSessionResponse::Parse(TSharedPtr<FJsonObject> InBody)
{
	CHECK_PARSE(FJsonUtility::ParseString(InBody, CPS::Properties::GSessionId, SessionId));

	return ResultOk;
}

const FString& FStartSessionResponse::GetSessionId() const
{
	return SessionId;
}

FStopSessionResponse::FStopSessionResponse()
	: FControlResponse(CPS::AddressPaths::GStopSession)
{
}

FGetServerInformationResponse::FGetServerInformationResponse()
	: FControlResponse(CPS::AddressPaths::GGetServerInformation)
{
}

TProtocolResult<void> FGetServerInformationResponse::Parse(TSharedPtr<FJsonObject> InBody)
{
	CHECK_PARSE(FJsonUtility::ParseString(InBody, CPS::Properties::GId, Id));
	CHECK_PARSE(FJsonUtility::ParseString(InBody, CPS::Properties::GName, Name));
	CHECK_PARSE(FJsonUtility::ParseString(InBody, CPS::Properties::GModel, Model));
	CHECK_PARSE(FJsonUtility::ParseString(InBody, CPS::Properties::GPlatformName, PlatformName));
	CHECK_PARSE(FJsonUtility::ParseString(InBody, CPS::Properties::GPlatformVersion, PlatformVersion));
	CHECK_PARSE(FJsonUtility::ParseString(InBody, CPS::Properties::GSoftwareName, SoftwareName));
	CHECK_PARSE(FJsonUtility::ParseString(InBody, CPS::Properties::GSoftwareVersion, SoftwareVersion));
	CHECK_PARSE(FJsonUtility::ParseNumber(InBody, CPS::Properties::GExportPort, ExportPort));

	return ResultOk;
}

const FString& FGetServerInformationResponse::GetId() const
{
	return Id;
}

const FString& FGetServerInformationResponse::GetName() const
{
	return Name;
}

const FString& FGetServerInformationResponse::GetModel() const
{
	return Model;
}

const FString& FGetServerInformationResponse::GetPlatformName() const
{
	return PlatformName;
}

const FString& FGetServerInformationResponse::GetPlatformVersion() const
{
	return PlatformVersion;
}

const FString& FGetServerInformationResponse::GetSoftwareName() const
{
	return SoftwareName;
}

const FString& FGetServerInformationResponse::GetSoftwareVersion() const
{
	return SoftwareVersion;
}

uint16 FGetServerInformationResponse::GetExportPort() const
{
	return ExportPort;
}

FSubscribeResponse::FSubscribeResponse()
	: FControlResponse(CPS::AddressPaths::GSubscribe)
{
}

FUnsubscribeResponse::FUnsubscribeResponse()
	: FControlResponse(CPS::AddressPaths::GUnsubscribe)
{
}

FGetStateResponse::FGetStateResponse()
	: FControlResponse(CPS::AddressPaths::GGetState)
{
}

TProtocolResult<void> FGetStateResponse::Parse(TSharedPtr<FJsonObject> InBody)
{
	CHECK_PARSE(FJsonUtility::ParseBool(InBody, CPS::Properties::GIsRecording, bIsRecording));

	// Optional
	const TSharedPtr<FJsonObject>* PlatformStatePtr;
	TProtocolResult<void> Result = FJsonUtility::ParseObject(InBody, CPS::Properties::GPlatformState, PlatformStatePtr);
	if (Result.HasValue())
	{
		PlatformState = *PlatformStatePtr;
	}

	return ResultOk;
}

bool FGetStateResponse::IsRecording() const
{
	return bIsRecording;
}

const TSharedPtr<FJsonObject>& FGetStateResponse::GetPlatformState() const
{
	return PlatformState;
}

FStartRecordingTakeResponse::FStartRecordingTakeResponse()
	: FControlResponse(CPS::AddressPaths::GStartRecordingTake)
{
}

FStopRecordingTakeResponse::FStopRecordingTakeResponse()
	: FControlResponse(CPS::AddressPaths::GStopRecordingTake)
{
}

FAbortRecordingTakeResponse::FAbortRecordingTakeResponse()
	: FControlResponse(CPS::AddressPaths::GAbortRecordingTake)
{
}

TProtocolResult<void> FStopRecordingTakeResponse::Parse(TSharedPtr<FJsonObject> InBody)
{
	CHECK_PARSE(FJsonUtility::ParseString(InBody, CPS::Properties::GName, TakeName));

	return ResultOk;
}

const FString& FStopRecordingTakeResponse::GetTakeName() const
{
	return TakeName;
}

FGetTakeListResponse::FGetTakeListResponse()
	: FControlResponse(CPS::AddressPaths::GGetTakeList)
{
}

TProtocolResult<void> FGetTakeListResponse::Parse(TSharedPtr<FJsonObject> InBody)
{
	const TArray<TSharedPtr<FJsonValue>>* NamesJson;
	CHECK_PARSE(FJsonUtility::ParseArray(InBody, CPS::Properties::GNames, NamesJson));

	for (const TSharedPtr<FJsonValue>& NameJson : *NamesJson)
	{
		Names.Add(NameJson->AsString());
	}

	return ResultOk;
}

const TArray<FString>& FGetTakeListResponse::GetNames() const
{
	return Names;
}

FGetTakeMetadataResponse::FGetTakeMetadataResponse()
	: FControlResponse(CPS::AddressPaths::GGetTakeMetadata)
{
}

TProtocolResult<void> FGetTakeMetadataResponse::Parse(TSharedPtr<FJsonObject> InBody)
{
	const TArray<TSharedPtr<FJsonValue>>* TakesJson;
	CHECK_PARSE(FJsonUtility::ParseArray(InBody, CPS::Properties::GTakes, TakesJson));

	for (const TSharedPtr<FJsonValue>& TakeJson : *TakesJson)
	{
		const TSharedPtr<FJsonObject>& TakeJsonObject = TakeJson->AsObject();

		FTakeObject Take;
		CHECK_PARSE(CreateTakeObject(TakeJsonObject, Take));
		Takes.Add(MoveTemp(Take));
	}

	return ResultOk;
}

const TArray<FGetTakeMetadataResponse::FTakeObject>& FGetTakeMetadataResponse::GetTakes() const
{
	return Takes;
}

TProtocolResult<void> FGetTakeMetadataResponse::CreateTakeObject(const TSharedPtr<FJsonObject>& InTakeObject, FTakeObject& OutTake) const
{
	CHECK_PARSE(FJsonUtility::ParseString(InTakeObject, CPS::Properties::GName, OutTake.Name));
	CHECK_PARSE(FJsonUtility::ParseString(InTakeObject, CPS::Properties::GSlateName, OutTake.Slate));
	CHECK_PARSE(FJsonUtility::ParseNumber(InTakeObject, CPS::Properties::GTakeNumber, OutTake.TakeNumber));
	CHECK_PARSE(FJsonUtility::ParseString(InTakeObject, CPS::Properties::GDateTime, OutTake.DateTime));
	CHECK_PARSE(FJsonUtility::ParseString(InTakeObject, CPS::Properties::GAppVersion, OutTake.AppVersion));
	CHECK_PARSE(FJsonUtility::ParseString(InTakeObject, CPS::Properties::GModel, OutTake.Model));

	// Optional
	FJsonUtility::ParseString(InTakeObject, CPS::Properties::GSubject, OutTake.Subject);
	FJsonUtility::ParseString(InTakeObject, CPS::Properties::GScenario, OutTake.Scenario);

	const TArray<TSharedPtr<FJsonValue>>* TagsJson;
	TProtocolResult<void> ParseArrayResult = FJsonUtility::ParseArray(InTakeObject, CPS::Properties::GTags, TagsJson);

	if (ParseArrayResult.HasValue())
	{
		for (const TSharedPtr<FJsonValue>& TagJson : *TagsJson)
		{
			OutTake.Tags.Add(TagJson->AsString());
		}
	}

	const TArray<TSharedPtr<FJsonValue>>* FilesJson;
	CHECK_PARSE(FJsonUtility::ParseArray(InTakeObject, CPS::Properties::GFiles, FilesJson));

	for (const TSharedPtr<FJsonValue>& FileJson : *FilesJson)
	{
		const TSharedPtr<FJsonObject>& FileJsonObject = FileJson->AsObject();
		FFileObject File;
		CHECK_PARSE(CreateFileObject(FileJsonObject, File));
		OutTake.Files.Add(MoveTemp(File));
	}

	const TSharedPtr<FJsonObject>* VideoMetadata;
	TProtocolResult<void> ParseVideoObject = FJsonUtility::ParseObject(InTakeObject, CPS::Properties::GVideo, VideoMetadata);
	if (ParseVideoObject.HasValue())
	{
		CHECK_PARSE(FJsonUtility::ParseNumber(*VideoMetadata, CPS::Properties::GFrames, OutTake.Video.Frames));
		CHECK_PARSE(FJsonUtility::ParseNumber(*VideoMetadata, CPS::Properties::GFrameRate, OutTake.Video.FrameRate));
		CHECK_PARSE(FJsonUtility::ParseNumber(*VideoMetadata, CPS::Properties::GHeight, OutTake.Video.Height));
		CHECK_PARSE(FJsonUtility::ParseNumber(*VideoMetadata, CPS::Properties::GWidth, OutTake.Video.Width));
	}

	const TSharedPtr<FJsonObject>* AudioMetadata;
	TProtocolResult<void> ParseAudioObject = FJsonUtility::ParseObject(InTakeObject, CPS::Properties::GAudio, AudioMetadata);
	if (ParseAudioObject.HasValue())
	{
		CHECK_PARSE(FJsonUtility::ParseNumber(*AudioMetadata, CPS::Properties::GChannels, OutTake.Audio.Channels));
		CHECK_PARSE(FJsonUtility::ParseNumber(*AudioMetadata, CPS::Properties::GSampleRate, OutTake.Audio.SampleRate));
		CHECK_PARSE(FJsonUtility::ParseNumber(*AudioMetadata, CPS::Properties::GBitsPerChannel, OutTake.Audio.BitsPerChannel));
	}

	return ResultOk;
}

TProtocolResult<void> FGetTakeMetadataResponse::CreateFileObject(const TSharedPtr<FJsonObject>& InFileObject, FFileObject& OutFile) const
{
	CHECK_PARSE(FJsonUtility::ParseString(InFileObject, CPS::Properties::GName, OutFile.Name));
	CHECK_PARSE(FJsonUtility::ParseNumber(InFileObject, CPS::Properties::GLength, OutFile.Length));

	return ResultOk;
}

FGetStreamingSubjectsResponse::FGetStreamingSubjectsResponse()
	: FControlResponse(CPS::AddressPaths::GGetStreamingSubjects)
{
	
}

TProtocolResult<void> FGetStreamingSubjectsResponse::Parse(TSharedPtr<FJsonObject> InBody)
{
	const TArray<TSharedPtr<FJsonValue>>* SubjectsJson;
	CHECK_PARSE(FJsonUtility::ParseArray(InBody, CPS::Properties::GSubjects, SubjectsJson));

	for (const TSharedPtr<FJsonValue>& SubjectJson : *SubjectsJson)
	{
		const TSharedPtr<FJsonObject>& SubjectJsonObject = SubjectJson->AsObject();

		FSubject Subject;
		CHECK_PARSE(CreateSubject(SubjectJsonObject, Subject));
		Subjects.Add(MoveTemp(Subject));
	}

	return ResultOk;
}

const TArray<FGetStreamingSubjectsResponse::FSubject>& FGetStreamingSubjectsResponse::GetSubjects() const
{
	return Subjects;
}

TProtocolResult<void> FGetStreamingSubjectsResponse::CreateSubject(const TSharedPtr<FJsonObject>& InSubjectObject, FSubject& OutSubject) const
{
	CHECK_PARSE(FJsonUtility::ParseString(InSubjectObject, CPS::Properties::GId, OutSubject.Id))
	CHECK_PARSE(FJsonUtility::ParseString(InSubjectObject, CPS::Properties::GName, OutSubject.Name))

	const TSharedPtr<FJsonObject>* AnimationMetadataObject;
	CHECK_PARSE(FJsonUtility::ParseObject(InSubjectObject, CPS::Properties::GAnimationMetadata, AnimationMetadataObject));
	
	FAnimationMetadata AnimationMetadata;
	CHECK_PARSE(CreateAnimationMetadata(*AnimationMetadataObject, AnimationMetadata))
	OutSubject.AnimationMetadata = MoveTemp(AnimationMetadata);

	return ResultOk;
}

TProtocolResult<void> FGetStreamingSubjectsResponse::CreateAnimationMetadata(const TSharedPtr<FJsonObject>& InAnimationObject, FAnimationMetadata& OutAnimation) const
{
	CHECK_PARSE(FJsonUtility::ParseString(InAnimationObject, CPS::Properties::GType, OutAnimation.Type))
	CHECK_PARSE(FJsonUtility::ParseNumber(InAnimationObject, CPS::Properties::GVersion, OutAnimation.Version))
	
	const TArray<TSharedPtr<FJsonValue>>* ControlsJson;
	CHECK_PARSE(FJsonUtility::ParseArray(InAnimationObject, CPS::Properties::GControls, ControlsJson))
	for (const TSharedPtr<FJsonValue>& ControlJson : *ControlsJson)
	{
		OutAnimation.Controls.Add(ControlJson->AsString());
	}

	return ResultOk;
}

FStartStreamingResponse::FStartStreamingResponse()
	: FControlResponse(CPS::AddressPaths::GStartStreaming)
{
}

FStopStreamingResponse::FStopStreamingResponse()
	: FControlResponse(CPS::AddressPaths::GStopStreaming)
{
}

FListAllCapabilitiesResponse::FListAllCapabilitiesResponse()
	: FControlResponse(FPaths::Combine(CPS::AddressPaths::GCapabilitiesRoot, CPS::AddressPaths::GCapabilitiesListAll))
{
}

TProtocolResult<void> FListAllCapabilitiesResponse::Parse(TSharedPtr<FJsonObject> InBody)
{
	if (!InBody.IsValid())
	{
		return FCaptureProtocolError(TEXT("Body not found in capabilities list all response."));
	}

	const TArray<TSharedPtr<FJsonValue>>* CapabilitiesPtr;
	CHECK_PARSE(FJsonUtility::ParseArray(InBody, CPS::Capabilities::GValue, CapabilitiesPtr));

	for (const TSharedPtr<FJsonValue>& JsonValue : *CapabilitiesPtr)
	{
		TSharedPtr<FJsonObject> JsonObject = JsonValue->AsObject();

		TProtocolResult<FCapability> CapabilityResult = ParseCapability(JsonObject);
		if (CapabilityResult.IsValid())
		{
			Capabilities.Add(MoveTemp(CapabilityResult.GetValue()));
		}
		else
		{
			return CapabilityResult.GetError();
		}
	}

	return ResultOk;
}

static TProtocolResult<void> CpsAccessStringToEnum(const FString& InAccessString, ECapabilityAccess::Type& OutType)
{
	if (InAccessString == CPS::Capabilities::GReadOnly)
	{
		OutType = ECapabilityAccess::Type::ReadOnly;
	}
	else if (InAccessString == CPS::Capabilities::GReadWrite)
	{
		OutType = ECapabilityAccess::Type::ReadWrite;
	}
	else
	{
		return FCaptureProtocolError(FString::Format(TEXT("Invalid access type value: {0}"), { InAccessString }));
	}
	return ResultOk;
}

TProtocolResult<FCapability> FListAllCapabilitiesResponse::ParseCapability(const TSharedPtr<FJsonObject>& InJsonObject)
{
	FCapability Capability;

	CHECK_PARSE(FJsonUtility::ParseString(InJsonObject, CPS::Capabilities::GId, Capability.Id));
	CHECK_PARSE(FJsonUtility::ParseString(InJsonObject, CPS::Capabilities::GName, Capability.Name));

	const TArray<TSharedPtr<FJsonValue>>* JsonProperties;
	CHECK_PARSE(FJsonUtility::ParseArray(InJsonObject, CPS::Capabilities::GProperties, JsonProperties));

	for (const TSharedPtr<FJsonValue>& Value : *JsonProperties)
	{
		TSharedPtr<FJsonObject> JsonProperty = Value->AsObject();
		FCapabilityProperty Property;

		CHECK_PARSE(FJsonUtility::ParseString(JsonProperty, CPS::Capabilities::GId, Property.Id));
		CHECK_PARSE(FJsonUtility::ParseString(JsonProperty, CPS::Capabilities::GName, Property.Name));

		FString PropertyTypeString;
		CHECK_PARSE(FJsonUtility::ParseString(JsonProperty, CPS::Capabilities::GType, PropertyTypeString));
		CHECK_PARSE(FCapabilityUtilities::TypeStringToEnum(PropertyTypeString, Property.Type));

		FString PropertyAccessString;
		CHECK_PARSE(FJsonUtility::ParseString(JsonProperty, CPS::Capabilities::GAccess, PropertyAccessString));
		CHECK_PARSE(CpsAccessStringToEnum(PropertyAccessString, Property.Access));

		if (JsonProperty->HasField(CPS::Capabilities::GMin))
		{
			TVariant<int32, float> MinVal;

			float MinLimit;
			CHECK_PARSE(FJsonUtility::ParseNumber(JsonProperty, CPS::Capabilities::GMin, MinLimit));

			if (Property.Type == ECapabilityValue::Type::Integer)
			{
				MinVal.Set<int32>(MinLimit);
			}
			else
			{
				MinVal.Set<float>(MinLimit);
			}
			Property.Min.Emplace(MoveTemp(MinVal));
		}

		if (JsonProperty->HasField(CPS::Capabilities::GMax))
		{
			TVariant<int32, float> MaxVal;

			float MaxLimit;
			CHECK_PARSE(FJsonUtility::ParseNumber(JsonProperty, CPS::Capabilities::GMax, MaxLimit));

			if (Property.Type == ECapabilityValue::Type::Integer)
			{
				MaxVal.Set<int32>(MaxLimit);
			}
			else
			{
				MaxVal.Set<float>(MaxLimit);
			}
			Property.Max.Emplace(MoveTemp(MaxVal));
		}

		if (Property.Type == ECapabilityValue::Type::Enum)
		{
			TArray<FString> EnumOptions;
			const TArray<TSharedPtr<FJsonValue>>* JsonEnumOptions;
			CHECK_PARSE(FJsonUtility::ParseArray(JsonProperty, CPS::Capabilities::GEnumOptions, JsonEnumOptions));

			for (const TSharedPtr<FJsonValue>& EnumOption : *JsonEnumOptions)
			{
				EnumOptions.Add(EnumOption->AsString());
			}

			Property.EnumOptions.Emplace(MoveTemp(EnumOptions));
		}

		CHECK_PARSE(FCapabilityUtilities::ValueFromJsonObject(
			JsonProperty, CPS::Capabilities::GCurrentValue, Property.Type, Property.CurrentValue));

		Capability.Properties.Add(Property.Id, MoveTemp(Property));
	}

	const TArray<TSharedPtr<FJsonValue>>* JsonCommands;
	CHECK_PARSE(FJsonUtility::ParseArray(InJsonObject, CPS::Capabilities::GCommands, JsonCommands));

	for (const TSharedPtr<FJsonValue>& Value : *JsonCommands)
	{
		TSharedPtr<FJsonObject> JsonCommand = Value->AsObject();
		FCapabilityCommand Command;

		CHECK_PARSE(FJsonUtility::ParseString(JsonCommand, CPS::Capabilities::GId, Command.Id));
		CHECK_PARSE(FJsonUtility::ParseString(JsonCommand, CPS::Capabilities::GName, Command.Name));

		const TArray<TSharedPtr<FJsonValue>>* JsonParameters;
		CHECK_PARSE(FJsonUtility::ParseArray(JsonCommand, CPS::Capabilities::GParameters, JsonParameters));

		for (const TSharedPtr<FJsonValue>& ParameterValue : *JsonParameters)
		{
			TSharedPtr<FJsonObject> JsonParameter = ParameterValue->AsObject();
			FCommandParameterDescriptor ParameterDescriptor;

			CHECK_PARSE(FJsonUtility::ParseString(JsonParameter, CPS::Capabilities::GName, ParameterDescriptor.Name));

			FString Type;
			CHECK_PARSE(FJsonUtility::ParseString(JsonParameter, CPS::Capabilities::GType, Type));
			CHECK_PARSE(FCapabilityUtilities::TypeStringToEnum(Type, ParameterDescriptor.Type));

			CHECK_PARSE(FJsonUtility::ParseBool(JsonParameter, CPS::Capabilities::GOptional, ParameterDescriptor.Optional));

			CHECK_PARSE(FCapabilityUtilities::ValueFromJsonObject(
				JsonParameter, CPS::Capabilities::GDefaultValue, ParameterDescriptor.Type, ParameterDescriptor.DefaultValue));

			Command.Parameters.Add(ParameterDescriptor);
		}

		if (JsonCommand->HasField(CPS::Capabilities::GReturnType))
		{
			FString ReturnTypeString;
			CHECK_PARSE(FJsonUtility::ParseString(JsonCommand, CPS::Capabilities::GReturnType, ReturnTypeString));
			ECapabilityValue::Type ReturnType;
			CHECK_PARSE(FCapabilityUtilities::TypeStringToEnum(ReturnTypeString, ReturnType));
			Command.ReturnType = ReturnType;
		}

		Capability.Commands.Add(Command.Id, MoveTemp(Command));
	}

	const TArray<TSharedPtr<FJsonValue>>* JsonEvents;
	CHECK_PARSE(FJsonUtility::ParseArray(InJsonObject, CPS::Capabilities::GEvents, JsonEvents));

	for (const TSharedPtr<FJsonValue>& Value : *JsonEvents)
	{
		TSharedPtr<FJsonObject> JsonEvent = Value->AsObject();
		FCapabilityEvent Event;

		CHECK_PARSE(FJsonUtility::ParseString(JsonEvent, CPS::Capabilities::GId, Event.Id));
		CHECK_PARSE(FJsonUtility::ParseString(JsonEvent, CPS::Capabilities::GName, Event.Name));

		const TArray<TSharedPtr<FJsonValue>>* JsonArguments;
		CHECK_PARSE(FJsonUtility::ParseArray(JsonEvent, CPS::Capabilities::GArguments, JsonArguments));

		for (const TSharedPtr<FJsonValue>& ArgumentValue : *JsonArguments)
		{
			TSharedPtr<FJsonObject> JsonArgument = ArgumentValue->AsObject();

			FEventArgumentDescriptor CpsArgumentDescriptor;
			CHECK_PARSE(FJsonUtility::ParseString(JsonArgument, CPS::Capabilities::GName, CpsArgumentDescriptor.Name));
			FString ArgumentTypeString;
			CHECK_PARSE(FJsonUtility::ParseString(JsonArgument, CPS::Capabilities::GType, ArgumentTypeString));
			CHECK_PARSE(FCapabilityUtilities::TypeStringToEnum(ArgumentTypeString, CpsArgumentDescriptor.Type));

			Event.Arguments.Add(MoveTemp(CpsArgumentDescriptor));
		}

		Capability.Events.Add(Event.Id, MoveTemp(Event));
	}

	return Capability;
}

TArray<FCapability> FListAllCapabilitiesResponse::GetCapabilities()
{
	return Capabilities;
}

FSetCapabilityPropertyResponse::FSetCapabilityPropertyResponse()
	: FControlResponse(CPS::AddressPaths::GCapabilitiesRoot)
{
}

FGetCapabilityPropertyResponse::FGetCapabilityPropertyResponse()
	: FControlResponse(CPS::AddressPaths::GCapabilitiesRoot)
{
}

TProtocolResult<void> FGetCapabilityPropertyResponse::Parse(TSharedPtr<FJsonObject> InBody)
{
	if (!InBody.IsValid())
	{
		return FCaptureProtocolError(TEXT("Body not found in capability property operation response."));
	}

	Body = MoveTemp(InBody);

	return ResultOk;
}

TProtocolResult<FCapabilityValue> FGetCapabilityPropertyResponse::GetValue(ECapabilityValue::Type InExpectedType) const
{
	if (Body->HasField(CPS::Capabilities::GValue))
	{
		FCapabilityValue CapabilityValue;
		TProtocolResult<void> Result =
			FCapabilityUtilities::ValueFromJsonObject(Body, CPS::Capabilities::GValue, InExpectedType, CapabilityValue);

		if (Result.HasError())
		{
			return Result.GetError();
		}

		return CapabilityValue;
	}

	return FCaptureProtocolError(TEXT("Command execution did not return value."));
}

FExecuteCapabilityCommandResponse::FExecuteCapabilityCommandResponse()
	: FControlResponse(CPS::AddressPaths::GCapabilitiesRoot)
{
}

TProtocolResult<void> FExecuteCapabilityCommandResponse::Parse(TSharedPtr<FJsonObject> InBody)
{
	if (!InBody.IsValid())
	{
		return FCaptureProtocolError(TEXT("Body not found in capability execute command operation response."));
	}

	Body = MoveTemp(InBody);
	
	return ResultOk;
}

TProtocolResult<FCapabilityValue> FExecuteCapabilityCommandResponse::GetReturnValue(ECapabilityValue::Type InExpectedType) const
{
	if (!Body->HasField(CPS::Capabilities::GValue))
	{
		return FCaptureProtocolError(TEXT("Command execution did not return value."));
	}

	FCapabilityValue CapabilityValue;

	TProtocolResult<void> Result = FCapabilityUtilities::ValueFromJsonObject(Body, CPS::Capabilities::GValue, InExpectedType, CapabilityValue);

	if (Result.HasError())
	{
		FString Message = FString::Format(TEXT("Failed to convert command return value to an expected type: {0}"), { *Result.GetError().GetMessage() });
		return FCaptureProtocolError(MoveTemp(Message));
	}

	return CapabilityValue;
}

FCapabilitySubscribeResponse::FCapabilitySubscribeResponse()
	: FControlResponse(CPS::AddressPaths::GCapabilitiesRoot)
{
}

FCapabilityUnsubscribeResponse::FCapabilityUnsubscribeResponse()
	: FControlResponse(CPS::AddressPaths::GCapabilitiesRoot)
{
}

} // namespace UE::CaptureManager
