// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Messages/ControlUpdate.h"
#include "Control/Messages/Constants.h"
#include "Control/Messages/ProtocolConstants.h"

#include "Control/Messages/ControlJsonUtilities.h"
#include "Control/Capabilities/Utils.h"

namespace UE::CaptureManager
{

TProtocolResult<TSharedRef<FControlUpdate>> FControlUpdateCreator::Create(const FString& InAddressPath)
{
	if (InAddressPath == CPS::AddressPaths::GSessionStopped)
	{
		return TProtocolResult<TSharedRef<FControlUpdate>>(MakeShared<FSessionStopped>());
	}
	else if (InAddressPath == CPS::AddressPaths::GTakeAdded)
	{
		return TProtocolResult<TSharedRef<FControlUpdate>>(MakeShared<FTakeAddedUpdate>());
	}
	else if (InAddressPath == CPS::AddressPaths::GTakeRemoved)
	{
		return TProtocolResult<TSharedRef<FControlUpdate>>(MakeShared<FTakeRemovedUpdate>());
	}
	else if (InAddressPath == CPS::AddressPaths::GTakeUpdated)
	{
		return TProtocolResult<TSharedRef<FControlUpdate>>(MakeShared<FTakeUpdatedUpdate>());
	}
	else if (InAddressPath == CPS::AddressPaths::GRecordingStatus)
	{
		return TProtocolResult<TSharedRef<FControlUpdate>>(MakeShared<FRecordingStatusUpdate>());
	}
	else if (InAddressPath == CPS::AddressPaths::GDiskCapacity)
	{
		return TProtocolResult<TSharedRef<FControlUpdate>>(MakeShared<FDiskCapacityUpdate>());
	}
	else if (InAddressPath == CPS::AddressPaths::GBattery)
	{
		return TProtocolResult<TSharedRef<FControlUpdate>>(MakeShared<FBatteryPercentageUpdate>());
	}
	else if (InAddressPath == CPS::AddressPaths::GThermalState)
	{
		return TProtocolResult<TSharedRef<FControlUpdate>>(MakeShared<FThermalStateUpdate>());
	}
	else if (InAddressPath.StartsWith(CPS::AddressPaths::GCapabilitiesRoot))
	{
		return TProtocolResult<TSharedRef<FControlUpdate>>(MakeShared<FEventUpdate>(InAddressPath));
	}
	else
	{
		return FCaptureProtocolError(TEXT("Unknown update arrived"));
	}
}

FControlUpdate::FControlUpdate(FString InAddressPath)
	: AddressPath(MoveTemp(InAddressPath))
{
}

const FString& FControlUpdate::GetAddressPath() const
{
	return AddressPath;
}

TProtocolResult<void> FControlUpdate::Parse(TSharedPtr<FJsonObject> InBody)
{
	if (InBody.IsValid() && !InBody->Values.IsEmpty())
	{
		return FCaptureProtocolError("Update must NOT have a body");
	}

	return ResultOk;
}

FSessionStopped::FSessionStopped()
	: FControlUpdate(CPS::AddressPaths::GSessionStopped)
{
}

FRecordingStatusUpdate::FRecordingStatusUpdate()
	: FControlUpdate(CPS::AddressPaths::GRecordingStatus)
{
}

TProtocolResult<void> FRecordingStatusUpdate::Parse(TSharedPtr<FJsonObject> InBody)
{
	CHECK_PARSE(FJsonUtility::ParseBool(InBody, CPS::Properties::GIsRecording, bIsRecording));

	return ResultOk;
}

bool FRecordingStatusUpdate::IsRecording() const
{
	return bIsRecording;
}

TProtocolResult<void> FBaseTakeUpdate::Parse(TSharedPtr<FJsonObject> InBody)
{
	CHECK_PARSE(FJsonUtility::ParseString(InBody, CPS::Properties::GName, TakeName));

	return ResultOk;
}

const FString& FBaseTakeUpdate::GetTakeName() const
{
	return TakeName;
}

FTakeAddedUpdate::FTakeAddedUpdate()
	: FBaseTakeUpdate(CPS::AddressPaths::GTakeAdded)
{
}

FTakeRemovedUpdate::FTakeRemovedUpdate()
	: FBaseTakeUpdate(CPS::AddressPaths::GTakeRemoved)
{
}

FTakeUpdatedUpdate::FTakeUpdatedUpdate()
	: FBaseTakeUpdate(CPS::AddressPaths::GTakeUpdated)
{
}

FDiskCapacityUpdate::FDiskCapacityUpdate()
	: FControlUpdate(CPS::AddressPaths::GDiskCapacity)
{
}

TProtocolResult<void> FDiskCapacityUpdate::Parse(TSharedPtr<FJsonObject> InBody)
{
	CHECK_PARSE(FJsonUtility::ParseNumber(InBody, CPS::Properties::GTotal, Total));
	CHECK_PARSE(FJsonUtility::ParseNumber(InBody, CPS::Properties::GRemaining, Remaining));

	return ResultOk;
}

uint64 FDiskCapacityUpdate::GetTotal() const
{
	return Total;
}

uint64 FDiskCapacityUpdate::GetRemaining() const
{
	return Remaining;
}

FBatteryPercentageUpdate::FBatteryPercentageUpdate()
	: FControlUpdate(CPS::AddressPaths::GBattery)
{
}

TProtocolResult<void> FBatteryPercentageUpdate::Parse(TSharedPtr<FJsonObject> InBody)
{
	CHECK_PARSE(FJsonUtility::ParseNumber(InBody, CPS::Properties::GLevel, Level));

	return ResultOk;
}

float FBatteryPercentageUpdate::GetLevel() const
{
	return Level;
}

FThermalStateUpdate::FThermalStateUpdate()
	: FControlUpdate(CPS::AddressPaths::GThermalState)
{
}

TProtocolResult<void> FThermalStateUpdate::Parse(TSharedPtr<FJsonObject> InBody)
{
	FString StateStr;
	CHECK_PARSE(FJsonUtility::ParseString(InBody, CPS::Properties::GState, StateStr));

	State = ConvertState(StateStr);
	if (State == EState::Invalid)
	{
		return FCaptureProtocolError("Invalid thermal state provided: " + StateStr);
	}

	return ResultOk;
}

FThermalStateUpdate::EState FThermalStateUpdate::GetState() const
{
	return State;
}

FThermalStateUpdate::EState FThermalStateUpdate::ConvertState(const FString& InStateString)
{
	if (InStateString == CPS::Properties::GNominal)
	{
		return EState::Nominal;
	}
	else if (InStateString == CPS::Properties::GFair)
	{
		return EState::Fair;
	}
	else if (InStateString == CPS::Properties::GSerious)
	{
		return EState::Serious;
	}
	else if (InStateString == CPS::Properties::GCritical)
	{
		return EState::Critical;
	}
	else
	{
		return EState::Invalid;
	}
}

FEventUpdate::FEventUpdate(FString InAddressPath)
	: FControlUpdate(MoveTemp(InAddressPath))
{
}

TProtocolResult<void> FEventUpdate::Parse(TSharedPtr<FJsonObject> InBody)
{
	if (!InBody.IsValid())
	{
		return FCaptureProtocolError(TEXT("Body not found in capability event update."));
	}

	const TArray<TSharedPtr<FJsonValue>>* ArgumentsPtr;
	CHECK_PARSE(FJsonUtility::ParseArray(InBody, CPS::Capabilities::GArguments, ArgumentsPtr));

	for (TSharedPtr<FJsonValue> JsonValue : *ArgumentsPtr)
	{
		TSharedPtr<FJsonObject> JsonObject = JsonValue->AsObject();

		FString ArgumentId;
		CHECK_PARSE(FJsonUtility::ParseString(JsonObject, CPS::Capabilities::GId, ArgumentId));		

		Arguments.Add(MoveTemp(ArgumentId), MoveTemp(JsonObject));
	}

	return ResultOk;
}

const TProtocolResult<TMap<FString, FCapabilityValue>> FEventUpdate::GetArguments(const TArray<FEventArgumentDescriptor>& InArgumentsDescriptor) const
{
	TMap<FString, FCapabilityValue> ArgumentValues;

	TArray<FString> ArgumentNames;
	
	for (const FEventArgumentDescriptor& ArgumentDescriptor : InArgumentsDescriptor)
	{
		const TSharedPtr<FJsonObject>* JsonObjectPtr = Arguments.Find(ArgumentDescriptor.Name);
		const ECapabilityValue::Type ArgumentType = ArgumentDescriptor.Type;
		if (JsonObjectPtr)
		{
			FCapabilityValue CapabilityValue;
			TProtocolResult<void> Result = FCapabilityUtilities::ValueFromJsonObject(*JsonObjectPtr, CPS::Capabilities::GValue, ArgumentType, CapabilityValue);
			if (Result.HasError())
			{
				return Result.GetError();
			}
			ArgumentValues.Add({ ArgumentDescriptor.Name, CapabilityValue });
		}
		
	}
	
	return ArgumentValues;
}

} // namespace UE::CaptureManager