// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

namespace UE::CaptureManager::CPS::AddressPaths
{
// Capabilities
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilitiesRoot;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilitiesListAll;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityProperties;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityPropertySet;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityPropertyGet;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityCommands;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityCommandExecute;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityEvents;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityEventSubscribe;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityEventUnsubscribe;
} // namespace UE::CaptureManager::CPS::AddressPaths

namespace UE::CaptureManager::CPS::Capabilities
{
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GId;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GName;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GValue;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GArguments;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GParameters;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GProperties;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GType;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GAccess;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GReadOnly;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GReadWrite;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GMin;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GMax;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GEnumOptions;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCurrentValue;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GGet;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSet;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCommands;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GExecute;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GOptional;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GReturnType;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GDefaultValue;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GEvents;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSubscribe;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GUnsubscribe;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GBoolean;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GInteger;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GFloat;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GString;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GEnumeration;
} // namespace UE::CaptureManager::CPS::Capabilities
