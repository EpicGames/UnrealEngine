// Copyright Epic Games, Inc. All Rights Reserved.

#include "Discovery/Communication/DiscoveryCommunication.h"

namespace UE::CaptureManager
{

DEFINE_LOG_CATEGORY(LogCPSDiscoveryCommunication)

const uint16 FDiscoveryCommunication::MulticastPort = 27838; // According to the specification
const FString FDiscoveryCommunication::MulticastAddress = TEXT("239.255.137.139"); // According to the specification
FDiscoveryCommunication::FDiscoveryCommunication()
	: SynchronizedReceiver(FRunnerType::FOnProcess::CreateRaw(this, &FDiscoveryCommunication::OnProcessReceivedPacket))
{
}

TProtocolResult<void> FDiscoveryCommunication::Start()
{
	CPS_CHECK_VOID_RESULT(Client.Init({ MulticastPort, MulticastAddress }, FOnSocketDataReceived::CreateRaw(this, &FDiscoveryCommunication::OnPacketArrived)));
	CPS_CHECK_VOID_RESULT(Client.Start());

	return ResultOk;
}

TProtocolResult<void> FDiscoveryCommunication::Stop()
{
	return Client.Stop();
}

void FDiscoveryCommunication::SendMessage(FDiscoveryPacket InMessage, const FString& InEndpoint)
{
	TProtocolResult<TArray<uint8>> SerializeResult = FDiscoveryPacket::Serialize(InMessage);

	if (SerializeResult.HasError())
	{
		UE_LOGF(LogCPSDiscoveryCommunication, Error, "Invalid message: %ls", *(SerializeResult.StealError().GetMessage()));
		return;
	}

	Client.SendMessage(SerializeResult.StealValue(), InEndpoint);
}

void FDiscoveryCommunication::SendMessage(FDiscoveryPacket InMessage, const FString& InEndpointIp, const uint16 InEndpointPort)
{
	SendMessage(MoveTemp(InMessage), InEndpointIp + TEXT(":") + FString::FromInt(InEndpointPort));
}

void FDiscoveryCommunication::SetReceiveHandler(FOnPacketReceived InReceiveHandler)
{
	OnPacketReceived = MoveTemp(InReceiveHandler);
}

void FDiscoveryCommunication::OnPacketArrived(const FArrayReaderPtr& InPayload, const FIPv4Endpoint& InEndpoint)
{
	const TArray<uint8>& Payload = *InPayload;

	TProtocolResult<FDiscoveryPacket> DeserializeResult = FDiscoveryPacket::Deserialize(Payload);
	if (DeserializeResult.HasError())
	{
		UE_LOGF(LogCPSDiscoveryCommunication, Error, "Invalid message: %ls", *(DeserializeResult.StealError().GetMessage()));
		return;
	}

	FDiscoveryPacket Packet = DeserializeResult.StealValue();
	SynchronizedReceiver.Add({ InEndpoint.Address.ToString(), MoveTemp(Packet) });
}

void FDiscoveryCommunication::OnProcessReceivedPacket(FContext InContex)
{
	OnPacketReceived.ExecuteIfBound(MoveTemp(InContex.ServerIp), MoveTemp(InContex.Packet));
}

}