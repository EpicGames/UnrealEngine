// Copyright Epic Games, Inc. All Rights Reserved.

#include "Discovery/Communication/DiscoveryPacket.h"

namespace UE::CaptureManager
{

const TArray<uint8> FDiscoveryPacket::Header = { 'C', 'P', 'S', 'D', 'I', 'S', 'C', 'O', 'V', 'E', 'R', '\0' };

const uint16 FDiscoveryPacket::Version = 2;

FDiscoveryPacket::FDiscoveryPacket(EMessageType InMessageType, TArray<uint8> InPayload)
	: MessageType(InMessageType)
	, Payload(MoveTemp(InPayload))
{
}

TProtocolResult<FDiscoveryPacket> FDiscoveryPacket::Deserialize(const TConstArrayView<uint8>& InData)
{
	if (InData.Num() < Header.Num() + 2 + 1) // Header (12) + Version (2) + Message Type (1)
	{
		return FCaptureProtocolError(TEXT("Message with incorrect number of bytes arrived"));
	}

	const uint8* Data = InData.GetData();
	int32 Result = FMemory::Memcmp(Data, Header.GetData(), Header.Num());
	if (Result != 0)
	{
		return FCaptureProtocolError(TEXT("Message with incorrect header arrived"));
	}
	Data += Header.Num();

	const uint16 PacketVersion = *reinterpret_cast<const uint16*>(Data);

	if (PacketVersion != Version)
	{
		return FCaptureProtocolError(FString::Format(TEXT("Incompatible discovery packet version {0}"), { PacketVersion }));
	}
	Data += sizeof(Version);

	EMessageType MessageType = ToMessageType(*Data);

	if (MessageType == EMessageType::Invalid)
	{
		return FCaptureProtocolError(TEXT("Invalid message type field"));
	}

	FDiscoveryPacket Packet;

	Packet.MessageType = MessageType;
	Packet.Payload = InData.RightChop(Header.Num() + sizeof(PacketVersion) + sizeof(MessageType));

	return Packet;
}

TProtocolResult<TArray<uint8>> FDiscoveryPacket::Serialize(const FDiscoveryPacket& InMessage)
{
	TArray<uint8> OutData;

	OutData.Append(InMessage.Header.GetData(), InMessage.Header.Num());
	OutData.Append((uint8*) &Version, sizeof(Version));
	OutData.Add(FromMessageType(InMessage.MessageType));
	OutData.Append(InMessage.Payload.GetData(), InMessage.Payload.Num());

	return OutData;
}

const FDiscoveryPacket::EMessageType FDiscoveryPacket::GetMessageType() const
{
	return MessageType;
}

const TArray<uint8>& FDiscoveryPacket::GetPayload() const
{
	return Payload;
}

FDiscoveryPacket::EMessageType FDiscoveryPacket::ToMessageType(uint8 InMessageType)
{
	if (InMessageType == static_cast<uint8>(EMessageType::Request))
	{
		return EMessageType::Request;
	}
	else if (InMessageType == static_cast<uint8>(EMessageType::Response))
	{
		return EMessageType::Response;
	}
	else if (InMessageType == static_cast<uint8>(EMessageType::Notify))
	{
		return EMessageType::Notify;
	}
	else
	{
		return EMessageType::Invalid;
	}
}

uint8 FDiscoveryPacket::FromMessageType(EMessageType InMessageType)
{
	switch (InMessageType)
	{
		case EMessageType::Request:
			return 0;
		case EMessageType::Response:
			return 1;
		case EMessageType::Notify:
			return 2;
		default:
			return MAX_uint8;
	}
}

}