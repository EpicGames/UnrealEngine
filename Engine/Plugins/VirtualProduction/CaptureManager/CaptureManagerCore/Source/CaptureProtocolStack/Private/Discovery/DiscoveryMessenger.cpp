// Copyright Epic Games, Inc. All Rights Reserved.

#include "Discovery/DiscoveryMessenger.h"

namespace UE::CaptureManager
{

DEFINE_LOG_CATEGORY(LogCPSDiscoveryMessenger)

FDiscoveryMessenger::FDiscoveryMessenger()
{
}

FDiscoveryMessenger::~FDiscoveryMessenger()
{
	Stop();
}

TProtocolResult<void> FDiscoveryMessenger::Start()
{
	CPS_CHECK_VOID_RESULT(Communication.Start());

	Communication.SetReceiveHandler(
		FDiscoveryCommunication::FOnPacketReceived::CreateRaw(this, &FDiscoveryMessenger::OnPacketArrived));

	return ResultOk;
}

TProtocolResult<void> FDiscoveryMessenger::Stop()
{
	return Communication.Stop();
}

TProtocolResult<void> FDiscoveryMessenger::SendRequest()
{
	FDiscoveryRequest Request;
	TProtocolResult<FDiscoveryPacket> SerializeResult = FDiscoveryRequest::Serialize(Request);

	if (SerializeResult.HasError())
	{
		return FCaptureProtocolError(TEXT("Failed to serialize request"));
	}
	
	FDiscoveryPacket Packet = SerializeResult.GetValue();
	Communication.SendMessage(MoveTemp(Packet), FDiscoveryCommunication::MulticastAddress, FDiscoveryCommunication::MulticastPort);

	return ResultOk;
}

void FDiscoveryMessenger::SetResponseHandler(FOnResponseArrived InOnResponse)
{
	OnResponse = MoveTemp(InOnResponse);
}

void FDiscoveryMessenger::SetNotifyHandler(FOnNotifyArrived InOnNotify)
{
	OnNotify = MoveTemp(InOnNotify);
}

void FDiscoveryMessenger::OnPacketArrived(FString InServerIp, FDiscoveryPacket InPacket)
{
	const TArray<uint8>& Payload = InPacket.GetPayload();

	if (InPacket.GetMessageType() == FDiscoveryPacket::EMessageType::Request)
	{
		// Do nothing
		return;
	}
	else if (InPacket.GetMessageType() == FDiscoveryPacket::EMessageType::Response)
	{
		TProtocolResult<FDiscoveryResponse> DeserializeResult = FDiscoveryResponse::Deserialize(InPacket);

		if (DeserializeResult.HasError())
		{
			UE_LOGF(LogCPSDiscoveryMessenger, Error, "Failed to parse the response message.");
			return;
		}

		FDiscoveryResponse Response = DeserializeResult.StealValue();

		OnResponse.ExecuteIfBound(MoveTemp(InServerIp), MoveTemp(Response));
	}
	else if (InPacket.GetMessageType() == FDiscoveryPacket::EMessageType::Notify)
	{
		TProtocolResult<FDiscoveryNotify> DeserializeResult = FDiscoveryNotify::Deserialize(InPacket);

		if (DeserializeResult.HasError())
		{
			UE_LOGF(LogCPSDiscoveryMessenger, Error, "Failed to parse the notify message.");
			return;
		}

		FDiscoveryNotify Notify = DeserializeResult.StealValue();

		OnNotify.ExecuteIfBound(MoveTemp(InServerIp), MoveTemp(Notify));
	}
	else
	{
		UE_LOGF(LogCPSDiscoveryMessenger, Error, "Invalid message arrived.");
		return;
	}
}

}