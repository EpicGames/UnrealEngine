// Copyright Epic Games, Inc. All Rights Reserved.

#include "Discovery/Messages/DiscoveryNotify.h"
#include "Containers/ArrayView.h"
#include "Serialization/MemoryReader.h"

namespace UE::CaptureManager
{

const uint32 FDiscoveryNotify::MinPayloadSize = 16 + 1 + 2 + 1; // ServerId (16) + Server Name Length (1) + ControlPort (2) + ConnectionState (1)

// Deprecated in 5.6
FDiscoveryNotify::FDiscoveryNotify(FServerId InServerId, uint16 InControlPort, EConnectionState InConnectionState, TArray<uint16> InSupportedVersions)
	: ServerId(MoveTemp(InServerId))
	, ControlPort(InControlPort)
	, ConnectionState(InConnectionState)
	, SupportedVersions(MoveTemp(InSupportedVersions))
{
}

FDiscoveryNotify::FDiscoveryNotify(FServerId InServerId, FString InServerName, const uint16 InControlPort, const EConnectionState InConnectionState)
	: ServerId(MoveTemp(InServerId))
	, ServerName(MoveTemp(InServerName))
	, ControlPort(InControlPort)
	, ConnectionState(InConnectionState)
{
}

TProtocolResult<FDiscoveryNotify> FDiscoveryNotify::Deserialize(const FDiscoveryPacket& InPacket)
{
	if (InPacket.GetMessageType() != FDiscoveryPacket::EMessageType::Notify)
	{
		return FCaptureProtocolError(TEXT("Attempted to deserialize a packet as a 'notify' type but the packet message type does not match."));
	}
	
	FMemoryReaderView PayloadReader(InPacket.GetPayload());
	
	if (PayloadReader.TotalSize() < MinPayloadSize)
	{
		return FCaptureProtocolError(TEXT("Invalid number of bytes"));
	}
	
	// Server Id
	FServerId ServerId;
	PayloadReader << ServerId;

	// Server Name Length
	uint8 ServerNameLength;
	PayloadReader << ServerNameLength;
	
	// Server Name
	TArray<uint8> ServerNameBytes;
	ServerNameBytes.SetNumUninitialized(ServerNameLength);
	PayloadReader.Serialize(ServerNameBytes.GetData(), ServerNameLength);
	const auto StringConversion = StringCast<TCHAR>(reinterpret_cast<const UTF8CHAR*>(ServerNameBytes.GetData()), ServerNameBytes.Num());
	FString ServerName(StringConversion);

	// Control Port
	uint16 ControlPort;
	PayloadReader << ControlPort;

	// Connection State
	uint8 ConnectionState;
	PayloadReader << ConnectionState;
	const EConnectionState ConnectionStateEnum = ToConnectionState(ConnectionState);
	if (ConnectionStateEnum == EConnectionState::Invalid)
	{
		return FCaptureProtocolError(TEXT("Invalid connection state field"));
	}

	return FDiscoveryNotify(MoveTemp(ServerId), MoveTemp(ServerName), ControlPort, ConnectionStateEnum);
}

TProtocolResult<FDiscoveryPacket> FDiscoveryNotify::Serialize(const FDiscoveryNotify& InNotify)
{
	const auto UTF8ServerName = StringCast<UTF8CHAR>(*InNotify.GetServerName());
	const uint8 ServerNameLength = UTF8ServerName.Length();
	
	TArray<uint8> Payload;
	Payload.Reserve(InNotify.ServerId.Num() + sizeof(ServerNameLength) + ServerNameLength + sizeof(InNotify.ControlPort) + sizeof(InNotify.ConnectionState));

	Payload.Append(InNotify.ServerId.GetData(), InNotify.ServerId.Num());
	Payload.Append(&ServerNameLength, sizeof(ServerNameLength));
	Payload.Append((uint8*) UTF8ServerName.Get(), UTF8ServerName.Length());
	Payload.Append((uint8*) &InNotify.ControlPort, sizeof(InNotify.ControlPort));

	const uint8 ConnectionState = FromConnectionState(InNotify.ConnectionState);
	if (ConnectionState == MAX_uint8)
	{
		return FCaptureProtocolError(TEXT("Invalid connection state field"));
	}
	Payload.Add(ConnectionState);

	return FDiscoveryPacket(FDiscoveryPacket::EMessageType::Notify, MoveTemp(Payload));
}

const FDiscoveryNotify::FServerId& FDiscoveryNotify::GetServerId() const
{
	return ServerId;
}

const FString& FDiscoveryNotify::GetServerName() const
{
	return ServerName;
}

uint16 FDiscoveryNotify::GetControlPort() const
{
	return ControlPort;
}

FDiscoveryNotify::EConnectionState FDiscoveryNotify::GetConnectionState() const
{
	return ConnectionState;
}

// Deprecated in 5.6
const TArray<uint16>& FDiscoveryNotify::GetSupportedVersions() const
{
	return SupportedVersions;
}

FDiscoveryNotify::EConnectionState FDiscoveryNotify::ToConnectionState(uint8 InConnectionState)
{
	if (InConnectionState == 0)
	{
		return EConnectionState::Offline;
	}
	else if (InConnectionState == 1)
	{
		return EConnectionState::Online;
	}
	else
	{
		return EConnectionState::Invalid;
	}
}

uint8 FDiscoveryNotify::FromConnectionState(EConnectionState InConnectionState)
{
	switch (InConnectionState)
	{
		case EConnectionState::Offline:
			return 0;
		case EConnectionState::Online:
			return 1;
		case EConnectionState::Invalid:
		default:
			return MAX_uint8;
	}
}

}
