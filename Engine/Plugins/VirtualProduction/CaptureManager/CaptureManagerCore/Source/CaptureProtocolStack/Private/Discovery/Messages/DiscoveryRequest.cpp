// Copyright Epic Games, Inc. All Rights Reserved.

#include "Discovery/Messages/DiscoveryRequest.h"
#include "Containers/ArrayView.h"

namespace UE::CaptureManager
{

const uint32 FDiscoveryRequest::PayloadSize = 0;

TProtocolResult<FDiscoveryRequest> FDiscoveryRequest::Deserialize(const FDiscoveryPacket& InPacket)
{
	if (InPacket.GetMessageType() != FDiscoveryPacket::EMessageType::Request)
	{
		return FCaptureProtocolError(TEXT("Invalid request arrived"));
	}

	if (InPacket.GetPayload().Num() != PayloadSize)
	{
		return FCaptureProtocolError(TEXT("Invalid number of bytes"));
	}

	return FDiscoveryRequest();
}

TProtocolResult<FDiscoveryPacket> FDiscoveryRequest::Serialize(const FDiscoveryRequest& InRequest)
{
	return FDiscoveryPacket(FDiscoveryPacket::EMessageType::Request, TArray<uint8>());
}

}