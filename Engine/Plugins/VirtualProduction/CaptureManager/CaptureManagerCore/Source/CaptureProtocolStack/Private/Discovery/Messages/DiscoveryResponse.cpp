// Copyright Epic Games, Inc. All Rights Reserved.

#include "Discovery/Messages/DiscoveryResponse.h"
#include "Serialization/MemoryReader.h"

namespace UE::CaptureManager
{

const uint32 FDiscoveryResponse::MinPayloadSize = 16 + 1 + 2; // Server Id (16) + Server Name Length(1) + Control Port (2)

// Deprecated in 5.6
FDiscoveryResponse::FDiscoveryResponse(FServerId InServerId, uint16 InControlPort, TArray<uint16> InSupportedVersions)
	: ServerId(MoveTemp(InServerId))
	, ControlPort(InControlPort)
	, SupportedVersions(MoveTemp(InSupportedVersions))
{
}
	
FDiscoveryResponse::FDiscoveryResponse(FServerId InServerId, FString InServerName, const uint16 InControlPort)
	: ServerId(MoveTemp(InServerId))
	, ServerName(MoveTemp(InServerName))
	, ControlPort(InControlPort)
{
}

TProtocolResult<FDiscoveryResponse> FDiscoveryResponse::Deserialize(const FDiscoveryPacket& InPacket)
{
	if (InPacket.GetMessageType() != FDiscoveryPacket::EMessageType::Response)
	{
		return FCaptureProtocolError(TEXT("Attempted to deserialize a packet as a 'response' type but the packet message type does not match."));
	}
	
	FMemoryReaderView PayloadReader(InPacket.GetPayload());
	
	if (PayloadReader.TotalSize() < MinPayloadSize)
	{
		return FCaptureProtocolError(TEXT("Invalid number of bytes"));
	}

	// Server Id
	FServerId ServerId;
	PayloadReader << ServerId;

	// Server Name Length
	uint8 ServerNameLength;
	PayloadReader << ServerNameLength;
	
	// Server Name
	TArray<uint8> ServerNameBytes;
	ServerNameBytes.SetNumUninitialized(ServerNameLength);
	PayloadReader.Serialize(ServerNameBytes.GetData(), ServerNameLength);
	const auto StringConversion = StringCast<TCHAR>(reinterpret_cast<const UTF8CHAR*>(ServerNameBytes.GetData()), ServerNameBytes.Num());
	FString ServerName(StringConversion);

	// Control Port
	uint16 ControlPort;
	PayloadReader << ControlPort;

	return FDiscoveryResponse(MoveTemp(ServerId), MoveTemp(ServerName), ControlPort);
}

TProtocolResult<FDiscoveryPacket> FDiscoveryResponse::Serialize(const FDiscoveryResponse& InResponse)
{
	const auto UTF8ServerName = StringCast<UTF8CHAR>(*InResponse.GetServerName());
	const uint8 ServerNameLength = UTF8ServerName.Length();
	
	TArray<uint8> Payload;	
	Payload.Reserve(InResponse.ServerId.Num() + sizeof(ServerNameLength) + ServerNameLength + sizeof(InResponse.ControlPort));
	
	Payload.Append(InResponse.ServerId.GetData(), InResponse.ServerId.Num());
	Payload.Append(&ServerNameLength, sizeof(ServerNameLength));
	Payload.Append((uint8*) UTF8ServerName.Get(), UTF8ServerName.Length());
	Payload.Append((uint8*) &InResponse.ControlPort, sizeof(InResponse.ControlPort));

	return FDiscoveryPacket(FDiscoveryPacket::EMessageType::Response, MoveTemp(Payload));
}

const FDiscoveryResponse::FServerId& FDiscoveryResponse::GetServerId() const
{
	return ServerId;
}

const FString& FDiscoveryResponse::GetServerName() const
{
	return ServerName;
}

uint16 FDiscoveryResponse::GetControlPort() const
{
	return ControlPort;
}

// Deprecated in 5.6
const TArray<uint16>& FDiscoveryResponse::GetSupportedVersions() const
{
	return SupportedVersions;
}

}