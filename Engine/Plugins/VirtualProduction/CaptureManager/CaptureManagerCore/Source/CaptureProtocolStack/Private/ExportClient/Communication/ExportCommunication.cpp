// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExportClient/Communication/ExportCommunication.h"
#include "ExportClient/Definitions.h"
#include "Templates/UnrealTemplate.h"

namespace UE::CaptureManager
{

FExportCommunication::FExportCommunication()
{
}

TProtocolResult<void> FExportCommunication::Init()
{
	return Client.Init();
}

TProtocolResult<void> FExportCommunication::Start(const FString& InServerIp, const uint16 InServerPort)
{
	return Client.Start(InServerIp + TEXT(":") + FString::FromInt(InServerPort));
}

TProtocolResult<void> FExportCommunication::Stop()
{
	return Client.Stop();
}

bool FExportCommunication::IsRunning() const
{
	return Client.IsRunning();
}

TProtocolResult<void> FExportCommunication::SendRequest(FExportHeader InHeader, FExportRequest InRequest)
{
	FExportRequestPacket Packet = { MoveTemp(InHeader) , MoveTemp(InRequest) };
	FTcpClientWriter Writer(Client);

	TProtocolResult<void> SerializeResult = FExportRequestPacket::Serialize(Packet, Writer);
	if (SerializeResult.HasError())
	{
		return SerializeResult.StealError();
	}

	return ResultOk;
}

TProtocolResult<FExportResponseHeader> FExportCommunication::ReceiveResponseHeader()
{
	FTcpClientReader Reader(Client);

	return FExportResponseHeader::Deserialize(Reader);
}

TProtocolResult<TArray<uint8>> FExportCommunication::ReceiveResponseData(const uint64 InSize)
{
	return Client.ReceiveMessage(InSize, InactivityTimeoutMs);
}

TProtocolResult<TStaticArray<uint8, 16>> FExportCommunication::ReceiveFileHash()
{
	FTcpClientReader Reader(Client);

	return FExportResponse::DeserializeHash(Reader);
}

TProtocolResult<void> FExportRequestPacket::Serialize(const FExportRequestPacket& InRequestPacket, ITcpSocketWriter& InWriter)
{
	CPS_CHECK_VOID_RESULT(FExportHeader::Serialize(InRequestPacket.Header, InWriter));
	CPS_CHECK_VOID_RESULT(FExportRequest::Serialize(InRequestPacket.Request, InWriter));

	return ResultOk;
}

TProtocolResult<FExportResponseHeader> FExportResponseHeader::Deserialize(ITcpSocketReader& InReader)
{
	TProtocolResult<FExportHeader> HeaderResult = FExportHeader::Deserialize(InReader);
	if (HeaderResult.HasError())
	{
		return HeaderResult.StealError();
	}

	TProtocolResult<FExportResponse> ResponseResult = FExportResponse::Deserialize(InReader);
	if (ResponseResult.HasError())
	{
		return ResponseResult.StealError();
	}

	FExportResponseHeader ResponseHeader = { HeaderResult.StealValue(), ResponseResult.StealValue() };
	return ResponseHeader;
}

}