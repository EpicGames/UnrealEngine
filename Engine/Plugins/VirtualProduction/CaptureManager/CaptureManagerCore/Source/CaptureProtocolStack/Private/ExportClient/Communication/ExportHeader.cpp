// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExportClient/Communication/ExportHeader.h"
#include "ExportClient/Definitions.h"

#define CPS_CHECK_RESULT_RET_ERROR(Result) if (Result.HasError()) { return Result.StealError(); }

namespace UE::CaptureManager
{

const TArray<uint8> FExportHeader::Header = { 'C', 'P', 'S', 'E', 'X', 'P', 'O', 'R', 'T', '\0' };

FExportHeader::FExportHeader(uint16 InVersion, uint32 InTransactionId)
	: Version(InVersion)
	, TransactionId(InTransactionId)
{
}

TProtocolResult<FExportHeader> FExportHeader::Deserialize(ITcpSocketReader& InReader)
{
	FExportHeader ExportHeader;

	TProtocolResult<TArray<uint8>> HeaderResult = InReader.ReceiveMessage(Header.Num(), InactivityTimeoutMs);
	CPS_CHECK_RESULT_RET_ERROR(HeaderResult);

	TArray<uint8> HeaderData = HeaderResult.StealValue();

	if (FMemory::Memcmp(HeaderData.GetData(), Header.GetData(), Header.Num()) != 0)
	{
		return FCaptureProtocolError(TEXT("Header doesn't match"));
	}

	TProtocolResult<TArray<uint8>> VersionResult = InReader.ReceiveMessage(sizeof(ExportHeader.Version), InactivityTimeoutMs);
	CPS_CHECK_RESULT_RET_ERROR(VersionResult);

	TArray<uint8> VersionData = VersionResult.StealValue();

	ExportHeader.Version = *reinterpret_cast<const uint16*>(VersionData.GetData());

	TProtocolResult<TArray<uint8>> TransactionIdResult = InReader.ReceiveMessage(sizeof(ExportHeader.TransactionId), InactivityTimeoutMs);
	CPS_CHECK_RESULT_RET_ERROR(TransactionIdResult);

	TArray<uint8> TransactionIdData = TransactionIdResult.StealValue();

	ExportHeader.TransactionId = *reinterpret_cast<const uint32*>(TransactionIdData.GetData());

	return ExportHeader;
}

TProtocolResult<void> FExportHeader::Serialize(const FExportHeader& InHeader, ITcpSocketWriter& InWriter)
{
	uint16 Version = InHeader.GetVersion();
	uint32 TransactionId = InHeader.GetTransactionId();

	TArray<uint8> Data;
	Data.Append(Header.GetData(), Header.Num());
	Data.Append(reinterpret_cast<uint8*>(&Version), sizeof(Version));
	Data.Append(reinterpret_cast<uint8*>(&TransactionId), sizeof(TransactionId));

	return InWriter.SendMessage(Data);
}

uint16 FExportHeader::GetVersion() const
{
	return Version;
}

uint32 FExportHeader::GetTransactionId() const
{
	return TransactionId;
}

}