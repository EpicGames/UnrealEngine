// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::CaptureManager
{

static constexpr uint32 InactivityTimeoutMs = 15000; // 15 seconds as recommended by the CPS Specification

}