// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExportClient/ExportClient.h"

#include "ExportClient/Messages/ExportRequest.h"

#include "Utility/Definitions.h"

namespace UE::CaptureManager
{

FExportTaskStopToken::FExportTaskStopToken()
	: bCanceled(false)
{
}

void FExportTaskStopToken::Cancel()
{
	bCanceled.store(true);
}

bool FExportTaskStopToken::IsCanceled()
{
	return bCanceled.load();
}

FExportClient::FExportClient(FString InServerIp, const uint16 InExportPort)
	: ServerIp(MoveTemp(InServerIp))
	, ServerPort(InExportPort)
	, Communication(MakeUnique<FExportCommunication>())
	, ExportWorker(*this)
	, TransactionIdCounter(0)
{
	Thread.Reset(FRunnableThread::Create(&ExportWorker, TEXT("Queue Runner"), 128 * 1024, TPri_Normal, FPlatformAffinity::GetPoolThreadMask()));
}

FExportClient::~FExportClient()
{
	Thread->Kill(true);
	Stop();
}

FExportClient::FTaskId FExportClient::ExportTakeFiles(FString InTakeName, FTakeFileArray InTakeFileArray, TUniquePtr<FBaseStream> InStream)
{
	check(InStream);
	check(ServerPort != 0);

	FExportTakeTask::FExportContexts ExportContexts;
	for (FTakeFile& File : InTakeFileArray)
	{
		ExportContexts.Add({ InTakeName, MoveTemp(File) });
	}

	return StartExport(MoveTemp(ExportContexts), MoveTemp(InStream));
}

FExportClient::FTaskId FExportClient::ExportFiles(TMap<FString, FTakeFileArray> InTakesFilesMap, TUniquePtr<FBaseStream> InStream)
{
	check(InStream);
	check(ServerPort != 0);

	FExportTakeTask::FExportContexts ExportContexts;
	for (TPair<FString, FTakeFileArray>& TakeFiles : InTakesFilesMap)
	{
		for (FTakeFile& File : TakeFiles.Value)
		{
			ExportContexts.Add({ TakeFiles.Key, MoveTemp(File) });
		}
	}

	return StartExport(MoveTemp(ExportContexts), MoveTemp(InStream));
}

void FExportClient::AbortExport(const FTaskId InTaskId)
{
	TProtocolResult<TUniquePtr<FExportTakeTask>> RemovedTaskResult = ExportWorker.Remove(InTaskId);
	if (RemovedTaskResult.HasValue())
	{
		TUniquePtr<FExportTakeTask> RemovedTask = RemovedTaskResult.StealValue();

		RemovedTask->Stream->Finalize(FCaptureProtocolError(TEXT("Take aborted"), FExportClient::AbortedError));
	}
	else
	{
		CurrentTaskStopToken.IsValid() ? CurrentTaskStopToken->Cancel() : void();
	}
}

void FExportClient::AbortAllExports()
{
	TArray<TUniquePtr<FExportTakeTask>> Tasks = ExportWorker.GetAndEmpty();

	for (const TUniquePtr<FExportTakeTask>& Task : Tasks)
	{
		Task->Stream->Finalize(FCaptureProtocolError(TEXT("Take aborted"), FExportClient::AbortedError));
	}

	CurrentTaskStopToken.IsValid() ? CurrentTaskStopToken->Cancel() : void();
}

FExportClient::FTaskId FExportClient::StartExport(FExportTakeTask::FExportContexts InContexts, TUniquePtr<FBaseStream> InStream)
{
	TUniquePtr<FExportTakeTask> ExportTakeTask =
		MakeUnique<FExportTakeTask>(MoveTemp(InContexts), MoveTemp(InStream));

	FTaskId TaskIdToReturn = CurrentTaskId.fetch_add(1);

	ExportWorker.Add(TaskIdToReturn, MoveTemp(ExportTakeTask));

	return TaskIdToReturn;
}

TProtocolResult<void> FExportClient::Start()
{
	if (!Communication->IsRunning())
	{
		CPS_CHECK_VOID_RESULT(Communication->Init());

		CPS_CHECK_VOID_RESULT(Communication->Start(ServerIp, ServerPort));
	}

	return ResultOk;
}

TProtocolResult<void> FExportClient::Stop()
{
	if (Communication->IsRunning())
	{
		CPS_CHECK_VOID_RESULT(Communication->Stop());
	}

	return ResultOk;
}

void FExportClient::OnTask(TUniquePtr<FExportTakeTask> InTask)
{
	TProtocolResult<void> StartResult = Start();
	if (StartResult.HasError())
	{
		InTask->Stream->Finalize(StartResult.StealError());
		return;
	}

	CurrentTaskStopToken = MakeUnique<FExportTaskStopToken>();

	bool bShouldKeepConnection = OnExportTask(MoveTemp(InTask));

	if (ExportWorker.IsEmpty() ||
		CurrentTaskStopToken->IsCanceled() ||
		!bShouldKeepConnection)
	{
		CurrentTaskStopToken = nullptr;

		Stop();
	}
}

bool FExportClient::OnExportTask(TUniquePtr<FExportTakeTask> InTask)
{
	if (!Communication.IsValid())
	{
		InTask->Stream->Finalize(FCaptureProtocolError(TEXT("Failed to start export")));
		return false;
	}

	// Send
	TProtocolResult<TMap<uint32, TPair<FString, FTakeFile>>> SendResult = SendRequests(MoveTemp(InTask->ExportContexts));
	if (SendResult.HasError())
	{
		InTask->Stream->Finalize(SendResult.StealError());
		return false;
	}

	TMap<uint32, TPair<FString, FTakeFile>> RequestToResponseData = SendResult.StealValue();

	// Receive
	TProtocolResult<void> ReceiveResult = ReceiveResponses(MoveTemp(RequestToResponseData), *InTask->Stream);
	if (ReceiveResult.HasError())
	{
		InTask->Stream->Finalize(ReceiveResult.StealError());
		return false;
	}

	InTask->Stream->Finalize(ResultOk);

	return true;
}

TProtocolResult<TMap<uint32, TPair<FString, FTakeFile>>> FExportClient::SendRequests(FExportTakeTask::FExportContexts InContexts)
{
	TMap<uint32, TPair<FString, FTakeFile>> ResponseMap;

	for (FExportTakeTask::FExportContext& ExportContext : InContexts)
	{
		if (CurrentTaskStopToken->IsCanceled())
		{
			return FCaptureProtocolError(TEXT("Take aborted"), FExportClient::AbortedError);
		}

		uint32 TransactionId = TransactionIdCounter++;

		FExportHeader ExportHeader(CPS_VERSION, TransactionId);
		FExportRequest ExportRequest(ExportContext.TakeName, ExportContext.File.FileName, ExportContext.File.Offset);

		TProtocolResult<void> SendRequestResult = Communication->SendRequest(MoveTemp(ExportHeader), MoveTemp(ExportRequest));

		if (SendRequestResult.HasError())
		{
			return SendRequestResult.StealError();
		}

		ResponseMap.Emplace(TransactionId, { ExportContext.TakeName, MoveTemp(ExportContext.File) });
	}

	return ResponseMap;
}

TProtocolResult<void> FExportClient::ReceiveResponses(TMap<uint32, TPair<FString, FTakeFile>> InResponseMap,
													  FBaseStream& InBaseStream)
{
	while (!InResponseMap.IsEmpty())
	{
		if (CurrentTaskStopToken->IsCanceled())
		{
			return FCaptureProtocolError(TEXT("Take aborted"), FExportClient::AbortedError);
		}

		TProtocolResult<FExportResponseHeader> ResponseHeaderResult = Communication->ReceiveResponseHeader();

		if (ResponseHeaderResult.HasError())
		{
			return ResponseHeaderResult.StealError();
		}

		FExportResponseHeader ResponseHeader = ResponseHeaderResult.StealValue();

		uint32 ResponseTransactionId = ResponseHeader.Header.GetTransactionId();
		FExportResponse::EStatus Status = ResponseHeader.Response.GetStatus();

		if (Status != FExportResponse::EStatus::Success)
		{
			return FCaptureProtocolError(TEXT("Server responded with error status"), static_cast<int32>(Status));
		}

		const TPair<FString, FTakeFile>& TakeFile = InResponseMap[ResponseTransactionId];

		const FTakeFile& File = TakeFile.Value;
		TProtocolResult<void> StartFileResult = InBaseStream.StartFile(TakeFile.Key, File.FileName);
		if (StartFileResult.HasError())
		{
			return StartFileResult;
		}

		uint64 BytesLeft = File.Length;

		constexpr uint32 MaxChunkSize = 64 * 1024;

		while (BytesLeft != 0)
		{
			if (CurrentTaskStopToken->IsCanceled())
			{
				return FCaptureProtocolError(TEXT("Take aborted"), FExportClient::AbortedError);
			}

			uint32 ChunkSize = BytesLeft > MaxChunkSize ? MaxChunkSize : BytesLeft;

			TProtocolResult<TArray<uint8>> FileDataResult = Communication->ReceiveResponseData(ChunkSize);

			if (FileDataResult.HasError())
			{
				return FileDataResult.StealError();
			}

			TArray<uint8> FileData = FileDataResult.StealValue();

			TProtocolResult<void> ProcessDataResult = InBaseStream.ProcessData(TakeFile.Key, File.FileName, FileData);
			if (ProcessDataResult.HasError())
			{
				return ProcessDataResult;
			}

			BytesLeft -= FileData.Num();
		}

		TProtocolResult<TStaticArray<uint8, 16>> FileHashResult = Communication->ReceiveFileHash();
		if (FileHashResult.HasError())
		{
			return FileHashResult.StealError();
		}

		TProtocolResult<void> FinishStream = InBaseStream.FinishFile(TakeFile.Key, File.FileName, FileHashResult.StealValue());
		if (FinishStream.HasError())
		{
			return FinishStream;
		}

		InResponseMap.Remove(ResponseTransactionId);
	}

	return ResultOk;
}

}