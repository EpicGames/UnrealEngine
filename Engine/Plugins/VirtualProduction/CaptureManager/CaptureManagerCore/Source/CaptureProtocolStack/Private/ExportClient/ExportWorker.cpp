// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExportClient/ExportWorker.h"

namespace UE::CaptureManager
{

FExportTakeTask::FExportTakeTask(FExportContexts InExportContexts,
								 TUniquePtr<FBaseStream> InStream)
	: ExportContexts(MoveTemp(InExportContexts))
	, Stream(MoveTemp(InStream))
{
}

FExportQueue::FExportQueue()
	: Semaphore(0, MaxNumberOfElements)
{
}


TProtocolResult<void> FExportQueue::Add(const uint32 InTaskId, TUniquePtr<FExportTakeTask> InElement)
{
	{
		FScopeLock Lock(&Mutex);
		if (Queue.Num() >= MaxNumberOfElements)
		{
			return FCaptureProtocolError(TEXT("Number of elements exceeded"));
		}

		Queue.Add({ InTaskId, MoveTemp(InElement) });
	}

	Semaphore.Release();

	return ResultOk;
}

TUniquePtr<FExportTakeTask> FExportQueue::Pop()
{
	while (true)
	{
		Semaphore.Acquire();

		FScopeLock Lock(&Mutex);
		if (!Queue.IsEmpty())
		{
			TUniquePtr<FExportTakeTask> RemovedItem = MoveTemp(Queue[0].Value);
			Queue.RemoveAt(0);

			return MoveTemp(RemovedItem);
		}
	}
}

TProtocolResult<TUniquePtr<FExportTakeTask>> FExportQueue::Remove(uint32 InTaskId)
{
	int32 Index = 0;

	FScopeLock Lock(&Mutex);
	for (; Index < Queue.Num(); ++Index)
	{
		if (Queue[Index].Key == InTaskId)
		{
			break;
		}
	}

	if (Index == Queue.Num())
	{
		return FCaptureProtocolError(TEXT("Element doesn't exist"));
	}

	TUniquePtr<FExportTakeTask> RemovedItem = MoveTemp(Queue[Index].Value);
	Queue.RemoveAt(Index);

	return RemovedItem;
}

bool FExportQueue::IsEmpty() const
{
	FScopeLock Lock(&Mutex);
	return Queue.IsEmpty();
}

TArray<TUniquePtr<FExportTakeTask>> FExportQueue::GetAndEmpty()
{
	TArray<TUniquePtr<FExportTakeTask>> TaskArray;

	FScopeLock Lock(&Mutex);

	for (TPair<uint32, TUniquePtr<FExportTakeTask>>& Task : Queue)
	{
		TaskArray.Add(MoveTemp(Task.Value));
	}

	Queue.Empty();

	return TaskArray;
}

FExportWorker::FExportWorker(FExportTaskExecutor& InClient)
	: bRunning(true)
	, Client(InClient)
{
}

FExportWorker::~FExportWorker()
{
}

TProtocolResult<void> FExportWorker::Add(const uint32 InTaskId, TUniquePtr<FExportTakeTask> InElement)
{
	return Queue.Add(InTaskId, MoveTemp(InElement));
}

TProtocolResult<TUniquePtr<FExportTakeTask>> FExportWorker::Remove(uint32 InTaskId)
{
	return Queue.Remove(InTaskId);
}

bool FExportWorker::IsEmpty() const
{
	return Queue.IsEmpty();
}

TArray<TUniquePtr<FExportTakeTask>> FExportWorker::GetAndEmpty()
{
	return Queue.GetAndEmpty();
}

uint32 FExportWorker::Run()
{
	while (bRunning.load())
	{
		TUniquePtr<FExportTakeTask> Elem = Queue.Pop();
		if (Elem.IsValid())
		{
			Client.OnTask(MoveTemp(Elem));
		}
	}

	return 0;
}

void FExportWorker::Stop()
{
	bRunning.store(false);

	Queue.Add(0, nullptr);
}

}