// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExportClient/Messages/ExportRequest.h"

#include "Containers/StringConv.h"

#define CPS_CHECK_RESULT_RET_ERROR(Result) if (Result.HasError()) { return Result.StealError(); }

namespace UE::CaptureManager
{

FExportRequest::FExportRequest(FString InTakeName, FString InFileName, uint64 InOffset)
	: TakeName(MoveTemp(InTakeName))
	, FileName(MoveTemp(InFileName))
	, Offset(InOffset)
{
}

TProtocolResult<FExportRequest> FExportRequest::Deserialize(ITcpSocketReader& InReader)
{
	FExportRequest Request;

	uint16 Length;
	TProtocolResult<TArray<uint8>> TakeNameLengthResult = InReader.ReceiveMessage(sizeof(Length));
	CPS_CHECK_RESULT_RET_ERROR(TakeNameLengthResult);

	Length = *reinterpret_cast<uint16*>(TakeNameLengthResult.StealValue().GetData());

	TProtocolResult<TArray<uint8>> TakeNameResult = InReader.ReceiveMessage(Length);
	CPS_CHECK_RESULT_RET_ERROR(TakeNameResult);

	TArray<uint8> TakeNameData = TakeNameResult.StealValue();
	Request.TakeName = FString::ConstructFromPtrSize((const UTF8CHAR*) TakeNameData.GetData(), Length);

	TProtocolResult<TArray<uint8>> FileNameLengthResult = InReader.ReceiveMessage(sizeof(Length));
	CPS_CHECK_RESULT_RET_ERROR(FileNameLengthResult);

	Length = *reinterpret_cast<uint16*>(FileNameLengthResult.StealValue().GetData());

	TProtocolResult<TArray<uint8>> FileNameResult = InReader.ReceiveMessage(Length);
	CPS_CHECK_RESULT_RET_ERROR(FileNameResult);

	TArray<uint8> FileNameData = FileNameResult.StealValue();
	Request.FileName = FString::ConstructFromPtrSize((const UTF8CHAR*) FileNameData.GetData(), Length);

	TProtocolResult<TArray<uint8>> OffsetResult = InReader.ReceiveMessage(sizeof(Request.Offset));
	CPS_CHECK_RESULT_RET_ERROR(OffsetResult);

	TArray<uint8> OffsetData = OffsetResult.StealValue();

	Request.Offset = *reinterpret_cast<uint64*>(OffsetData.GetData());

	return Request;
}

TProtocolResult<void> FExportRequest::Serialize(const FExportRequest& InRequest, ITcpSocketWriter& InWriter)
{
	const FString& TakeName = InRequest.GetTakeName();
	const FString& FileName = InRequest.GetFileName();

	const uint64 Offset = InRequest.GetOffset();

	TArray<uint8> Data;

	auto TakeNameUTF8 = StringCast<UTF8CHAR>(*TakeName);
	auto FileNameUTF8 = StringCast<UTF8CHAR>(*FileName);

	const uint16 TakeNameLen = TakeNameUTF8.Length();
	const uint16 FileNameLen = FileNameUTF8.Length();

	Data.Append(reinterpret_cast<const uint8*>(&TakeNameLen), sizeof(TakeNameLen));
	Data.Append(reinterpret_cast<const uint8*>(TakeNameUTF8.Get()), TakeNameLen);
	Data.Append(reinterpret_cast<const uint8*>(&FileNameLen), sizeof(FileNameLen));
	Data.Append(reinterpret_cast<const uint8*>(FileNameUTF8.Get()), FileNameLen);
	Data.Append(reinterpret_cast<const uint8*>(&Offset), sizeof(Offset));

	return InWriter.SendMessage(Data);
}

const FString& FExportRequest::GetTakeName() const
{
	return TakeName;
}

const FString& FExportRequest::GetFileName() const
{
	return FileName;
}

uint64 FExportRequest::GetOffset() const
{
	return Offset;
}

}