// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExportClient/Messages/ExportResponse.h"
#include "ExportClient/Definitions.h"

#define CPS_CHECK_RESULT_RET_ERROR(Result) if (Result.HasError()) { return Result.StealError(); }

namespace UE::CaptureManager
{

FExportResponse::FExportResponse(EStatus InStatus, uint64 InLength)
	: Status(InStatus)
	, Length(InLength)
{
}

TProtocolResult<FExportResponse> FExportResponse::Deserialize(ITcpSocketReader& InReader)
{
	FExportResponse Response;

	TProtocolResult<TArray<uint8>> StatusResult = InReader.ReceiveMessage(sizeof(Response.Status), InactivityTimeoutMs);
	CPS_CHECK_RESULT_RET_ERROR(StatusResult);

	Response.Status = ToStatus(StatusResult.StealValue()[0]);

	if (Response.Status == EStatus::Success)
	{
		TProtocolResult<TArray<uint8>> LengthResult = InReader.ReceiveMessage(sizeof(Response.Length), InactivityTimeoutMs);
		CPS_CHECK_RESULT_RET_ERROR(LengthResult);

		TArray<uint8> LengthData = LengthResult.StealValue();

		Response.Length = *reinterpret_cast<const uint64*>(LengthData.GetData());
	}

	return Response;
}

TProtocolResult<TStaticArray<uint8, 16>> FExportResponse::DeserializeHash(ITcpSocketReader& InReader)
{
	TStaticArray<uint8, 16> Hash(0); // Initializing to zeros

	TProtocolResult<TArray<uint8>> HashResult = InReader.ReceiveMessage(Hash.Num(), InactivityTimeoutMs);
	CPS_CHECK_RESULT_RET_ERROR(HashResult);

	TArray<uint8> HashData = HashResult.StealValue();

	FMemory::Memcpy(Hash.GetData(), HashData.GetData(), Hash.Num());

	return Hash;
}

TProtocolResult<void> FExportResponse::Serialize(const FExportResponse& InResponse, ITcpSocketWriter& InWriter)
{
	uint8 Status = FromStatus(InResponse.GetStatus());

	TArray<uint8> Data;
	Data.Append(reinterpret_cast<uint8*>(&Status), sizeof(Status));

	if (InResponse.GetStatus() == EStatus::Success)
	{
		uint64 Length = InResponse.GetLength();
		Data.Append(reinterpret_cast<uint8*>(&Length), sizeof(Length));
	}

	return InWriter.SendMessage(Data);
}

TProtocolResult<void> FExportResponse::SerializeHash(const TStaticArray<uint8, 16>& InHash, ITcpSocketWriter& InWriter)
{
	TArray<uint8> Data;

	Data.Append(InHash.GetData(), InHash.Num());

	return InWriter.SendMessage(Data);
}

FExportResponse::EStatus FExportResponse::GetStatus() const
{
	return Status;
}

uint64 FExportResponse::GetLength() const
{
	return Length;
}

FExportResponse::EStatus FExportResponse::ToStatus(uint8 InStatus)
{
	if (InStatus == 0)
	{
		return EStatus::Success;
	}
	else if (InStatus == 1)
	{
		return EStatus::InvalidTakeName;
	}
	else if (InStatus == 2)
	{
		return EStatus::InvalidFileName;
	}
	else if (InStatus == 3)
	{
		return EStatus::InvalidOffset;
	}
	else if (InStatus == 4)
	{
		return EStatus::ServerError;
	}
	else
	{
		return EStatus::Reserved;
	}
}

uint8 FExportResponse::FromStatus(EStatus InStatus)
{
	switch (InStatus)
	{
		case EStatus::Success:
			return 0;
		case EStatus::InvalidTakeName:
			return 1;
		case EStatus::InvalidFileName:
			return 2;
		case EStatus::InvalidOffset:
			return 3;
		case EStatus::ServerError:
			return 4;
		case EStatus::Reserved:
		default:
			return 255;
	}
}

}