// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Messages/ControlJsonUtilities.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::CaptureManager
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlJsonUtilitiesParseNumber, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlJsonUtilities.ParseNumber.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlJsonUtilitiesParseNumber::RunTest(const FString& InParameters)
{
    FString KeyInteger = TEXT("KeyInteger");
    int32 ValueInteger = 10;
    FString KeyFloat = TEXT("KeyFloat");
    float ValueFloat = 10.0;

    TSharedPtr<FJsonObject> Object = MakeShared<FJsonObject>();
    Object->SetNumberField(KeyInteger, ValueInteger);
    Object->SetNumberField(KeyFloat, ValueFloat);

    int32 ParseValueInteger;
    TestTrue(TEXT("ParseNumber"), FJsonUtility::ParseNumber(Object, KeyInteger, ParseValueInteger).HasValue());
    TestEqual(TEXT("ParseNumber"), ParseValueInteger, ValueInteger);

    float ParseValueFloat;
    TestTrue(TEXT("ParseNumber"), FJsonUtility::ParseNumber(Object, KeyFloat, ParseValueFloat).HasValue());
    TestEqual(TEXT("ParseNumber"), ParseValueFloat, ValueFloat);

    return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlJsonUtilitiesParseNumberMissingField, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlJsonUtilities.ParseNumber.MissingField", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlJsonUtilitiesParseNumberMissingField::RunTest(const FString& InParameters)
{
    FString Key = TEXT("KeyInteger");
    FString KeyInvalid = TEXT("KeyIntegerInvalid");
    int32 Value = 10;

    TSharedPtr<FJsonObject> Object = MakeShared<FJsonObject>();
    Object->SetNumberField(Key, Value);

    int32 ParseValue;
    TestTrue(TEXT("ParseNumber"), FJsonUtility::ParseNumber(Object, KeyInvalid, ParseValue).HasError());

    return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlJsonUtilitiesParseString, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlJsonUtilities.ParseString.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlJsonUtilitiesParseString::RunTest(const FString& InParameters)
{
    FString Key = TEXT("KeyString");
    FString Value = TEXT("ValueString");

    TSharedPtr<FJsonObject> Object = MakeShared<FJsonObject>();
    Object->SetStringField(Key, Value);

    FString ParseValue;
    TestTrue(TEXT("ParseString"), FJsonUtility::ParseString(Object, Key, ParseValue).HasValue());
    TestEqual(TEXT("ParseString"), ParseValue, Value);

    return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlJsonUtilitiesParseStringMissingField, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlJsonUtilities.ParseString.MissingField", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlJsonUtilitiesParseStringMissingField::RunTest(const FString& InParameters)
{
    FString Key = TEXT("KeyString");
    FString KeyInvalid = TEXT("KeyStringInvalid");
    FString Value = TEXT("ValueString");

    TSharedPtr<FJsonObject> Object = MakeShared<FJsonObject>();
    Object->SetStringField(Key, Value);

    FString ParseValue;
    TestTrue(TEXT("ParseString"), FJsonUtility::ParseString(Object, KeyInvalid, ParseValue).HasError());

    return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlJsonUtilitiesParseBool, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlJsonUtilities.ParseBool.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlJsonUtilitiesParseBool::RunTest(const FString& InParameters)
{
    FString Key = TEXT("KeyBool");
    bool Value = true;

    TSharedPtr<FJsonObject> Object = MakeShared<FJsonObject>();
    Object->SetBoolField(Key, Value);

    bool ParseValue;
    TestTrue(TEXT("ParseBool"), FJsonUtility::ParseBool(Object, Key, ParseValue).HasValue());
    TestEqual(TEXT("ParseBool"), ParseValue, Value);

    return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlJsonUtilitiesParseBoolMissingField, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlJsonUtilities.ParseString.MissingField", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlJsonUtilitiesParseBoolMissingField::RunTest(const FString& InParameters)
{
    FString Key = TEXT("KeyBool");
    FString KeyInvalid = TEXT("KeyInvalid");
    bool Value = true;

    TSharedPtr<FJsonObject> Object = MakeShared<FJsonObject>();
    Object->SetBoolField(Key, Value);

    bool ParseValue;
    TestTrue(TEXT("ParseBool"), FJsonUtility::ParseBool(Object, KeyInvalid, ParseValue).HasError());

    return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlJsonUtilitiesParseObject, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlJsonUtilities.ParseObject.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlJsonUtilitiesParseObject::RunTest(const FString& InParameters)
{
    FString Key = TEXT("KeyObject");
    TSharedPtr<FJsonObject> Value = MakeShared<FJsonObject>();
    Value->SetStringField(TEXT("Field"), TEXT("Value"));

    TSharedPtr<FJsonObject> Object = MakeShared<FJsonObject>();
    Object->SetObjectField(Key, Value);

    const TSharedPtr<FJsonObject>* ParseValue;
    TestTrue(TEXT("ParseObject"), FJsonUtility::ParseObject(Object, Key, ParseValue).HasValue());

    for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : Value->Values)
    {
        TestTrue(TEXT("ParseObject"), (*ParseValue)->HasField(Pair.Key));

        TSharedPtr<FJsonValue> ParseFieldValue = (*ParseValue)->TryGetField(Pair.Key);

        TestNotNull(TEXT("ParseObject"), ParseFieldValue.Get());
        TestEqual(TEXT("ParseObject"), *Pair.Value, *ParseFieldValue);
    }

    return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlJsonUtilitiesParseObjectMissingField, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlJsonUtilities.ParseObject.MissingField", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlJsonUtilitiesParseObjectMissingField::RunTest(const FString& InParameters)
{
    FString Key = TEXT("KeyObject");
    FString KeyInvalid = TEXT("KeyInvalid");
    TSharedPtr<FJsonObject> Value = MakeShared<FJsonObject>();
    Value->SetStringField(TEXT("Field"), TEXT("Value"));

    TSharedPtr<FJsonObject> Object = MakeShared<FJsonObject>();
    Object->SetObjectField(Key, Value);

    const TSharedPtr<FJsonObject>* ParseValue;
    TestTrue(TEXT("ParseObject"), FJsonUtility::ParseObject(Object, KeyInvalid, ParseValue).HasError());

    return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlJsonUtilitiesParseArray, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlJsonUtilities.ParseArray.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlJsonUtilitiesParseArray::RunTest(const FString& InParameters)
{
    FString Key = TEXT("KeyArray");

    constexpr uint32 MaxNumberOfElemens = 10;
    TArray<TSharedPtr<FJsonValue>> Value;
    Value.Reserve(MaxNumberOfElemens);

    for (int32 ArrayValue = 0; ArrayValue < MaxNumberOfElemens; ++ArrayValue)
    {
        Value.Add(MakeShared<FJsonValueNumber>(ArrayValue));
    }

    TSharedPtr<FJsonObject> Object = MakeShared<FJsonObject>();
    Object->SetArrayField(Key, Value);

    const TArray<TSharedPtr<FJsonValue>>* ParseValue;
    TestTrue(TEXT("ParseArray"), FJsonUtility::ParseArray(Object, Key, ParseValue).HasValue());

    for (int32 Index = 0; Index < MaxNumberOfElemens; ++Index)
    {
        TestEqual(TEXT("ParseArray"), Value[Index], (*ParseValue)[Index]);
    }

    return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlJsonUtilitiesParseArrayMissingField, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlJsonUtilities.ParseArray.MissingField", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlJsonUtilitiesParseArrayMissingField::RunTest(const FString& InParameters)
{
    FString Key = TEXT("KeyArray");
    FString KeyInvalid = TEXT("KeyInvalid");

    constexpr uint32 MaxNumberOfElemens = 10;
    TArray<TSharedPtr<FJsonValue>> Value;
    Value.Reserve(MaxNumberOfElemens);

    for (int32 ArrayValue = 0; ArrayValue < MaxNumberOfElemens; ++ArrayValue)
    {
        Value.Add(MakeShared<FJsonValueNumber>(ArrayValue));
    }

    TSharedPtr<FJsonObject> Object = MakeShared<FJsonObject>();
    Object->SetArrayField(Key, Value);

    const TArray<TSharedPtr<FJsonValue>>* ParseValue;
    TestTrue(TEXT("ParseArray"), FJsonUtility::ParseArray(Object, KeyInvalid, ParseValue).HasError());

    return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlJsonUtilitiesCreateJsonFromData, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlJsonUtilities.CreateJsonFromData.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlJsonUtilitiesCreateJsonFromData::RunTest(const FString& InParameters)
{
	auto ConvertedString = StringCast<UTF8CHAR>(TEXT("{\"Hello\":\"World\",\"Time\":123123123}"));
	
	TArray<uint8> Data(reinterpret_cast<const uint8*>(ConvertedString.Get()), ConvertedString.Length());

	TSharedPtr<FJsonObject> Json = MakeShared<FJsonObject>();
	TestTrue(TEXT("CreateJsonFromData"), FJsonUtility::CreateJsonFromUTF8Data(Data, Json));

	FString ValueStr;
	TestTrue(TEXT("CreateJsonFromData"), Json->TryGetStringField(TEXT("Hello"), ValueStr));
	TestEqual(TEXT("CreateJsonFromData"), TEXT("World"), ValueStr);

	uint32 ValueInt;
	TestTrue(TEXT("CreateJsonFromData"), Json->TryGetNumberField(TEXT("Time"), ValueInt));
	TestEqual(TEXT("CreateJsonFromData"), 123123123, ValueInt);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlJsonUtilitiesCreateJsonFromDataFail, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlJsonUtilities.CreateJsonFromData.Failure", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlJsonUtilitiesCreateJsonFromDataFail::RunTest(const FString& InParameters)
{
	auto ConvertedString = StringCast<UTF8CHAR>(TEXT("\"Hello\":\"World\",\"Time\":123123123"));

	TArray<uint8> Data(reinterpret_cast<const uint8*>(ConvertedString.Get()), ConvertedString.Length());

	TSharedPtr<FJsonObject> Json = MakeShared<FJsonObject>();
	TestFalse(TEXT("CreateJsonFromData"), FJsonUtility::CreateJsonFromUTF8Data(Data, Json));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlJsonUtilitiesCreateDataFromJson, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlJsonUtilities.CreateDataFromJson.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlJsonUtilitiesCreateDataFromJson::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Json = MakeShared<FJsonObject>();
	Json->SetStringField(TEXT("Hello"), TEXT("World"));
	Json->SetNumberField(TEXT("Time"), 123123123);

	auto ConvertedString = StringCast<UTF8CHAR>(TEXT("{\"Hello\":\"World\",\"Time\":123123123}"));
	TArray<uint8> ExpectedData(reinterpret_cast<const uint8*>(ConvertedString.Get()), ConvertedString.Length());

	TArray<uint8> Data;
	TestTrue(TEXT("CreateDataFromJson"), FJsonUtility::CreateUTF8DataFromJson(Json, Data));
	TestEqual(TEXT("CreateDataFromJson"), Data, ExpectedData);

	return true;
}

}

#endif // WITH_DEV_AUTOMATION_TESTS