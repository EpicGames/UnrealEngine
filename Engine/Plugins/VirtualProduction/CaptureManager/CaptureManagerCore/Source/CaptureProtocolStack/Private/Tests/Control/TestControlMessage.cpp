// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Messages/ControlMessage.h"

#include "Control/Messages/ControlJsonUtilities.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#define VERSION 1

namespace UE::CaptureManager
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlMessageDeserializeTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlMessage.Deserialize.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlMessageDeserializeTest::RunTest(const FString& InParameters)
{
	FString Payload = TEXT("{\"sessionId\":\"handshake\",\"transactionId\":123456789,\"timestamp\":112233445566,\"type\":\"request\",\"addressPath\":\"/session/start\"}");

	TArray<uint8> Data;
	Data.Append(Payload);

	TProtocolResult<FControlMessage> DeserializeResult = FControlMessage::Deserialize(FControlPacket(VERSION, MoveTemp(Data)));
	TestTrue(TEXT("Deserialize"), DeserializeResult.HasValue());

	FControlMessage Message = DeserializeResult.StealValue();
	TestEqual(TEXT("Deserialize"), Message.GetAddressPath(), TEXT("/session/start"));
	TestEqual(TEXT("Deserialize"), Message.GetSessionId(), TEXT("handshake"));
	TestEqual(TEXT("Deserialize"), Message.GetTransactionId(), (uint32) 123456789);
	TestEqual(TEXT("Deserialize"), Message.GetTimestamp(), (uint64) 112233445566);
	TestEqual(TEXT("Deserialize"), Message.GetType(), FControlMessage::EType::Request);
	TestNull(TEXT("Deserialize"), Message.GetBody().Get());
	TestTrue(TEXT("Deserialize"), Message.GetErrorName().IsEmpty());
	TestTrue(TEXT("Deserialize"), Message.GetErrorDescription().IsEmpty());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlMessageDeserializeTestInvalidPayload, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlMessage.DeserializeOne.InvalidPayload", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlMessageDeserializeTestInvalidPayload::RunTest(const FString& InParameters)
{
	// Prepare test
	FString Payload = TEXT("Hello");

	TArray<uint8> Data;
	Data.Append(Payload);

	TProtocolResult<FControlMessage> DeserializeResult = FControlMessage::Deserialize(FControlPacket(VERSION, MoveTemp(Data)));
	TestTrue(TEXT("Deserialize"), DeserializeResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlMessageDeserializeSessionIdMissingTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlMessage.Deserialize.SessionIdMissing", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlMessageDeserializeSessionIdMissingTest::RunTest(const FString& InParameters)
{
	FString Payload = TEXT("{\"transactionId\":123456789,\"timestamp\":112233445566,\"type\":\"request\",\"addressPath\":\"/session/start\"}");

	TArray<uint8> Data;
	Data.Append(Payload);

	TProtocolResult<FControlMessage> DeserializeResult = FControlMessage::Deserialize(FControlPacket(VERSION, MoveTemp(Data)));
	TestTrue(TEXT("Deserialize"), DeserializeResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlMessageDeserializeTransactionIdMissingTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlMessage.Deserialize.TransactionIdMissing", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlMessageDeserializeTransactionIdMissingTest::RunTest(const FString& InParameters)
{
	FString Payload = TEXT("{\"sessionId\":\"handshake\",\"timestamp\":112233445566,\"type\":\"request\",\"addressPath\":\"/session/start\"}");

	TArray<uint8> Data;
	Data.Append(Payload);

	TProtocolResult<FControlMessage> DeserializeResult = FControlMessage::Deserialize(FControlPacket(VERSION, MoveTemp(Data)));
	TestTrue(TEXT("Deserialize"), DeserializeResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlMessageDeserializeTimestampMissingTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlMessage.Deserialize.TimestampMissing", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlMessageDeserializeTimestampMissingTest::RunTest(const FString& InParameters)
{
	FString Payload = TEXT("{\"sessionId\":\"handshake\",\"transactionId\":123456789,\"type\":\"request\",\"addressPath\":\"/session/start\"}");

	TArray<uint8> Data;
	Data.Append(Payload);

	TProtocolResult<FControlMessage> DeserializeResult = FControlMessage::Deserialize(FControlPacket(VERSION, MoveTemp(Data)));
	TestTrue(TEXT("Deserialize"), DeserializeResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlMessageDeserializeTypeMissingTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlMessage.Deserialize.TypeMissing", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlMessageDeserializeTypeMissingTest::RunTest(const FString& InParameters)
{
	FString Payload = TEXT("{\"sessionId\":\"handshake\",\"transactionId\":123456789,\"timestamp\":112233445566,\"addressPath\":\"/session/start\"}");

	TArray<uint8> Data;
	Data.Append(Payload);

	TProtocolResult<FControlMessage> DeserializeResult = FControlMessage::Deserialize(FControlPacket(VERSION, MoveTemp(Data)));
	TestTrue(TEXT("Deserialize"), DeserializeResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlMessageDeserializeAddressPathMissingTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlMessage.Deserialize.AddressPathMissing", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlMessageDeserializeAddressPathMissingTest::RunTest(const FString& InParameters)
{
	FString Payload = TEXT("{\"sessionId\":\"handshake\",\"transactionId\":123456789,\"timestamp\":112233445566,\"type\":\"request\"}");

	TArray<uint8> Data;
	Data.Append(Payload);

	TProtocolResult<FControlMessage> DeserializeResult = FControlMessage::Deserialize(FControlPacket(VERSION, MoveTemp(Data)));
	TestTrue(TEXT("Deserialize"), DeserializeResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlMessageSerializeTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlMessage.Serialize.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlMessageSerializeTest::RunTest(const FString& InParameters)
{
	FControlMessage Message(TEXT("/session/start"), FControlMessage::EType::Request, nullptr);
	Message.SetSessionId(TEXT("handshake"));
	Message.SetTransactionId(123456789);
	Message.SetTimestamp(112233445566);

	TProtocolResult<FControlPacket> SerializeResult = FControlMessage::Serialize(Message);
	TestTrue(TEXT("Serialize"), SerializeResult.HasValue());

	TProtocolResult<FControlMessage> DeserializeResult = FControlMessage::Deserialize(SerializeResult.StealValue());
	TestTrue(TEXT("Serialize"), DeserializeResult.HasValue());

	FControlMessage DeserializedMessage = DeserializeResult.StealValue();
	TestEqual(TEXT("Serialize"), Message.GetAddressPath(), DeserializedMessage.GetAddressPath());
	TestEqual(TEXT("Serialize"), Message.GetSessionId(), DeserializedMessage.GetSessionId());
	TestEqual(TEXT("Serialize"), Message.GetTransactionId(), DeserializedMessage.GetTransactionId());
	TestEqual(TEXT("Serialize"), Message.GetTimestamp(), DeserializedMessage.GetTimestamp());
	TestEqual(TEXT("Serialize"), Message.GetType(), DeserializedMessage.GetType());
	TestEqual(TEXT("Serialize"), Message.GetBody(), DeserializedMessage.GetBody());
	TestEqual(TEXT("Serialize"), Message.GetErrorName(), DeserializedMessage.GetErrorName());
	TestEqual(TEXT("Serialize"), Message.GetErrorDescription(), DeserializedMessage.GetErrorDescription());

	return true;
}

}

#endif //WITH_DEV_AUTOMATION_TESTS