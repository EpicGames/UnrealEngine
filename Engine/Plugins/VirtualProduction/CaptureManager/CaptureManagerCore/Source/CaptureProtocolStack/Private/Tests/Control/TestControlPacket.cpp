// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Communication/ControlPacket.h"

#include "Control/Messages/ControlJsonUtilities.h"

#include "Misc/AutomationTest.h"

#include "Tests/Utility.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::CaptureManager
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlPacketDeserializeOneTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlPacket.DeserializeOne.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlPacketDeserializeOneTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'C', 'O', 'N', 'T', 'R', 'O', 'L', '\0' };
	const uint16 Version = 1;
	const FString Payload = TEXT("{\"Hello\": \"World\"}");
	const uint32 Length = Payload.Len();

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(Length) + Length);
	Packet.Append(Header);
	Packet.Append(reinterpret_cast<const uint8*>(&Version), sizeof(Version));
	Packet.Append(reinterpret_cast<const uint8*>(&Length), sizeof(Length));
	Packet.Append(Payload);

	FDataProvider DataProvider(MoveTemp(Packet));

	// Deserialization
	TProtocolResult<FControlPacketHeader> HeaderResult = FControlPacketHeader::Deserialize(DataProvider);
	TestTrue(TEXT("Header Deserialize"), HeaderResult.HasValue());

	FControlPacketHeader ControlHeader = HeaderResult.StealValue();
	TProtocolResult<FControlPacket> PacketProtocolResult = FControlPacket::Deserialize(ControlHeader, DataProvider);
	TestTrue(TEXT("Packet Deserialize"), PacketProtocolResult.HasValue());

	FControlPacket ControlPacket = PacketProtocolResult.StealValue();
	TestEqual(TEXT("Packet Deserialize"), ControlPacket.GetVersion(), Version);
	TestEqual(TEXT("Packet Deserialize"), ControlPacket.GetPayloadSize(), Length);

	TArray<uint8> PayloadData = ControlPacket.GetPayload();

	TSharedPtr<FJsonObject> JsonPayload;
	bool Result = FJsonUtility::CreateJsonFromUTF8Data(PayloadData, JsonPayload);
	TestTrue(TEXT("Packet Deserialize"), Result);

	FString Field;
	TestTrue(TEXT("Packet Deserialize"), JsonPayload->TryGetStringField(TEXT("Hello"), Field));
	TestEqual(TEXT("Packet Deserialize"), TEXT("World"), Field);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlPacketDeserializeOneTestInvalidHeader, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlPacket.DeserializeOne.InvalidHeader", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlPacketDeserializeOneTestInvalidHeader::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'A', 'A', 'A', '\0' };
	const uint16 Version = 1;
	const FString Payload = TEXT("{\"Hello\": \"World\"}");
	const uint32 Length = Payload.Len();

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(Length) + Length);
	Packet.Append(Header);
	Packet.Append(reinterpret_cast<const uint8*>(&Version), sizeof(Version));
	Packet.Append(reinterpret_cast<const uint8*>(&Length), sizeof(Length));
	Packet.Append(Payload);

	FDataProvider DataProvider(MoveTemp(Packet));

	// Deserialization
	TProtocolResult<FControlPacketHeader> HeaderResult = FControlPacketHeader::Deserialize(DataProvider);
	TestTrue(TEXT("Header Deserialize"), HeaderResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlPacketDeserializeMoreTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlPacket.DeserializeMore.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlPacketDeserializeMoreTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'C', 'O', 'N', 'T', 'R', 'O', 'L', '\0' };
	const uint16 Version = 1;
	const FString Payload = TEXT("{\"Hello\": \"World\"}");
	const uint32 Length = Payload.Len();

	constexpr uint32 NumberOfPackets = 5;

	TArray<uint8> Packet;
	for (int32 PacketIndex = 0; PacketIndex < NumberOfPackets; ++PacketIndex)
	{
		Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(Length) + Length);
		Packet.Append(Header);
		Packet.Append(reinterpret_cast<const uint8*>(&Version), sizeof(Version));
		Packet.Append(reinterpret_cast<const uint8*>(&Length), sizeof(Length));
		Packet.Append(Payload);
	}

	FDataProvider DataProvider(MoveTemp(Packet));

	// Deserialization
	uint32 PacketsDeserialized = 0;
	while (PacketsDeserialized != NumberOfPackets)
	{
		TProtocolResult<FControlPacketHeader> HeaderResult = FControlPacketHeader::Deserialize(DataProvider);
		TestTrue(TEXT("Header Deserialize"), HeaderResult.HasValue());

		FControlPacketHeader ControlHeader = HeaderResult.StealValue();
		TProtocolResult<FControlPacket> PacketProtocolResult = FControlPacket::Deserialize(ControlHeader, DataProvider);
		TestTrue(TEXT("Packet Deserialize"), PacketProtocolResult.HasValue());

		FControlPacket ControlPacket = PacketProtocolResult.StealValue();
		TestEqual(TEXT("Packet Deserialize"), ControlPacket.GetVersion(), Version);
		TestEqual(TEXT("Packet Deserialize"), ControlPacket.GetPayloadSize(), Length);

		TArray<uint8> PayloadData = ControlPacket.GetPayload();

		TSharedPtr<FJsonObject> JsonPayload;
		bool Result = FJsonUtility::CreateJsonFromUTF8Data(PayloadData, JsonPayload);
		TestTrue(TEXT("Packet Deserialize"), Result);

		FString Field;
		TestTrue(TEXT("Packet Deserialize"), JsonPayload->TryGetStringField(TEXT("Hello"), Field));
		TestEqual(TEXT("Packet Deserialize"), TEXT("World"), Field);

		++PacketsDeserialized;
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlPacketSerializeOneTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlPacket.SerializeOne.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlPacketSerializeOneTest::RunTest(const FString& InParameters)
{
	FDataSender DataReceiver;

	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'C', 'O', 'N', 'T', 'R', 'O', 'L', '\0' };
	const uint16 Version = 1;
	const FString Payload = TEXT("{\"Hello\":\"World\"}");
	const uint32 Length = Payload.Len();

	TArray<uint8> PayloadData;
	PayloadData.Append(Payload);

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(Length) + Length);
	Packet.Append(Header);
	Packet.Append(reinterpret_cast<const uint8*>(&Version), sizeof(Version));
	Packet.Append(reinterpret_cast<const uint8*>(&Length), sizeof(Length));
	Packet.Append(PayloadData);

	FControlPacket ControlPacket(Version, MoveTemp(PayloadData));

	TestTrue(TEXT("Packet Serialize"), FControlPacket::Serialize(ControlPacket, DataReceiver).HasValue());
	TestEqual(TEXT("Packet Serialize"), Packet, DataReceiver.GetData());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TControlPacketSerializeOneTestFailed, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlPacket.SerializeOne.Failure", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TControlPacketSerializeOneTestFailed::RunTest(const FString& InParameters)
{
	FFailedDataSender DataReceiver;

	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'C', 'O', 'N', 'T', 'R', 'O', 'L', '\0' };
	const uint16 Version = 1;
	const FString Payload = TEXT("{\"Hello\":\"World\"}");
	const uint32 Length = Payload.Len();

	TArray<uint8> PayloadData;
	PayloadData.Append(Payload);

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(Length) + Length);
	Packet.Append(Header);
	Packet.Append(reinterpret_cast<const uint8*>(&Version), sizeof(Version));
	Packet.Append(reinterpret_cast<const uint8*>(&Length), sizeof(Length));
	Packet.Append(PayloadData);

	FControlPacket ControlPacket(Version, MoveTemp(PayloadData));

	TestTrue(TEXT("Packet Serialize"), FControlPacket::Serialize(ControlPacket, DataReceiver).HasError());

	return true;
}

}

#endif //WITH_DEV_AUTOMATION_TESTS