// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Messages/ControlRequest.h"

#include "Control/Messages/ControlJsonUtilities.h"
#include "Control/Messages/Constants.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::CaptureManager
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TGetServerInformationTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlRequest.GetServerInformation.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TGetServerInformationTest::RunTest(const FString& InParameters)
{
	FGetServerInformationRequest Request;

	TestEqual(TEXT("GetAddresPath"), Request.GetAddressPath(), CPS::AddressPaths::GGetServerInformation);
	TestFalse(TEXT("GetBody"), Request.GetBody().IsValid()); // Request doesn't have a Body

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TKeepAliveRequestTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlRequest.KeepAlive.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TKeepAliveRequestTest::RunTest(const FString& InParameters)
{
	FKeepAliveRequest Request;

	TestEqual(TEXT("GetAddresPath"), Request.GetAddressPath(), CPS::AddressPaths::GKeepAlive);
	TestFalse(TEXT("GetBody"), Request.GetBody().IsValid()); // Request doesn't have a Body

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TStartSessionRequestTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlRequest.StartSession.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TStartSessionRequestTest::RunTest(const FString& InParameters)
{
	FStartSessionRequest Request;

	TestEqual(TEXT("GetAddresPath"), Request.GetAddressPath(), CPS::AddressPaths::GStartSession);
	TestFalse(TEXT("GetBody"), Request.GetBody().IsValid()); // Request doesn't have a Body

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TStopSessionRequestTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlRequest.StopSession.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TStopSessionRequestTest::RunTest(const FString& InParameters)
{
	FStopSessionRequest Request;

	TestEqual(TEXT("GetAddresPath"), Request.GetAddressPath(), CPS::AddressPaths::GStopSession);
	TestFalse(TEXT("GetBody"), Request.GetBody().IsValid()); // Request doesn't have a Body

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TSubscribeRequestTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlRequest.Subscribe.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TSubscribeRequestTest::RunTest(const FString& InParameters)
{
	FSubscribeRequest Request;

	TestEqual(TEXT("GetAddresPath"), Request.GetAddressPath(), CPS::AddressPaths::GSubscribe);
	TestFalse(TEXT("GetBody"), Request.GetBody().IsValid()); // Request doesn't have a Body

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TUnsubscribeRequestTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlRequest.Unsubscribe.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TUnsubscribeRequestTest::RunTest(const FString& InParameters)
{
	FUnsubscribeRequest Request;

	TestEqual(TEXT("GetAddresPath"), Request.GetAddressPath(), CPS::AddressPaths::GUnsubscribe);
	TestFalse(TEXT("GetBody"), Request.GetBody().IsValid()); // Request doesn't have a Body

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TGetStateRequestTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlRequest.GetState.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TGetStateRequestTest::RunTest(const FString& InParameters)
{
	FGetStateRequest Request;

	TestEqual(TEXT("GetAddresPath"), Request.GetAddressPath(), CPS::AddressPaths::GGetState);
	TestFalse(TEXT("GetBody"), Request.GetBody().IsValid()); // Request doesn't have a Body

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TStartRecordingTakeRequestTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlRequest.StartRecordingTake.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TStartRecordingTakeRequestTest::RunTest(const FString& InParameters)
{
	FString String = TEXT("{\
        \"slateName\": \"Slate\",\
        \"takeNumber\": 0,\
        \"subject\": \"Subject\",\
        \"scenario\": \"Scenario\",\
        \"tags\": [\"Tag1\", \"Tag2\", \"Tag3\"]\
    }");

	String.ConvertTabsToSpacesInline(2);
	String.RemoveSpacesInline();

	auto ConvertedString = StringCast<UTF8CHAR>(*String);

	FString SlateName = TEXT("Slate");
	FString Subject = TEXT("Subject");
	FString Scenario = TEXT("Scenario");
	TArray<FString> Tags = { TEXT("Tag1"), TEXT("Tag2"), TEXT("Tag3") };

	FStartRecordingTakeRequest Request(SlateName, 0, Subject, Scenario, Tags);

	TArray<uint8> Body;
	TestTrue(TEXT("CreateDataFromJson"), FJsonUtility::CreateUTF8DataFromJson(Request.GetBody(), Body));

	TestEqual(TEXT("GetAddresPath"), Request.GetAddressPath(), CPS::AddressPaths::GStartRecordingTake);

	TArray<uint8> BodyExpected(reinterpret_cast<const uint8*>(ConvertedString.Get()), ConvertedString.Length());
	TestEqual(TEXT("GetBody"), Body, BodyExpected);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TStopRecordingTakeRequestTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlRequest.StopRecordingTake.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TStopRecordingTakeRequestTest::RunTest(const FString& InParameters)
{
	FStopRecordingTakeRequest Request;

	TestEqual(TEXT("GetAddresPath"), Request.GetAddressPath(), CPS::AddressPaths::GStopRecordingTake);
	TestFalse(TEXT("GetBody"), Request.GetBody().IsValid()); // Request doesn't have a Body

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TAbortRecordingTakeRequestTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlRequest.AbortRecordingTake.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TAbortRecordingTakeRequestTest::RunTest(const FString& InParameters)
{
	FAbortRecordingTakeRequest Request;

	TestEqual(TEXT("GetAddresPath"), Request.GetAddressPath(), CPS::AddressPaths::GAbortRecordingTake);
	TestFalse(TEXT("GetBody"), Request.GetBody().IsValid()); // Request doesn't have a Body

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TGetTakeListRequestTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlRequest.GetTakeList.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TGetTakeListRequestTest::RunTest(const FString& InParameters)
{
	FGetTakeListRequest Request;

	TestEqual(TEXT("GetAddresPath"), Request.GetAddressPath(), CPS::AddressPaths::GGetTakeList);
	TestFalse(TEXT("GetBody"), Request.GetBody().IsValid()); // Request doesn't have a Body

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TGetTakeMetadataRequestTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlRequest.GetTakeMetadata.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TGetTakeMetadataRequestTest::RunTest(const FString& InParameters)
{
	FString String = TEXT("{\
        \"names\": [\"TakeName1\", \"TakeName2\", \"TakeName3\"]\
    }");

	String.ConvertTabsToSpacesInline(2);
	String.RemoveSpacesInline();

	auto ConvertedString = StringCast<UTF8CHAR>(*String);

	TArray<FString> Takes = { TEXT("TakeName1"), TEXT("TakeName2"), TEXT("TakeName3") };
	FGetTakeMetadataRequest Request(Takes);

	TArray<uint8> Body;
	TestTrue(TEXT("CreateDataFromJson"), FJsonUtility::CreateUTF8DataFromJson(Request.GetBody(), Body));

	TestEqual(TEXT("GetAddresPath"), Request.GetAddressPath(), CPS::AddressPaths::GGetTakeMetadata);

	TArray<uint8> BodyExpected(reinterpret_cast<const uint8*>(ConvertedString.Get()), ConvertedString.Length());
	TestEqual(TEXT("GetBody"), Body, BodyExpected);

	return true;
}

}

#endif // WITH_DEV_AUTOMATION_TESTS