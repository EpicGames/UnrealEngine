// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Messages/ControlResponse.h"

#include "Control/Messages/ControlJsonUtilities.h"
#include "Control/Messages/Constants.h"

#include "Misc/AutomationTest.h"
#include "Misc/DateTime.h"

#ifdef WITH_DEV_AUTOMATION_TESTS

namespace UE::CaptureManager
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TKeepAliveResponseTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlResponse.KeepAlive.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TKeepAliveResponseTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body;

	FKeepAliveResponse Response;

	TestEqual(TEXT("GetAddresPath"), Response.GetAddressPath(), CPS::AddressPaths::GKeepAlive);
	TestTrue(TEXT("Parse"), Response.Parse(Body).HasValue());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TStartSessionResponseTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlResponse.StartSession.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TStartSessionResponseTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();
	Body->SetStringField(CPS::Properties::GSessionId, TEXT("SessionId"));

	FStartSessionResponse Response;

	TestEqual(TEXT("GetAddresPath"), Response.GetAddressPath(), CPS::AddressPaths::GStartSession);
	TestTrue(TEXT("Parse"), Response.Parse(Body).HasValue());

	TestEqual(TEXT("Parse"), Response.GetSessionId(), TEXT("SessionId"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TStopSessionResponseTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlResponse.StopSession.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TStopSessionResponseTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body;

	FStopSessionResponse Response;

	TestEqual(TEXT("GetAddresPath"), Response.GetAddressPath(), CPS::AddressPaths::GStopSession);
	TestTrue(TEXT("Parse"), Response.Parse(Body).HasValue());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TGetServerInformationResponseTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlResponse.GetServerInformation.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TGetServerInformationResponseTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();
	Body->SetStringField(CPS::Properties::GId, TEXT("Id"));
	Body->SetStringField(CPS::Properties::GName, TEXT("Name"));
	Body->SetStringField(CPS::Properties::GModel, TEXT("Model"));
	Body->SetStringField(CPS::Properties::GPlatformName, TEXT("PlatformName"));
	Body->SetStringField(CPS::Properties::GPlatformVersion, TEXT("PlatformVersion"));
	Body->SetStringField(CPS::Properties::GSoftwareName, TEXT("SoftwareName"));
	Body->SetStringField(CPS::Properties::GSoftwareVersion, TEXT("SoftwareVersion"));
	Body->SetNumberField(CPS::Properties::GExportPort, 12345);

	FGetServerInformationResponse Response;

	TestEqual(TEXT("GetAddresPath"), Response.GetAddressPath(), CPS::AddressPaths::GGetServerInformation);
	TestTrue(TEXT("Parse"), Response.Parse(Body).HasValue());

	TestEqual(TEXT("Parse"), Response.GetId(), TEXT("Id"));
	TestEqual(TEXT("Parse"), Response.GetName(), TEXT("Name"));
	TestEqual(TEXT("Parse"), Response.GetModel(), TEXT("Model"));
	TestEqual(TEXT("Parse"), Response.GetPlatformName(), TEXT("PlatformName"));
	TestEqual(TEXT("Parse"), Response.GetPlatformVersion(), TEXT("PlatformVersion"));
	TestEqual(TEXT("Parse"), Response.GetSoftwareName(), TEXT("SoftwareName"));
	TestEqual(TEXT("Parse"), Response.GetSoftwareVersion(), TEXT("SoftwareVersion"));
	TestEqual(TEXT("Parse"), Response.GetExportPort(), 12345);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TSubscribeResponseTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlResponse.Subscribe.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TSubscribeResponseTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body;

	FSubscribeResponse Response;

	TestEqual(TEXT("GetAddresPath"), Response.GetAddressPath(), CPS::AddressPaths::GSubscribe);
	TestTrue(TEXT("Parse"), Response.Parse(Body).HasValue());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TUnsubscribeResponseTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlResponse.Unsubscribe.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TUnsubscribeResponseTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body;

	FUnsubscribeResponse Response;

	TestEqual(TEXT("GetAddresPath"), Response.GetAddressPath(), CPS::AddressPaths::GUnsubscribe);
	TestTrue(TEXT("Parse"), Response.Parse(Body).HasValue());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TGetStateResponseTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlResponse.GetState.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TGetStateResponseTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();
	Body->SetBoolField(CPS::Properties::GIsRecording, true);

	TSharedPtr<FJsonObject> IOSSpecific = MakeShared<FJsonObject>();
	IOSSpecific->SetNumberField(CPS::Properties::GTotalCapacity, 100);
	IOSSpecific->SetNumberField(CPS::Properties::GRemainingCapacity, 100);
	IOSSpecific->SetNumberField(CPS::Properties::GBatteryLevel, 100.0);
	IOSSpecific->SetStringField(CPS::Properties::GThermalState, CPS::Properties::GNominal);

	Body->SetObjectField(CPS::Properties::GPlatformState, IOSSpecific);

	FGetStateResponse Response;

	TestEqual(TEXT("GetAddresPath"), Response.GetAddressPath(), CPS::AddressPaths::GGetState);
	TestTrue(TEXT("Parse"), Response.Parse(Body).HasValue());

	TestEqual(TEXT("Parse"), Response.IsRecording(), true);

	FJsonValueObject PlatformStateObj(Response.GetPlatformState());
	FJsonValueObject IOSSpecificObj(IOSSpecific);

	TestTrue(TEXT("Parse"), FJsonValue::CompareEqual(PlatformStateObj, IOSSpecificObj));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TStartRecordingTakeResponseTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlResponse.StartRecordingTake.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TStartRecordingTakeResponseTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body;

	FStartRecordingTakeResponse Response;

	TestEqual(TEXT("GetAddresPath"), Response.GetAddressPath(), CPS::AddressPaths::GStartRecordingTake);
	TestTrue(TEXT("Parse"), Response.Parse(Body).HasValue());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TStopRecordingTakeResponseTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlResponse.StopRecordingTake.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TStopRecordingTakeResponseTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();
	Body->SetStringField(CPS::Properties::GName, TEXT("TakeName"));

	FStopRecordingTakeResponse Response;

	TestEqual(TEXT("GetAddresPath"), Response.GetAddressPath(), CPS::AddressPaths::GStopRecordingTake);
	TestTrue(TEXT("Parse"), Response.Parse(Body).HasValue());

	TestEqual(TEXT("Parse"), Response.GetTakeName(), TEXT("TakeName"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TAbortRecordingTakeResponseTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlResponse.AbortRecordingTake.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TAbortRecordingTakeResponseTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body;

	FAbortRecordingTakeResponse Response;

	TestEqual(TEXT("GetAddresPath"), Response.GetAddressPath(), CPS::AddressPaths::GAbortRecordingTake);
	TestTrue(TEXT("Parse"), Response.Parse(Body).HasValue());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TGetTakeListResponseTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlResponse.GetTakeList.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TGetTakeListResponseTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();

	TArray<TSharedPtr<FJsonValue>> NamesJson = {
		MakeShared<FJsonValueString>(TEXT("Name1")),
		MakeShared<FJsonValueString>(TEXT("Name2")),
		MakeShared<FJsonValueString>(TEXT("Name3")),
	};

	Body->SetArrayField(CPS::Properties::GNames, NamesJson);

	FGetTakeListResponse Response;

	TestEqual(TEXT("GetAddresPath"), Response.GetAddressPath(), CPS::AddressPaths::GGetTakeList);
	TestTrue(TEXT("Parse"), Response.Parse(Body).HasValue());

	const TArray<FString>& Names = Response.GetNames();
	TestEqual(TEXT("Parse"), Names[0], TEXT("Name1"));
	TestEqual(TEXT("Parse"), Names[1], TEXT("Name2"));
	TestEqual(TEXT("Parse"), Names[2], TEXT("Name3"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TGetTakeMetadataResponseTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlResponse.GetTakeMetadata.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TGetTakeMetadataResponseTest::RunTest(const FString& InParameters)
{

	FString Date = FDateTime::Now().ToIso8601();
	TSharedPtr<FJsonObject> TakeJson = MakeShared<FJsonObject>();
	TakeJson->SetStringField(CPS::Properties::GName, TEXT("Name"));
	TakeJson->SetStringField(CPS::Properties::GSlateName, TEXT("Slate"));
	TakeJson->SetNumberField(CPS::Properties::GTakeNumber, 0);
	TakeJson->SetStringField(CPS::Properties::GDateTime, Date);
	TakeJson->SetStringField(CPS::Properties::GAppVersion, TEXT("AppVersion"));
	TakeJson->SetStringField(CPS::Properties::GModel, TEXT("Model"));
	TakeJson->SetStringField(CPS::Properties::GSubject, TEXT("Subject"));
	TakeJson->SetStringField(CPS::Properties::GScenario, TEXT("Scenario"));

	TArray<TSharedPtr<FJsonValue>> TagsJson = {
		MakeShared<FJsonValueString>(TEXT("Tag1")),
		MakeShared<FJsonValueString>(TEXT("Tag2")),
		MakeShared<FJsonValueString>(TEXT("Tag3")),
	};

	TakeJson->SetArrayField(CPS::Properties::GTags, TagsJson);

	TSharedPtr<FJsonObject> FileJson1 = MakeShared<FJsonObject>();
	FileJson1->SetStringField(CPS::Properties::GName, TEXT("File1"));
	FileJson1->SetNumberField(CPS::Properties::GLength, 1024);

	TSharedPtr<FJsonObject> FileJson2 = MakeShared<FJsonObject>();
	FileJson2->SetStringField(CPS::Properties::GName, TEXT("File2"));
	FileJson2->SetNumberField(CPS::Properties::GLength, 1024);

	TSharedPtr<FJsonObject> FileJson3 = MakeShared<FJsonObject>();
	FileJson3->SetStringField(CPS::Properties::GName, TEXT("File3"));
	FileJson3->SetNumberField(CPS::Properties::GLength, 1024);

	TArray<TSharedPtr<FJsonValue>> FilesJson = {
		MakeShared<FJsonValueObject>(FileJson1),
		MakeShared<FJsonValueObject>(FileJson2),
		MakeShared<FJsonValueObject>(FileJson3)
	};

	TakeJson->SetArrayField(CPS::Properties::GFiles, FilesJson);

	TSharedPtr<FJsonObject> VideoMetadataJson = MakeShared<FJsonObject>();
	VideoMetadataJson->SetNumberField(CPS::Properties::GFrames, 600);
	VideoMetadataJson->SetNumberField(CPS::Properties::GFrameRate, 60);
	VideoMetadataJson->SetNumberField(CPS::Properties::GHeight, 1024);
	VideoMetadataJson->SetNumberField(CPS::Properties::GWidth, 1024);

	TakeJson->SetObjectField(CPS::Properties::GVideo, VideoMetadataJson);

	TSharedPtr<FJsonObject> AudioMetadataJson = MakeShared<FJsonObject>();
	AudioMetadataJson->SetNumberField(CPS::Properties::GChannels, 2);
	AudioMetadataJson->SetNumberField(CPS::Properties::GSampleRate, 44100);
	AudioMetadataJson->SetNumberField(CPS::Properties::GBitsPerChannel, 8);

	TakeJson->SetObjectField(CPS::Properties::GAudio, AudioMetadataJson);

	TArray<TSharedPtr<FJsonValue>> TakesJson = {
		MakeShared<FJsonValueObject>(TakeJson)
	};

	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();
	Body->SetArrayField(CPS::Properties::GTakes, TakesJson);

	FGetTakeMetadataResponse Response;

	TestEqual(TEXT("GetAddresPath"), Response.GetAddressPath(), CPS::AddressPaths::GGetTakeMetadata);
	TestTrue(TEXT("Parse"), Response.Parse(Body).HasValue());

	const TArray<FGetTakeMetadataResponse::FTakeObject>& Takes = Response.GetTakes();
	const FGetTakeMetadataResponse::FTakeObject& Take = Takes[0];

	TestEqual(TEXT("Parse"), Take.Name, TEXT("Name"));
	TestEqual(TEXT("Parse"), Take.Slate, TEXT("Slate"));
	TestEqual(TEXT("Parse"), Take.TakeNumber, (uint16) 0);
	TestEqual(TEXT("Parse"), Take.DateTime, Date);
	TestEqual(TEXT("Parse"), Take.AppVersion, TEXT("AppVersion"));
	TestEqual(TEXT("Parse"), Take.Model, TEXT("Model"));
	TestEqual(TEXT("Parse"), Take.Subject, TEXT("Subject"));
	TestEqual(TEXT("Parse"), Take.Scenario, TEXT("Scenario"));

	TestEqual(TEXT("Parse"), Take.Tags[0], TEXT("Tag1"));
	TestEqual(TEXT("Parse"), Take.Tags[1], TEXT("Tag2"));
	TestEqual(TEXT("Parse"), Take.Tags[2], TEXT("Tag3"));

	TestEqual(TEXT("Parse"), Take.Files[0].Name, TEXT("File1"));
	TestEqual(TEXT("Parse"), Take.Files[0].Length, (uint64) 1024);
	TestEqual(TEXT("Parse"), Take.Files[1].Name, TEXT("File2"));
	TestEqual(TEXT("Parse"), Take.Files[1].Length, (uint64) 1024);
	TestEqual(TEXT("Parse"), Take.Files[2].Name, TEXT("File3"));
	TestEqual(TEXT("Parse"), Take.Files[2].Length, (uint64) 1024);

	TestEqual(TEXT("Parse"), Take.Video.Frames, (uint64) 600);
	TestEqual(TEXT("Parse"), Take.Video.FrameRate, (uint16) 60);
	TestEqual(TEXT("Parse"), Take.Video.Height, (uint32) 1024);
	TestEqual(TEXT("Parse"), Take.Video.Width, (uint32) 1024);

	TestEqual(TEXT("Parse"), Take.Audio.Channels, (uint8) 2);
	TestEqual(TEXT("Parse"), Take.Audio.SampleRate, (uint32) 44100);
	TestEqual(TEXT("Parse"), Take.Audio.BitsPerChannel, (uint8) 8);

	return true;
}

}

#endif // WITH_DEV_AUTOMATION_TESTS