// Copyright Epic Games, Inc. All Rights Reserved.

#include "Control/Messages/ControlUpdate.h"

#include "Control/Messages/ControlJsonUtilities.h"
#include "Control/Messages/Constants.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::CaptureManager
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TSessionStoppedUpdateTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlUpdate.SessionStopped.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TSessionStoppedUpdateTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body;

	FSessionStopped Update;

	TestEqual(TEXT("GetAddresPath"), Update.GetAddressPath(), CPS::AddressPaths::GSessionStopped);
	TestTrue(TEXT("Parse"), Update.Parse(Body).HasValue());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TTakeAddedUpdateTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlUpdate.TakeAdded.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TTakeAddedUpdateTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();
	Body->SetStringField(CPS::Properties::GName, TEXT("TakeName"));

	FTakeAddedUpdate Update;

	TestEqual(TEXT("GetAddresPath"), Update.GetAddressPath(), CPS::AddressPaths::GTakeAdded);

	TProtocolResult<void> UpdateParseResult = Update.Parse(Body);
	TestTrue(TEXT("Parse"), UpdateParseResult.HasValue());
	TestEqual(TEXT("Parse"), Update.GetTakeName(), TEXT("TakeName"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TTakeRemovedUpdateTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlUpdate.TakeRemoved.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TTakeRemovedUpdateTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();
	Body->SetStringField(CPS::Properties::GName, TEXT("TakeName"));

	FTakeRemovedUpdate Update;

	TestEqual(TEXT("GetAddresPath"), Update.GetAddressPath(), CPS::AddressPaths::GTakeRemoved);

	TProtocolResult<void> UpdateParseResult = Update.Parse(Body);
	TestTrue(TEXT("Parse"), UpdateParseResult.HasValue());
	TestEqual(TEXT("Parse"), Update.GetTakeName(), TEXT("TakeName"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TTakeUpdatedUpdateTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlUpdate.TakeUpdated.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TTakeUpdatedUpdateTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();
	Body->SetStringField(CPS::Properties::GName, TEXT("TakeName"));

	FTakeUpdatedUpdate Update;

	TestEqual(TEXT("GetAddresPath"), Update.GetAddressPath(), CPS::AddressPaths::GTakeUpdated);

	TProtocolResult<void> UpdateParseResult = Update.Parse(Body);
	TestTrue(TEXT("Parse"), UpdateParseResult.HasValue());
	TestEqual(TEXT("Parse"), Update.GetTakeName(), TEXT("TakeName"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TRecordingStatusUpdateTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlUpdate.RecordingStatus.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TRecordingStatusUpdateTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();
	Body->SetBoolField(CPS::Properties::GIsRecording, true);

	FRecordingStatusUpdate Update;

	TestEqual(TEXT("GetAddresPath"), Update.GetAddressPath(), CPS::AddressPaths::GRecordingStatus);

	TProtocolResult<void> UpdateParseResult = Update.Parse(Body);
	TestTrue(TEXT("Parse"), UpdateParseResult.HasValue());
	TestEqual(TEXT("Parse"), Update.IsRecording(), true);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiskCapacityUpdateTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlUpdate.DiskCapacity.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiskCapacityUpdateTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();
	Body->SetNumberField(CPS::Properties::GTotal, 1024);
	Body->SetNumberField(CPS::Properties::GRemaining, 512);

	FDiskCapacityUpdate Update;

	TestEqual(TEXT("GetAddresPath"), Update.GetAddressPath(), CPS::AddressPaths::GDiskCapacity);

	TProtocolResult<void> UpdateParseResult = Update.Parse(Body);
	TestTrue(TEXT("Parse"), UpdateParseResult.HasValue());
	TestEqual(TEXT("Parse"), Update.GetTotal(), (uint64) 1024);
	TestEqual(TEXT("Parse"), Update.GetRemaining(), (uint64) 512);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TBatteryPercentageUpdateTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlUpdate.BatteryPercentage.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TBatteryPercentageUpdateTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();
	Body->SetNumberField(CPS::Properties::GLevel, 100.0);

	FBatteryPercentageUpdate Update;

	TestEqual(TEXT("GetAddresPath"), Update.GetAddressPath(), CPS::AddressPaths::GBattery);

	TProtocolResult<void> UpdateParseResult = Update.Parse(Body);
	TestTrue(TEXT("Parse"), UpdateParseResult.HasValue());
	TestEqual(TEXT("Parse"), Update.GetLevel(), (float) 100.0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TThermalStateUpdateTest, "MetaHuman.Capture.CaptureProtocolStack.Control.ControlUpdate.ThermalState.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TThermalStateUpdateTest::RunTest(const FString& InParameters)
{
	TSharedPtr<FJsonObject> Body = MakeShared<FJsonObject>();

	Body->SetStringField(CPS::Properties::GState, TEXT("nominal"));

	FThermalStateUpdate Update;

	TestEqual(TEXT("GetAddresPath"), Update.GetAddressPath(), CPS::AddressPaths::GThermalState);

	TProtocolResult<void> UpdateParseResult = Update.Parse(Body);
	TestTrue(TEXT("Parse"), UpdateParseResult.HasValue());
	TestEqual(TEXT("Parse"), Update.GetState(), FThermalStateUpdate::EState::Nominal);

	return true;
}

}

#endif // WITH_DEV_AUTOMATION_TESTS