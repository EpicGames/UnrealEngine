// Copyright Epic Games, Inc. All Rights Reserved.

#include "Discovery/Messages/DiscoveryNotify.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::CaptureManager
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryNotifyDeserializeTest, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryNotify.Deserialize.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryNotifyDeserializeTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'D', 'I', 'S', 'C', 'O', 'V', 'E', 'R', '\0' };
	constexpr uint16 Version = 2;
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Notify;
	const TArray<uint8> ServerId = { 0x0d, 0x0e, 0x0a, 0x0d, 0x0b, 0x0e, 0x0e, 0x0f, 0x0d, 0x0e, 0x0a, 0x0d, 0x0b, 0x0e, 0x0e, 0x0f };
	const FString ServerName = "Test Server Name";
	const auto UTF8ServerName = StringCast<UTF8CHAR>(*ServerName);
	const uint8 ServerNameLength = UTF8ServerName.Length();
	const uint16 ControlPort = 8000;
	const FDiscoveryNotify::EConnectionState ConnectionState = FDiscoveryNotify::EConnectionState::Online;

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType) + ServerId.Num() +sizeof(ServerNameLength) + ServerNameLength + sizeof(ControlPort));
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));
	Packet.Append(ServerId);
	Packet.Append(&ServerNameLength, sizeof(ServerNameLength));
	Packet.Append((uint8*) UTF8ServerName.Get(), UTF8ServerName.Length());
	Packet.Append((uint8*) &ControlPort, sizeof(ControlPort));
	Packet.Add(static_cast<uint8>(ConnectionState));

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryPacket::Deserialize(Packet);
	TestTrue(TEXT("Packet Deserialize"), PacketResult.HasValue());

	FDiscoveryPacket DiscoveryPacket = PacketResult.StealValue();
	TProtocolResult<FDiscoveryNotify> NotifyResult = FDiscoveryNotify::Deserialize(DiscoveryPacket);
	TestTrue(TEXT("Notify Deserialize"), NotifyResult.HasValue());

	FDiscoveryNotify Notify = NotifyResult.StealValue();
	int32 ServerIdCompareResult = FMemory::Memcmp(Notify.GetServerId().GetData(), ServerId.GetData(), ServerId.Num());
	TestTrue(TEXT("Server Id"), (ServerIdCompareResult == 0));
	TestEqual(TEXT("Server Name"), Notify.GetServerName(), ServerName);
	TestEqual(TEXT("Control Port"), Notify.GetControlPort(), ControlPort);
	TestEqual(TEXT("Connection State"), Notify.GetConnectionState(), ConnectionState);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryNotifyDeserializeTestInvalidMessageType, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryNotify.Deserialize.InvalidMessageType", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryNotifyDeserializeTestInvalidMessageType::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'D', 'I', 'S', 'C', 'O', 'V', 'E', 'R', '\0' };
	constexpr uint16 Version = 2;
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Request;

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType));
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryPacket::Deserialize(Packet);
	TestTrue(TEXT("Packet Deserialize"), PacketResult.HasValue());

	FDiscoveryPacket DiscoveryPacket = PacketResult.StealValue();
	TProtocolResult<FDiscoveryNotify> NotifyResult = FDiscoveryNotify::Deserialize(DiscoveryPacket);
	TestTrue(TEXT("Notify Deserialize"), NotifyResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryNotifyDeserializeTestInvalidSize, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryNotify.Deserialize.InvalidSize", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryNotifyDeserializeTestInvalidSize::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'D', 'I', 'S', 'C', 'O', 'V', 'E', 'R', '\0' };
	constexpr uint16 Version = 2;
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Notify;
	const TArray<uint8> ServerId = { 0x0d, 0x0e, 0x0a, 0x0d, 0x0b, 0x0e, 0x0e, 0x0f };

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType));
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));
	Packet.Append(ServerId);

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryPacket::Deserialize(Packet);
	TestTrue(TEXT("Packet Deserialize"), PacketResult.HasValue());

	FDiscoveryPacket DiscoveryPacket = PacketResult.StealValue();
	TProtocolResult<FDiscoveryNotify> NotifyResult = FDiscoveryNotify::Deserialize(DiscoveryPacket);
	TestTrue(TEXT("Notify Deserialize"), NotifyResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryNotifyDeserializeTestInvalidConnectionState, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryNotify.Deserialize.InvalidConnectionState", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryNotifyDeserializeTestInvalidConnectionState::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'D', 'I', 'S', 'C', 'O', 'V', 'E', 'R', '\0' };
	constexpr uint16 Version = 2;
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Notify;
	const TArray<uint8> ServerId = { 0x0d, 0x0e, 0x0a, 0x0d, 0x0b, 0x0e, 0x0e, 0x0f, 0x0d, 0x0e, 0x0a, 0x0d, 0x0b, 0x0e, 0x0e, 0x0f };
	const FString ServerName = "Test Server Name";
	const auto UTF8ServerName = StringCast<UTF8CHAR>(*ServerName);
	const uint8 ServerNameLength = UTF8ServerName.Length();
	const uint16 ControlPort = 8000;
	const FDiscoveryNotify::EConnectionState ConnectionState = FDiscoveryNotify::EConnectionState::Invalid;

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType) + ServerId.Num() + sizeof(ServerNameLength) + ServerNameLength + sizeof(ControlPort));
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));
	Packet.Append(ServerId);
	Packet.Append(&ServerNameLength, sizeof(ServerNameLength));
	Packet.Append((uint8*) UTF8ServerName.Get(), UTF8ServerName.Length());
	Packet.Append((uint8*) &ControlPort, sizeof(ControlPort));
	Packet.Add(static_cast<uint8>(ConnectionState));

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryPacket::Deserialize(Packet);
	TestTrue(TEXT("Packet Deserialize"), PacketResult.HasValue());

	FDiscoveryPacket DiscoveryPacket = PacketResult.StealValue();
	TProtocolResult<FDiscoveryNotify> NotifyResult = FDiscoveryNotify::Deserialize(DiscoveryPacket);
	TestTrue(TEXT("Notify Deserialize"), NotifyResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryNotifySerializeTest, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryNotify.Serialize.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryNotifySerializeTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Notify;

	const TArray<uint8> ServerId = { 0x0d, 0x0e, 0x0a, 0x0d, 0x0b, 0x0e, 0x0e, 0x0f, 0x0d, 0x0e, 0x0a, 0x0d, 0x0b, 0x0e, 0x0e, 0x0f };
	constexpr uint16 Version = 2;
	const FString ServerName = "Test Server Name";
	const auto UTF8ServerName = StringCast<UTF8CHAR>(*ServerName);
	const uint8 ServerNameLength = UTF8ServerName.Length();
	const uint16 ControlPort = 8000;
	const FDiscoveryNotify::EConnectionState ConnectionState = FDiscoveryNotify::EConnectionState::Online;

	TArray<uint8> Payload;
	Payload.Reserve(ServerId.Num() + sizeof(ServerNameLength) + ServerNameLength + sizeof(ControlPort));
	Payload.Append(ServerId);
	Payload.Append(&ServerNameLength, sizeof(ServerNameLength));
    Payload.Append((uint8*) UTF8ServerName.Get(), UTF8ServerName.Length());
	Payload.Append((uint8*) &ControlPort, sizeof(ControlPort));
	Payload.Add(static_cast<uint8>(ConnectionState));

	FDiscoveryNotify::FServerId ServerIdStatic;
	FMemory::Memcpy(ServerIdStatic.GetData(), ServerId.GetData(), ServerId.Num());

	FDiscoveryNotify Notify(MoveTemp(ServerIdStatic), ServerName, ControlPort, ConnectionState);

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryNotify::Serialize(Notify);
	TestTrue(TEXT("Packet Serialize"), PacketResult.HasValue());

	FDiscoveryPacket DiscoveryPacket = PacketResult.StealValue();
	TestEqual(TEXT("Message Type"), DiscoveryPacket.GetMessageType(), MessageType);
	TestEqual(TEXT("Payload"), DiscoveryPacket.GetPayload(), Payload);

	return true;
}

}

#endif // WITH_DEV_AUTOMATION_TESTS