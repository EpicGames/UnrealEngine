// Copyright Epic Games, Inc. All Rights Reserved.

#include "Discovery/Communication/DiscoveryPacket.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::CaptureManager
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryPacketDeserializeTest, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryPacket.Deserialize.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryPacketDeserializeTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'D', 'I', 'S', 'C', 'O', 'V', 'E', 'R', '\0' };
	constexpr uint16 Version = 2;
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Request;
	const TArray<uint8> Payload = { 0x0d, 0x0e, 0x0a, 0x0d, 0x0b, 0x0e, 0x0e, 0x0f, 0x00 };

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType) + Payload.Num());
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));
	Packet.Append(Payload);

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryPacket::Deserialize(Packet);
	TestTrue(TEXT("Packet Deserialize"), PacketResult.HasValue());

	FDiscoveryPacket DiscoveryPacket = PacketResult.StealValue();
	TestEqual(TEXT("Message Type"), DiscoveryPacket.GetMessageType(), MessageType);
	TestEqual(TEXT("Payload Size"), DiscoveryPacket.GetPayload().Num(), Payload.Num());
	TestEqual(TEXT("Payload"), DiscoveryPacket.GetPayload(), Payload);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryPacketDeserializeTestInvalidHeaderSize, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryPacket.Deserialize.InvalidHeaderSize", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryPacketDeserializeTestInvalidHeaderSize::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'A', 'A', 'A', '\0' };
	constexpr uint16 Version = 2;
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Request;

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType));
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryPacket::Deserialize(Packet);
	TestTrue(TEXT("Packet Deserialize"), PacketResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryPacketDeserializeTestInvalidHeader, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryPacket.Deserialize.InvalidHeader", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryPacketDeserializeTestInvalidHeader::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'A', 'A', 'A', 'A', 'A', 'A', 'A', 'A', 'A', 'A', 'A', '\0' };
	constexpr uint16 Version = 2;
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Request;

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType));
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryPacket::Deserialize(Packet);
	TestTrue(TEXT("Packet Deserialize"), PacketResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryPacketDeserializeTestInvalidVersion, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryPacket.Deserialize.InvalidVersion", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryPacketDeserializeTestInvalidVersion::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'D', 'I', 'S', 'C', 'O', 'V', 'E', 'R', '\0' };
	constexpr uint16 Version = 1; // Invalid Version
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Request;

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType));
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryPacket::Deserialize(Packet);
	TestTrue(TEXT("Packet Deserialize"), PacketResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryPacketDeserializeTestInvalidMessageType, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryPacket.Deserialize.InvalidMessageType", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryPacketDeserializeTestInvalidMessageType::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'D', 'I', 'S', 'C', 'O', 'V', 'E', 'R', '\0' };
	constexpr uint16 Version = 2;
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Invalid;

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType));
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryPacket::Deserialize(Packet);
	TestTrue(TEXT("Packet Deserialize"), PacketResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryPacketSerializeTest, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryPacket.Serialize.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryPacketSerializeTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'D', 'I', 'S', 'C', 'O', 'V', 'E', 'R', '\0' };
	constexpr uint16 Version = 2;
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Request;

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType));
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));

	FDiscoveryPacket DiscoveryPacket(MessageType, TArray<uint8>());

	// Deserialization
	TProtocolResult<TArray<uint8>> DataResult = FDiscoveryPacket::Serialize(DiscoveryPacket);
	TestTrue(TEXT("Packet Serialize"), DataResult.HasValue());

	TArray<uint8> Data = DataResult.StealValue();
	TestEqual(TEXT("Packet Serialize"), Packet, Data);

	return true;
}

}

#endif //WITH_DEV_AUTOMATION_TESTS