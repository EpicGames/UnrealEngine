// Copyright Epic Games, Inc. All Rights Reserved.

#include "Discovery/Messages/DiscoveryRequest.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::CaptureManager
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryRequestDeserializeTest, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryRequest.Deserialize.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryRequestDeserializeTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'D', 'I', 'S', 'C', 'O', 'V', 'E', 'R', '\0' };
	constexpr uint16 Version = 2;
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Request;

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType));
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryPacket::Deserialize(Packet);
	TestTrue(TEXT("Packet Deserialize"), PacketResult.HasValue());

	FDiscoveryPacket DiscoveryPacket = PacketResult.StealValue();
	TProtocolResult<FDiscoveryRequest> RequestResult = FDiscoveryRequest::Deserialize(DiscoveryPacket);
	TestTrue(TEXT("Request Deserialize"), RequestResult.HasValue());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryRequestDeserializeTestInvalidMessageType, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryRequest.Deserialize.InvalidMessageType", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryRequestDeserializeTestInvalidMessageType::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'D', 'I', 'S', 'C', 'O', 'V', 'E', 'R', '\0' };
	constexpr uint16 Version = 2;
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Response;

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType));
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryPacket::Deserialize(Packet);
	TestTrue(TEXT("Packet Deserialize"), PacketResult.HasValue());

	FDiscoveryPacket DiscoveryPacket = PacketResult.StealValue();

	TProtocolResult<FDiscoveryRequest> RequestResult = FDiscoveryRequest::Deserialize(DiscoveryPacket);
	TestTrue(TEXT("Request Deserialize"), RequestResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryRequestSerializeTest, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryRequest.Serialize.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryRequestSerializeTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Request;

	FDiscoveryRequest Request;

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryRequest::Serialize(Request);
	TestTrue(TEXT("Request Serialize"), PacketResult.HasValue());

	FDiscoveryPacket DiscoveryPacket = PacketResult.StealValue();
	TestEqual(TEXT("Request Serialize"), DiscoveryPacket.GetMessageType(), MessageType);
	TestTrue(TEXT("Request Serialize"), DiscoveryPacket.GetPayload().IsEmpty());

	return true;
}

}

#endif // WITH_DEV_AUTOMATION_TESTS