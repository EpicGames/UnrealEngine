// Copyright Epic Games, Inc. All Rights Reserved.

#include "Discovery/Messages/DiscoveryResponse.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::CaptureManager
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryResponseDeserializeTest, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryResponse.Deserialize.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryResponseDeserializeTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'D', 'I', 'S', 'C', 'O', 'V', 'E', 'R', '\0' };
	constexpr uint16 Version = 2;
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Response;
	const TArray<uint8> ServerId = { 0x0d, 0x0e, 0x0a, 0x0d, 0x0b, 0x0e, 0x0e, 0x0f, 0x0d, 0x0e, 0x0a, 0x0d, 0x0b, 0x0e, 0x0e, 0x0f };
	const FString ServerName = "Test Server Name";
	const auto UTF8ServerName = StringCast<UTF8CHAR>(*ServerName);
	const uint8 ServerNameLength = UTF8ServerName.Length();
	const uint16 ControlPort = 8000;

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType) + ServerId.Num() + sizeof(ServerNameLength) + ServerNameLength + sizeof(ControlPort));
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));
	Packet.Append(ServerId);
	Packet.Append(&ServerNameLength, sizeof(ServerNameLength));
	Packet.Append((uint8*) UTF8ServerName.Get(), UTF8ServerName.Length());
	Packet.Append((uint8*) &ControlPort, sizeof(ControlPort));

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryPacket::Deserialize(Packet);
	TestTrue(TEXT("Packet Deserialize"), PacketResult.HasValue());

	FDiscoveryPacket DiscoveryPacket = PacketResult.StealValue();
	TProtocolResult<FDiscoveryResponse> ResponseResult = FDiscoveryResponse::Deserialize(DiscoveryPacket);
	TestTrue(TEXT("Response Deserialize"), ResponseResult.HasValue());

	FDiscoveryResponse Response = ResponseResult.StealValue();
	int32 ServerIdCompareResult = FMemory::Memcmp(Response.GetServerId().GetData(), ServerId.GetData(), ServerId.Num());
	TestTrue(TEXT("ServerId"), (ServerIdCompareResult == 0));
	TestEqual(TEXT("Server Name"), Response.GetServerName(), ServerName);
	TestEqual(TEXT("Control Port"), Response.GetControlPort(), ControlPort);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryResponseDeserializeTestInvalidMessageType, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryResponse.Deserialize.InvalidMessageType", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryResponseDeserializeTestInvalidMessageType::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'D', 'I', 'S', 'C', 'O', 'V', 'E', 'R', '\0' };
	constexpr uint16 Version = 2;
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Request;

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType));
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryPacket::Deserialize(Packet);
	TestTrue(TEXT("Packet Deserialize"), PacketResult.HasValue());

	FDiscoveryPacket DiscoveryPacket = PacketResult.StealValue();
	TProtocolResult<FDiscoveryResponse> ResponseResult = FDiscoveryResponse::Deserialize(DiscoveryPacket);
	TestTrue(TEXT("Response Deserialize"), ResponseResult.HasError());

	return true;
}
	
IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryResponseDeserializeTestInvalidSize, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryResponse.Deserialize.InvalidSize", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryResponseDeserializeTestInvalidSize::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'D', 'I', 'S', 'C', 'O', 'V', 'E', 'R', '\0' };
	constexpr uint16 Version = 2;
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Response;
	const TArray<uint8> ServerId = { 0x0d, 0x0e, 0x0a, 0x0d, 0x0b, 0x0e, 0x0e, 0x0f };

	TArray<uint8> Packet;
	Packet.Reserve(Header.Num() + sizeof(Version) + sizeof(MessageType));
	Packet.Append(Header);
	Packet.Append((uint8*) &Version, sizeof(Version));
	Packet.Add(static_cast<uint8>(MessageType));
	Packet.Append(ServerId);

	// Deserialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryPacket::Deserialize(Packet);
	TestTrue(TEXT("Packet Deserialize"), PacketResult.HasValue());

	FDiscoveryPacket DiscoveryPacket = PacketResult.StealValue();
	TProtocolResult<FDiscoveryResponse> ResponseResult = FDiscoveryResponse::Deserialize(DiscoveryPacket);
	TestTrue(TEXT("Response Deserialize"), ResponseResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TDiscoveryResponseSerializeTest, "MetaHuman.Capture.CaptureProtocolStack.Discovery.DiscoveryResponse.Serialize.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TDiscoveryResponseSerializeTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const FDiscoveryPacket::EMessageType MessageType = FDiscoveryPacket::EMessageType::Response;

	const TArray<uint8> ServerId = { 0x0d, 0x0e, 0x0a, 0x0d, 0x0b, 0x0e, 0x0e, 0x0f, 0x0d, 0x0e, 0x0a, 0x0d, 0x0b, 0x0e, 0x0e, 0x0f };
	const FString ServerName = "Test Server Name";
	const auto UTF8ServerName = StringCast<UTF8CHAR>(*ServerName);
	const uint8 ServerNameLength = UTF8ServerName.Length();
	const uint16 ControlPort = 8000;

	TArray<uint8> Payload;
	Payload.Reserve(ServerId.Num() + sizeof(ServerNameLength) + ServerNameLength + sizeof(ControlPort));
	Payload.Append(ServerId);
	Payload.Append(&ServerNameLength, sizeof(ServerNameLength));
	Payload.Append((uint8*) UTF8ServerName.Get(), UTF8ServerName.Length());
	Payload.Append((uint8*) &ControlPort, sizeof(ControlPort));

	FDiscoveryResponse::FServerId ServerIdStatic;
	FMemory::Memcpy(ServerIdStatic.GetData(), ServerId.GetData(), ServerId.Num());

	FDiscoveryResponse Response(MoveTemp(ServerIdStatic), ServerName, ControlPort);

	// Serialization
	TProtocolResult<FDiscoveryPacket> PacketResult = FDiscoveryResponse::Serialize(Response);
	TestTrue(TEXT("Packet Serialize"), PacketResult.HasValue());

	FDiscoveryPacket DiscoveryPacket = PacketResult.StealValue();
	TestEqual(TEXT("Message Type"), DiscoveryPacket.GetMessageType(), MessageType);
	TestEqual(TEXT("Payload"), DiscoveryPacket.GetPayload(), Payload);

	return true;
}

}

#endif // WITH_DEV_AUTOMATION_TESTS