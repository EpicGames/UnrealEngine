// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExportClient/Communication/ExportHeader.h"

#include "Misc/AutomationTest.h"

#include "Tests/Utility.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::CaptureManager
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TExportHeaderDeserializeOneTest, "MetaHuman.Capture.CaptureProtocolStack.Export.ExportHeader.DeserializeOne.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TExportHeaderDeserializeOneTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'E', 'X', 'P', 'O', 'R', 'T', '\0' };
	const uint16 Version = 1;
	const uint32 TransactionId = 11223344;

	TArray<uint8> Data;
	Data.Append(Header);
	Data.Append(reinterpret_cast<const uint8*>(&Version), sizeof(Version));
	Data.Append(reinterpret_cast<const uint8*>(&TransactionId), sizeof(TransactionId));

	FDataProvider DataProvider(MoveTemp(Data));

	// Deserialization
	TProtocolResult<FExportHeader> ExportHeaderResult = FExportHeader::Deserialize(DataProvider);
	TestTrue(TEXT("Header Deserialize"), ExportHeaderResult.HasValue());

	FExportHeader ExportHeader = ExportHeaderResult.StealValue();

	TestEqual(TEXT("Header Deserialize"), ExportHeader.GetVersion(), Version);
	TestEqual(TEXT("Header Deserialize"), ExportHeader.GetTransactionId(), TransactionId);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TExportHeaderDeserializeOneTestInvalidHeader, "MetaHuman.Capture.CaptureProtocolStack.Export.ExportHeader.DeserializeOne.InvalidHeader", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TExportHeaderDeserializeOneTestInvalidHeader::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'A', 'A', 'A', 'A', 'A', 'A', 'A', 'A', 'A', '\0' };
	const uint16 Version = 1;
	const uint32 TransactionId = 11223344;

	TArray<uint8> Data;
	Data.Append(Header);
	Data.Append(reinterpret_cast<const uint8*>(&Version), sizeof(Version));
	Data.Append(reinterpret_cast<const uint8*>(&TransactionId), sizeof(TransactionId));

	FDataProvider DataProvider(MoveTemp(Data));

	// Deserialization
	TProtocolResult<FExportHeader> ExportHeaderResult = FExportHeader::Deserialize(DataProvider);
	TestTrue(TEXT("Header Deserialize"), ExportHeaderResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TExportHeaderDeserializeOneTestInvalidSize, "MetaHuman.Capture.CaptureProtocolStack.Export.ExportHeader.DeserializeOne.InvalidSize", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TExportHeaderDeserializeOneTestInvalidSize::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'E', 'X', 'P', 'O', 'R', 'T', '\0' };

	TArray<uint8> Data;
	Data.Reserve(Header.Num());
	Data.Append(Header);

	FDataProvider DataProvider(MoveTemp(Data));

	// Deserialization
	TProtocolResult<FExportHeader> ExportHeaderResult = FExportHeader::Deserialize(DataProvider);
	TestTrue(TEXT("Header Deserialize"), ExportHeaderResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TExportHeaderSerializeOneTest, "MetaHuman.Capture.CaptureProtocolStack.Export.ExportHeader.SerializeOne.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TExportHeaderSerializeOneTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'E', 'X', 'P', 'O', 'R', 'T', '\0' };
	const uint16 Version = 1;
	const uint32 TransactionId = 11223344;

	TArray<uint8> Data;
	Data.Append(Header);
	Data.Append(reinterpret_cast<const uint8*>(&Version), sizeof(Version));
	Data.Append(reinterpret_cast<const uint8*>(&TransactionId), sizeof(TransactionId));

	FDataSender DataSender;

	// Serialization
	FExportHeader ExportHeader(Version, TransactionId);
	TProtocolResult<void> ExportHeaderResult = FExportHeader::Serialize(ExportHeader, DataSender);
	TestTrue(TEXT("Header Serialize"), ExportHeaderResult.HasValue());

	TestEqual(TEXT("Header Serialize"), DataSender.GetData(), Data);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TExportHeaderSerializeOneTestError, "MetaHuman.Capture.CaptureProtocolStack.Export.ExportHeader.SerializeOne.Error", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TExportHeaderSerializeOneTestError::RunTest(const FString& InParameters)
{
	// Prepare test
	const TArray<uint8> Header = { 'C', 'P', 'S', 'E', 'X', 'P', 'O', 'R', 'T', '\0' };
	const uint16 Version = 1;
	const uint32 TransactionId = 11223344;

	TArray<uint8> Data;
	Data.Append(Header);
	Data.Append(reinterpret_cast<const uint8*>(&Version), sizeof(Version));
	Data.Append(reinterpret_cast<const uint8*>(&TransactionId), sizeof(TransactionId));

	FFailedDataSender DataSender;

	// Serialization
	FExportHeader ExportHeader(Version, TransactionId);
	TProtocolResult<void> ExportHeaderResult = FExportHeader::Serialize(ExportHeader, DataSender);
	TestTrue(TEXT("Header Serialize"), ExportHeaderResult.HasError());

	return true;
}

}

#endif // WITH_DEV_AUTOMATION_TESTS