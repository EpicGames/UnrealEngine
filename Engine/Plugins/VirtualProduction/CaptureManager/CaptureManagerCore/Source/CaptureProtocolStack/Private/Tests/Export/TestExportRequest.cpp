// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExportClient/Messages/ExportRequest.h"

#include "Misc/AutomationTest.h"

#include "Tests/Utility.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::CaptureManager
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TExportRequestDeserializeOneTest, "MetaHuman.Capture.CaptureProtocolStack.Export.ExportRequest.DeserializeOne.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TExportRequestDeserializeOneTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const FString TakeName = TEXT("TakeName");
	const FString FileName = TEXT("FileName");
	const uint16 TakeNameLen = TakeName.Len();
	const uint16 FileNameLen = FileName.Len();

	const uint64 Offset = 0;

	TArray<uint8> Packet;
	Packet.Append(reinterpret_cast<const uint8*>(&TakeNameLen), sizeof(TakeNameLen));
	Packet.Append(reinterpret_cast<const uint8*>(TCHAR_TO_UTF8(*TakeName)), TakeName.Len());
	Packet.Append(reinterpret_cast<const uint8*>(&FileNameLen), sizeof(FileNameLen));
	Packet.Append(reinterpret_cast<const uint8*>(TCHAR_TO_UTF8(*FileName)), FileName.Len());
	Packet.Append(reinterpret_cast<const uint8*>(&Offset), sizeof(Offset));

	FDataProvider DataProvider(MoveTemp(Packet));

	// Deserialization
	TProtocolResult<FExportRequest> ExportRequestResult = FExportRequest::Deserialize(DataProvider);
	TestTrue(TEXT("Request Deserialize"), ExportRequestResult.HasValue());

	FExportRequest ExportRequest = ExportRequestResult.StealValue();

	TestEqual(TEXT("Request Deserialize"), ExportRequest.GetTakeName(), TakeName);
	TestEqual(TEXT("Request Deserialize"), ExportRequest.GetFileName(), FileName);
	TestEqual(TEXT("Request Deserialize"), ExportRequest.GetOffset(), Offset);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TExportRequestDeserializeOneTestInvalidSize, "MetaHuman.Capture.CaptureProtocolStack.Export.ExportRequest.DeserializeOne.InvalidSize", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TExportRequestDeserializeOneTestInvalidSize::RunTest(const FString& InParameters)
{
	// Prepare test
	TArray<uint8> Packet;

	FDataProvider DataProvider(MoveTemp(Packet));

	// Deserialization
	TProtocolResult<FExportRequest> ExportRequestResult = FExportRequest::Deserialize(DataProvider);
	TestTrue(TEXT("Request Deserialize"), ExportRequestResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TExportRequestSerializeOneTest, "MetaHuman.Capture.CaptureProtocolStack.Export.ExportRequest.SerializeOne.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TExportRequestSerializeOneTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const FString TakeName = TEXT("TakeName");
	const FString FileName = TEXT("FileName");
	const uint16 TakeNameLen = TakeName.Len();
	const uint16 FileNameLen = FileName.Len();

	const uint64 Offset = 0;

	TArray<uint8> Packet;
	Packet.Append(reinterpret_cast<const uint8*>(&TakeNameLen), sizeof(TakeNameLen));
	Packet.Append(reinterpret_cast<const uint8*>(TCHAR_TO_UTF8(*TakeName)), TakeName.Len());
	Packet.Append(reinterpret_cast<const uint8*>(&FileNameLen), sizeof(FileNameLen));
	Packet.Append(reinterpret_cast<const uint8*>(TCHAR_TO_UTF8(*FileName)), FileName.Len());
	Packet.Append(reinterpret_cast<const uint8*>(&Offset), sizeof(Offset));

	FDataSender DataSender;

	FExportRequest Request(TakeName, FileName, Offset);

	// Serialization
	TProtocolResult<void> ExportRequestResult = FExportRequest::Serialize(Request, DataSender);
	TestTrue(TEXT("Request Serialize"), ExportRequestResult.HasValue());

	TestEqual(TEXT("Request Serialize"), DataSender.GetData(), Packet);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TExportRequestSerializeOneTestError, "MetaHuman.Capture.CaptureProtocolStack.Export.ExportRequest.SerializeOne.Error", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TExportRequestSerializeOneTestError::RunTest(const FString& InParameters)
{
	// Prepare test
	const FString TakeName = TEXT("TakeName");
	const FString FileName = TEXT("FileName");
	const uint16 TakeNameLen = TakeName.Len();
	const uint16 FileNameLen = FileName.Len();

	const uint64 Offset = 0;

	TArray<uint8> Packet;
	Packet.Append(reinterpret_cast<const uint8*>(&TakeNameLen), sizeof(TakeNameLen));
	Packet.Append(reinterpret_cast<const uint8*>(TCHAR_TO_UTF8(*TakeName)), TakeName.Len());
	Packet.Append(reinterpret_cast<const uint8*>(&FileNameLen), sizeof(FileNameLen));
	Packet.Append(reinterpret_cast<const uint8*>(TCHAR_TO_UTF8(*FileName)), FileName.Len());
	Packet.Append(reinterpret_cast<const uint8*>(&Offset), sizeof(Offset));

	FFailedDataSender DataSender;

	FExportRequest Request(TakeName, FileName, Offset);

	// Serialization
	TProtocolResult<void> ExportRequestResult = FExportRequest::Serialize(Request, DataSender);
	TestTrue(TEXT("Request Serialize"), ExportRequestResult.HasError());

	return true;
}

}

#endif // WITH_DEV_AUTOMATION_TESTS