// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExportClient/Messages/ExportResponse.h"

#include "Misc/AutomationTest.h"

#include "Tests/Utility.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::CaptureManager
{

TStaticArray<uint8, 16> CreateHash()
{
	TStaticArray<uint8, 16> StaticArray(0); // Initializing to zeros
	for (int32 Index = 0; Index < StaticArray.Num(); ++Index)
	{
		StaticArray[Index] = Index;
	}

	return StaticArray;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TExportResponseDeserializeOneTest, "MetaHuman.Capture.CaptureProtocolStack.Export.ExportResponse.DeserializeOne.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TExportResponseDeserializeOneTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const FExportResponse::EStatus Status = FExportResponse::EStatus::Success;
	const uint64 Length = 50;

	TArray<uint8> Packet;
	Packet.Append(reinterpret_cast<const uint8*>(&Status), sizeof(Status));
	Packet.Append(reinterpret_cast<const uint8*>(&Length), sizeof(Length));

	FDataProvider DataProvider(MoveTemp(Packet));

	// Deserialization
	TProtocolResult<FExportResponse> ExportResponseResult = FExportResponse::Deserialize(DataProvider);
	TestTrue(TEXT("Response Deserialize"), ExportResponseResult.HasValue());

	FExportResponse ExportResponse = ExportResponseResult.StealValue();

	TestEqual(TEXT("Response Deserialize"), ExportResponse.GetStatus(), Status);
	TestEqual(TEXT("Response Deserialize"), ExportResponse.GetLength(), Length);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TExportResponseDeserializeOneTestInvalidSize, "MetaHuman.Capture.CaptureProtocolStack.Export.ExportResponse.DeserializeOne.InvalidSize", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TExportResponseDeserializeOneTestInvalidSize::RunTest(const FString& InParameters)
{
	// Prepare test
	TArray<uint8> Packet;

	FDataProvider DataProvider(MoveTemp(Packet));

	// Deserialization
	TProtocolResult<FExportResponse> ExportResponseResult = FExportResponse::Deserialize(DataProvider);
	TestTrue(TEXT("Response Deserialize"), ExportResponseResult.HasError());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TExportResponseSerializeOneTest, "MetaHuman.Capture.CaptureProtocolStack.Export.ExportResponse.SerializeOne.Success", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TExportResponseSerializeOneTest::RunTest(const FString& InParameters)
{
	// Prepare test
	const FExportResponse::EStatus Status = FExportResponse::EStatus::Success;
	const uint64 Length = 50;

	TArray<uint8> Packet;
	Packet.Append(reinterpret_cast<const uint8*>(&Status), sizeof(Status));
	Packet.Append(reinterpret_cast<const uint8*>(&Length), sizeof(Length));

	FDataSender DataSender;

	FExportResponse Response(Status, Length);

	// Serialization
	TProtocolResult<void> ExportResponseResult = FExportResponse::Serialize(Response, DataSender);
	TestTrue(TEXT("Response Serialize"), ExportResponseResult.HasValue());

	TestEqual(TEXT("Response Serialize"), DataSender.GetData(), Packet);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(TExportResponseSerializeOneTestError, "MetaHuman.Capture.CaptureProtocolStack.Export.ExportResponse.SerializeOne.Error", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool TExportResponseSerializeOneTestError::RunTest(const FString& InParameters)
{
	// Prepare test
	const FExportResponse::EStatus Status = FExportResponse::EStatus::Success;
	const uint64 Length = 50;

	TArray<uint8> Packet;
	Packet.Append(reinterpret_cast<const uint8*>(&Status), sizeof(Status));
	Packet.Append(reinterpret_cast<const uint8*>(&Length), sizeof(Length));

	FFailedDataSender DataSender;

	FExportResponse Response(Status, Length);

	// Serialization
	TProtocolResult<void> ExportResponseResult = FExportResponse::Serialize(Response, DataSender);
	TestTrue(TEXT("Response Serialize"), ExportResponseResult.HasError());

	return true;
}

}

#endif // WITH_DEV_AUTOMATION_TESTS