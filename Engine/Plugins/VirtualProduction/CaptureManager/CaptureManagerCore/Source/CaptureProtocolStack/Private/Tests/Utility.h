// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/TcpReaderWriter.h"

namespace UE::CaptureManager
{

class FDataProvider : public ITcpSocketReader
{
public:

	FDataProvider(TArray<uint8> InData)
		: Data(MoveTemp(InData))
	{
	}

	virtual TProtocolResult<TArray<uint8>> ReceiveMessage(const uint64 InSize, const uint32 InMaxWaitTimeMs = DefaultWaitTimeoutMs) override
	{
		if (InSize > Data.Num())
		{
			return FCaptureProtocolError(TEXT("Failed to receive data"));
		}

		TArray<uint8> DataToBeReturned(Data.GetData(), InSize);

		Data.RemoveAt(0, InSize);

		return DataToBeReturned;
	}

private:

	TArray<uint8> Data;
};

class FDataSender : public ITcpSocketWriter
{
public:

	FDataSender()
	{
	}

	virtual TProtocolResult<void> SendMessage(const TArray<uint8>& InPayload) override
	{
		Data.Append(InPayload.GetData(), InPayload.Num());

		return ResultOk;
	}

	const TArray<uint8>& GetData() const
	{
		return Data;
	}

private:

	TArray<uint8> Data;
};

class FFailedDataSender : public FDataSender
{
public:

	virtual TProtocolResult<void> SendMessage(const TArray<uint8>& InPayload) override
	{
		return FCaptureProtocolError(TEXT("Failed to send the data"));
	}
};

}