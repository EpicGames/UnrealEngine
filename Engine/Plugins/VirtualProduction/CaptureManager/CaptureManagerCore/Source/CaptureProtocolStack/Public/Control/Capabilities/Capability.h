// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"

class FString;

namespace UE::CaptureManager
{
namespace ECapabilityValue
{
enum Type
{
	Boolean,
	Integer,
	Float,
	String,
	Enum
};
} // namespace ECapabilityValue

namespace ECapabilityAccess
{
enum Type
{
	ReadOnly = 0,
	ReadWrite = 1
};
} // namespace ECapabilityAccess

using FCapabilityValue = TVariant<bool, int32, float, FString>;

using FCapabilityValues = TMap<FString, FCapabilityValue>;

struct FCapabilityProperty
{
	FString Id;
	FString Name;
	ECapabilityValue::Type Type;
	ECapabilityAccess::Type Access;
	TOptional<TVariant<int32, float>> Min;
	TOptional<TVariant<int32, float>> Max;
	TOptional<TArray<FString>> EnumOptions;
	FCapabilityValue CurrentValue;
};

struct FCommandParameterDescriptor
{
	FString Name;
	ECapabilityValue::Type Type;
	bool Optional;
	FCapabilityValue DefaultValue;
};

struct FCapabilityCommand
{
	FString Id;
	FString Name;

	TArray<FCommandParameterDescriptor> Parameters;
	TOptional<ECapabilityValue::Type> ReturnType;
};

struct FEventArgumentDescriptor
{
	FString Name;
	ECapabilityValue::Type Type;
};

struct FCapabilityEvent
{
	FString Id;
	FString Name;

	TArray<FEventArgumentDescriptor> Arguments;
};

struct FCapability
{
	FString Id;
	FString Name;
	TMap<FString, FCapabilityProperty> Properties;
	TMap<FString, FCapabilityCommand> Commands;
	TMap<FString, FCapabilityEvent> Events;
};

} // namespace UE::CaptureManager
