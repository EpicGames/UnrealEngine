// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlPacket.h"

#include "Network/TcpClient.h"

#include "Async/QueueRunner.h"
#include "Utility/Error.h"

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/Event.h"

namespace UE::CaptureManager
{

class FControlCommunication final
{
public:

	DECLARE_DELEGATE_OneParam(FOnPacketReceived, FControlPacket InPacket)
	DECLARE_DELEGATE(FCommunicationStoppedHandler)

	FControlCommunication();

	TProtocolResult<void> Init();
	TProtocolResult<void> Start(const FString& InServerIp, const uint16 InServerPort);
	void Stop();

	bool IsRunning() const;

	void SendMessage(FControlPacket InMessage);

	void SetReceiveHandler(FOnPacketReceived InReceiveHandler);
	void SetCommunicationStoppedHandler(FCommunicationStoppedHandler InConnectionStateHandler);

private:

	class FCommunicationRunnable :
		public FRunnable
	{
	public:
		FCommunicationRunnable(FControlCommunication& InCommunication);
		virtual ~FCommunicationRunnable();

		uint32 Run() override;
		void Stop() override;

		void Join();

	private:

		void HandleError(const FCaptureProtocolError& Error);

		FControlCommunication& Communication;
		bool bIsRunning;
		FEventRef DoneEvent;
	};


	void RunnableHandler(TSharedPtr<FRunnable> InRunnable);
	void CommunicationRunnableStopped();

	void SendNextPacket();
	void SendPacket(FControlPacket InPacket);
	void ReceiveNextPacket();

	TProtocolResult<FControlPacketHeader> ReceiveControlHeader();
	TProtocolResult<void> ReceiveControlPacket(const FControlPacketHeader& InHeader);

	FTcpClient Client;
	FOnPacketReceived ReceiveHandler;
	FCommunicationStoppedHandler CommunicationStoppedHandler;

	TQueue<FControlPacket> SenderQueue;
	std::atomic_int SenderQueueCurrentSize = 0;
	TQueue<FControlPacket> ReceiverQueue;
	TQueueRunner<TSharedPtr<FRunnable>> SynchronizedRunnable;
	TWeakPtr<FCommunicationRunnable> Runnable;
};

}