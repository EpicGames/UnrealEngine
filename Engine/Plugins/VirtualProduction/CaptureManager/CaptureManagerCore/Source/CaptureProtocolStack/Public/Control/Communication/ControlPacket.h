// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/TcpReaderWriter.h"

#include "Utility/Error.h"

#include "Containers/Array.h"

#include "Dom/JsonObject.h"

namespace UE::CaptureManager
{

class FControlPacketHeader final
{
public:

	static const TArray<uint8> Header;

	static TProtocolResult<FControlPacketHeader> Deserialize(ITcpSocketReader& InReader, 
															 uint32 InTimeoutForControlHeader = ITcpSocketReader::DefaultWaitTimeoutMs);
	static uint32 GetHeaderSize();

	FControlPacketHeader();
	FControlPacketHeader(uint16 InVersion, uint32 InPayloadSize);

	uint16 GetVersion() const;
	uint32 GetPayloadSize() const;

private:

	uint16 Version = 0;
	uint32 PayloadSize = 0;
};

class FControlPacket final
{
public:

	static TProtocolResult<FControlPacket> Deserialize(const FControlPacketHeader& InPacketHeader, ITcpSocketReader& InReader);
	static TProtocolResult<void> Serialize(const FControlPacket& InMessage, ITcpSocketWriter& InWriter);

	FControlPacket();
	FControlPacket(uint16 InVersion, TArray<uint8> InPayload);

	FControlPacket(const FControlPacket& InOther) = default;
	FControlPacket(FControlPacket&& InOther) = default;

	FControlPacket& operator=(const FControlPacket& InOther) = default;
	FControlPacket& operator=(FControlPacket&& InOther) = default;

	uint16 GetVersion() const;
	const TArray<uint8>& GetPayload() const;
	uint32 GetPayloadSize() const;

private:

	FControlPacketHeader Header;
	TArray<uint8> Payload;
};

}