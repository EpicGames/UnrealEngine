// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Utility/Definitions.h"

#include "Control/Communication/ControlCommunication.h"
#include "Control/Communication/ControlPacket.h"

#include "Messages/ControlMessage.h"
#include "Messages/ControlRequest.h"
#include "Messages/ControlResponse.h"
#include "Messages/ControlUpdate.h"

#include "Utility/Error.h"
#include "Async/CaptureTimerManager.h"

#include "Misc/ScopedEvent.h"

#include "Async/Async.h"

#include <atomic>

#define UE_API CAPTUREPROTOCOLSTACK_API

namespace UE::CaptureManager
{

class FKeepAliveCounter
{
public:
	FKeepAliveCounter();

	void Increment();
	void Reset();
	bool HasReached(uint16 InBound);

private:

	std::atomic<uint16> Counter;
};

class FControlMessenger final
{
public:

	template<class RequestType>
	using FOnControlResponse = TDelegate<void(TProtocolResult<typename RequestType::ResponseType> InResult)>;

	DECLARE_DELEGATE_OneParam(FOnDisconnect, const FString& InCause);

	static constexpr int32 ResponseWaitTime = 3; // Seconds
	static constexpr int32 KeepAliveInterval = 5; // Seconds
	static UE_API const TCHAR* HandshakeSessionId;

	UE_API FControlMessenger();
	UE_API ~FControlMessenger();

	UE_API void RegisterUpdateHandler(FString InAddressPath, FControlUpdate::FOnUpdateMessage InUpdateHandler);
	UE_API void RegisterDisconnectHandler(FOnDisconnect InOnDisconnectHandler);

	UE_API TProtocolResult<void> Start(const FString& InServerIp, const uint16 InServerPort);
	UE_API void Stop();

	UE_API TProtocolResult<void> StartSession();
	UE_API TProtocolResult<FGetServerInformationResponse> GetServerInformation();
	UE_API TProtocolResult<FGetTakeListResponse> GetTakeList();

	template<class RequestType>
	TProtocolResult<typename RequestType::ResponseType> SendRequest(RequestType InRequest)
	{
		FControlMessage Message(InRequest.GetAddressPath(), FControlMessage::EType::Request, InRequest.GetBody());

		uint32 TransactionId = GenerateTransactionId();

		{
			FScopeLock SessionLock(&SessionIdMutex);
			Message.SetSessionId(SessionId);
		}

		Message.SetTransactionId(TransactionId);
		Message.SetTimestamp(GetTimestamp());

		TProtocolResult<FControlPacket> SerializeResult = FControlMessage::Serialize(Message);

		if (SerializeResult.HasError())
		{
			return FCaptureProtocolError(TEXT("Failed to serialize a request."));
		}

		TPromise<TProtocolResult<FControlMessage>> Promise;
		TFuture<TProtocolResult<FControlMessage>> Future = Promise.GetFuture();

		{
			FScopeLock Lock(&RequestsMutex);
			RequestContexts.Emplace(TransactionId, MakeUnique<FRequestContext>(MoveTemp(Message), MoveTemp(Promise)));
		}

		SendPacket(SerializeResult.StealValue());

		if (!Future.WaitFor(FTimespan::FromSeconds(ResponseWaitTime)))
		{
			FScopeLock Lock(&RequestsMutex);
			if (TUniquePtr<FRequestContext>* Found = RequestContexts.Find(TransactionId))
			{
				// MessageHandler may have fulfilled the promise between WaitFor and lock acquisition.
				if (!Future.IsReady())
				{
					(*Found)->Promise.SetValue(FCaptureProtocolError("Broken promise"));
				}
				RequestContexts.Remove(TransactionId);
			}

			return FCaptureProtocolError(TEXT("Server failed to respond within 3 seconds."));
		}

		{
			FScopeLock Lock(&RequestsMutex);
			RequestContexts.Remove(TransactionId);
		}

		const TProtocolResult<FControlMessage>& FutureResult = Future.Get();
		check(FutureResult.HasValue());

		const FControlMessage& ResponseMessage = FutureResult.GetValue();

		{
			FScopeLock SessionLock(&SessionIdMutex);
			if (ResponseMessage.GetSessionId() != SessionId)
			{
				return FCaptureProtocolError(TEXT("Invalid session ID arrived"));
			}
		}

		if (!ResponseMessage.GetErrorName().IsEmpty())
		{
			return FCaptureProtocolError(FString::Format(TEXT("Server responded with error: {0} : {1}"), { ResponseMessage.GetErrorName(), ResponseMessage.GetErrorDescription() }));
		}

		typename RequestType::ResponseType Response;
		Response.SetAddressPath(InRequest.GetAddressPath());

		TProtocolResult<void> ParseResult = Response.Parse(ResponseMessage.GetBody());
		if (ParseResult.HasError())
		{
			return FCaptureProtocolError(TEXT("Failed to parse the response: ") + ParseResult.StealError().GetMessage());
		}

		return Response;
	}

	template<class RequestType>
	void SendAsyncRequest(RequestType InRequest, FOnControlResponse<RequestType> InOnResponse)
	{
		AsyncRequestRunner.Add(FAsyncRequestDelegate::CreateLambda([this, Request(MoveTemp(InRequest)), OnResponse(MoveTemp(InOnResponse))]() mutable
		{
			TProtocolResult<typename RequestType::ResponseType> Result = SendRequest(MoveTemp(Request));

			OnResponse.ExecuteIfBound(MoveTemp(Result));
		}));
	}

private:
	DECLARE_DELEGATE(FAsyncRequestDelegate)

	struct FRequestContext
	{
		FRequestContext(FControlMessage InRequest, TPromise<TProtocolResult<FControlMessage>> InPromise)
			: Request(MoveTemp(InRequest))
			, Promise(MoveTemp(InPromise))
		{
		}

		FControlMessage Request;

		TPromise<TProtocolResult<FControlMessage>> Promise;
	};

	UE_API void SendPacket(FControlPacket InPacket);
	UE_API void KeepAlive();
	UE_API void MessageHandler(FControlPacket InPacket);
	UE_API void CommunicationStoppedHandler();

	UE_API uint32 GenerateTransactionId() const;
	UE_API uint64 GetTimestamp() const;

	UE_API void StartKeepAliveTimer();
	UE_API void StopKeepAliveTimer();

	UE_API void OnAsyncRequestProcess(FAsyncRequestDelegate InAsyncDelegate);
	
	FControlCommunication Communication;

	FCriticalSection SessionIdMutex;
	FString SessionId;

	FCriticalSection RequestsMutex;
	TMap<uint32, TUniquePtr<FRequestContext>> RequestContexts;

	FCriticalSection UpdatesMutex;
	TMap<FString, FControlUpdate::FOnUpdateMessage> UpdateHandlers;

	FCaptureTimerManager::FTimerHandle KeepAliveTimer;
	FKeepAliveCounter KeepAliveFailures;

	TQueueRunner<FAsyncRequestDelegate> AsyncRequestRunner;
	FOnDisconnect OnDisconnectHandler;

	mutable std::atomic<uint32> NextTransactionId{0};
};

}

#undef UE_API
