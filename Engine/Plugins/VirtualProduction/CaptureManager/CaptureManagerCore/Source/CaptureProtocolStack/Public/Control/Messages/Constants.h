// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

namespace UE::CaptureManager::CPS::AddressPaths
{
// Requests/Responses
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GKeepAlive;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GStartSession;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GStopSession;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSubscribe;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GUnsubscribe;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GGetServerInformation;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GGetState;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GStartRecordingTake;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GStopRecordingTake;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GAbortRecordingTake;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GGetTakeList;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GGetTakeMetadata;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GExportTakeData;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GExportTakeVideoFrame;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GExportCameraFeedFrame;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GPauseExport;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCancelExport;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GGetStreamingSubjects;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GStartStreaming;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GStopStreaming;

// Updates
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSessionStopped;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GRecordingStatus;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GTakeAdded;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GTakeRemoved;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GTakeUpdated;

// Capabilities
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilitiesRoot;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilitiesListAll;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityProperties;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityPropertySet;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityPropertyGet;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityCommands;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityCommandExecute;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityEvents;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityEventSubscribe;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCapabilityEventUnsubscribe;

// Platform Specific: IOS
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GDiskCapacity;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GBattery;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GThermalState;
}

namespace UE::CaptureManager::CPS::Properties
{
// Keys
// Messages
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSessionId;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GAddressPath;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GTransactionId;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GTimestamp;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GType;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GBody;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GError;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GDescription;

// Requests
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSlateName;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GTakeNumber;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSubject;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GScenario;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GTags;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GNames;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GTakeName;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GFiles;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GName;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GOffset;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GStreamPort;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSubjectIds;


// Responses
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GId;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GModel;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GPlatformName;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GPlatformVersion;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSoftwareName;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSoftwareVersion;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GExportPort;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GIsRecording;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GPlatformState;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GTakes;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GDateTime;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GFrames;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GAppVersion;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GLength;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GVideo;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GFrameRate;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GHeight;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GWidth;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GAudio;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GChannels;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSampleRate;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GBitsPerChannel;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSubjects;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GControls;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GVersion;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GAnimationMetadata;

// Platform Specific: iOS
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GTotalCapacity;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GRemainingCapacity;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GBatteryLevel;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GThermalState;

extern CAPTUREPROTOCOLSTACK_API const TCHAR* GTotal;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GRemaining;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GLevel;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GState;

// Values
// Messages
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GRequest;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GResponse;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GUpdate;

extern CAPTUREPROTOCOLSTACK_API const TCHAR* GIOS;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GAndroid;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GWindows;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GWindowsServer;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GLinux;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GMacOS;

// Platform Specific: iOS
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GNominal;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GFair;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSerious;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCritical;
}

namespace UE::CaptureManager::CPS::ErrorNames
{
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GUnsupportedProtocolVersion;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GRequestFailed;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GRequestParsingFailed;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GNotSupported;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GNoSession;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GMissingBody;
}

namespace UE::CaptureManager::CPS::Capabilities
{
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GId;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GName;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GValue;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GProperties;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GType;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GAccess;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GReadOnly;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GReadWrite;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GEnumOptions;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCurrentValue;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GMin;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GMax;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GGet;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSet;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GCommands;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GParameters;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GExecute;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GOptional;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GReturnType;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GDefaultValue;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GEvents;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GArguments;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GSubscribe;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GUnsubscribe;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GBoolean;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GInteger;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GFloat;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GString;
extern CAPTUREPROTOCOLSTACK_API const TCHAR* GEnumeration;
} // namespace UE::CaptureManager::CPS::Capabilities