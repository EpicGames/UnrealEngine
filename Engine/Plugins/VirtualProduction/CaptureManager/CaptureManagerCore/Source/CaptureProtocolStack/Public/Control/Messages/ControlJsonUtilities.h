// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Utility/Error.h"

#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"

#include "Policies/CondensedJsonPrintPolicy.h"

#define UE_API CAPTUREPROTOCOLSTACK_API

#define CHECK_PARSE CPS_CHECK_VOID_RESULT

namespace UE::CaptureManager
{

class FJsonUtility final
{
public:
	static UE_API bool CreateJsonFromUTF8Data(const TArray<uint8>& InData, TSharedPtr<FJsonObject>& OutObject);
	static UE_API bool CreateUTF8DataFromJson(TSharedPtr<FJsonObject> InObject, TArray<uint8>& OutData);

	template <typename T>
	static TProtocolResult<void> ParseNumber(const TSharedPtr<FJsonObject>& InBody, const FString& InFieldName, T& OutFieldValue)
	{
		if (!InBody->TryGetNumberField(InFieldName, OutFieldValue))
		{
			return FCaptureProtocolError(TEXT("Failed to parse key: ") + InFieldName);
		}

		return ResultOk;
	}
	static UE_API TProtocolResult<void> ParseString(const TSharedPtr<FJsonObject>& InBody, const FString& InFieldName, FString& OutFieldValue);
	static UE_API TProtocolResult<void> ParseBool(const TSharedPtr<FJsonObject>& InBody, const FString& InFieldName, bool& OutFieldValue);
	static UE_API TProtocolResult<void> ParseObject(const TSharedPtr<FJsonObject>& InBody, const FString& InFieldName, const TSharedPtr<FJsonObject>*& OutFieldValue);
	static UE_API TProtocolResult<void> ParseArray(const TSharedPtr<FJsonObject>& InBody, const FString& InFieldName, const TArray<TSharedPtr<FJsonValue>>*& OutFieldValue);
};

}

#undef UE_API
