// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Control/Communication/ControlPacket.h"

#define UE_API CAPTUREPROTOCOLSTACK_API

namespace UE::CaptureManager
{

class FControlMessage final
{
public:

	enum class EType
	{
		Request,
		Response,
		Update,

		Invalid
	};

	UE_API FControlMessage(FString InAddressPath, EType InType, TSharedPtr<FJsonObject> InBody);

	FControlMessage(const FControlMessage& InOther) = default;
	FControlMessage(FControlMessage&& InOther) = default;

	FControlMessage& operator=(const FControlMessage& InOther) = default;
	FControlMessage& operator=(FControlMessage&& InOther) = default;

	static UE_API TProtocolResult<FControlMessage> Deserialize(const FControlPacket& InPacket);
	static UE_API TProtocolResult<FControlPacket> Serialize(const FControlMessage& InMessage);

	UE_API void SetSessionId(FString InSessionId);
	UE_API void SetTransactionId(uint32 InTrasactionId);
	UE_API void SetTimestamp(uint64 InTimestamp);

	UE_API const FString& GetSessionId() const;
	UE_API const FString& GetAddressPath() const;
	UE_API uint32 GetTransactionId() const;
	UE_API uint64 GetTimestamp() const;
	UE_API EType GetType() const;
	UE_API const TSharedPtr<FJsonObject>& GetBody() const;
	UE_API TSharedPtr<FJsonObject>& GetBody();

	UE_API const FString& GetErrorName() const;
	UE_API const FString& GetErrorDescription() const;

private:

	struct FErrorResponse
	{
		FString Name = TEXT("");
		FString Description = TEXT("");
	};

	UE_API FControlMessage(FString InSessionId,
					FString InAddressPath,
					uint32 InTransactionId,
					uint64 InTimestamp,
					EType InMessageType,
					TSharedPtr<FJsonObject> InBody,
					FErrorResponse InError);

	static UE_API EType DeserializeType(FString InMessageTypeStr);
	static UE_API FString SerializeType(EType InMessageType);

	FString SessionId;
	FString AddressPath;
	uint32 TransactionId = 0;
	uint64 Timestamp = 0;
	EType MessageType;
	TSharedPtr<FJsonObject> Body;
	FErrorResponse Error;
};

}

#undef UE_API
