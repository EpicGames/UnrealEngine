// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlResponse.h"
#include "Misc/Optional.h"

#define UE_API CAPTUREPROTOCOLSTACK_API

namespace UE::CaptureManager
{

class FControlRequest
{
public:

	UE_API FControlRequest(FString InAddressPath);
	virtual ~FControlRequest() = default;

	UE_API const FString& GetAddressPath() const;
	UE_API virtual TSharedPtr<FJsonObject> GetBody() const;

private:

	FString AddressPath;
};

class FKeepAliveRequest final : public FControlRequest
{
public:

	using ResponseType = FKeepAliveResponse;

	UE_API FKeepAliveRequest();
};

class FStartSessionRequest final : public FControlRequest
{
public:

	using ResponseType = FStartSessionResponse;

	UE_API FStartSessionRequest();
};

class FStopSessionRequest final : public FControlRequest
{
public:

	using ResponseType = FStopSessionResponse;

	UE_API FStopSessionRequest();
};


class FGetServerInformationRequest final : public FControlRequest
{
public:

	using ResponseType = FGetServerInformationResponse;

	UE_API FGetServerInformationRequest();
};

class FSubscribeRequest final : public FControlRequest
{
public:

	using ResponseType = FSubscribeResponse;

	UE_API FSubscribeRequest();
};

class FUnsubscribeRequest final : public FControlRequest
{
public:

	using ResponseType = FUnsubscribeResponse;

	UE_API FUnsubscribeRequest();
};

class FGetStateRequest final : public FControlRequest
{
public:

	using ResponseType = FGetStateResponse;

	UE_API FGetStateRequest();
};

class FStartRecordingTakeRequest final : public FControlRequest
{
public:

	using ResponseType = FStartRecordingTakeResponse;

	UE_API FStartRecordingTakeRequest(FString InSlateName,
									  uint16 InTakeNumber,
									  TOptional<FString> InSubject = TOptional<FString>(),
									  TOptional<FString> InScenario = TOptional<FString>(),
									  TOptional<TArray<FString>> InTags = TOptional<TArray<FString>>());

	UE_API virtual TSharedPtr<FJsonObject> GetBody() const override;

private:

	FString SlateName;
	uint16 TakeNumber;
	TOptional<FString> Subject;
	TOptional<FString> Scenario;
	TOptional<TArray<FString>> Tags;
};

class FStopRecordingTakeRequest final : public FControlRequest
{
public:

	using ResponseType = FStopRecordingTakeResponse;

	UE_API FStopRecordingTakeRequest();
};

class FAbortRecordingTakeRequest final : public FControlRequest
{
public:

	using ResponseType = FAbortRecordingTakeResponse;

	UE_API FAbortRecordingTakeRequest();
};

class FGetTakeListRequest final : public FControlRequest
{
public:

	using ResponseType = FGetTakeListResponse;

	UE_API FGetTakeListRequest();
};

class FGetTakeMetadataRequest final : public FControlRequest
{
public:

	using ResponseType = FGetTakeMetadataResponse;

	UE_API FGetTakeMetadataRequest(TArray<FString> InNames);

	UE_API virtual TSharedPtr<FJsonObject> GetBody() const override;

private:

	TArray<FString> Names;
};

class FGetStreamingSubjectsRequest final : public FControlRequest
{
public:
	using ResponseType = FGetStreamingSubjectsResponse;

	UE_API FGetStreamingSubjectsRequest();
};

class FStartStreamingRequest final : public FControlRequest
{
public:

	struct FSubject
	{
		FString Id;
		TOptional<FString> Name;

		FSubject() = delete;
		
		explicit FSubject(const FString& InId)
		: Id(InId)
		{
		}
		
		FSubject(const FString& InId, const FString& InName)
		: Id(InId)
		, Name(InName)
		{
		}
	};
	
	using ResponseType = FStartStreamingResponse;

	UE_API FStartStreamingRequest(uint16 InStreamPort, TArray<FSubject> InSubjects);

	UE_API virtual TSharedPtr<FJsonObject> GetBody() const override;
	
private:
	
	uint16 StreamPort;
	TArray<FSubject> Subjects;
};

class FStopStreamingRequest final : public FControlRequest
{
public:

	using ResponseType = FStopStreamingResponse;

	UE_API FStopStreamingRequest(TOptional<TArray<FString>> InSubjectIds = TOptional<TArray<FString>>());
	
	UE_API virtual TSharedPtr<FJsonObject> GetBody() const override;
	
private:
	TOptional<TArray<FString>> SubjectIds;
};

class FListAllCapabilitiesRequest final : public FControlRequest
{
public:
	using ResponseType = FListAllCapabilitiesResponse;

	UE_API FListAllCapabilitiesRequest();
};

class FSetCapabilityPropertyRequest final : public FControlRequest
{
public:
	using ResponseType = FSetCapabilityPropertyResponse;

	UE_API FSetCapabilityPropertyRequest(FString InCapabilityId, FString InPropertyId, FCapabilityValue InValue);

	UE_API virtual TSharedPtr<FJsonObject> GetBody() const override;

private:
	FCapabilityValue Value;
};

class FGetCapabilityPropertyRequest final : public FControlRequest
{
public:
	using ResponseType = FGetCapabilityPropertyResponse;

	UE_API FGetCapabilityPropertyRequest(FString InCapabilityId, FString InPropertyId);
};

class FExecuteCapabilityCommandRequest final : public FControlRequest
{
public:
	using ResponseType = FExecuteCapabilityCommandResponse;

	UE_API FExecuteCapabilityCommandRequest(FString InCapabilityId, FString InCommandId, FCapabilityValues InArguments);

	UE_API virtual TSharedPtr<FJsonObject> GetBody() const override;

private:
	FCapabilityValues Arguments;
};

class FCapabilitySubscribeRequest final : public FControlRequest
{
public:
	using ResponseType = FCapabilitySubscribeResponse;

	UE_API FCapabilitySubscribeRequest(FString InCapabilityId);
};

class FCapabilityUnsubscribeRequest final : public FControlRequest
{
public:
	using ResponseType = FCapabilityUnsubscribeResponse;

	UE_API FCapabilityUnsubscribeRequest(FString InCapabilityId);
};

} // namespace UE::CaptureManager

#undef UE_API
