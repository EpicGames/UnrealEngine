// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlJsonUtilities.h"
#include "Control/Capabilities/Capability.h"

#define UE_API CAPTUREPROTOCOLSTACK_API

namespace UE::CaptureManager
{

class FControlResponse
{
public:

	UE_API FControlResponse(FString InAddressPath);
	virtual ~FControlResponse() = default;

	UE_API virtual TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody);

	UE_API void SetAddressPath(const FString& InAddressPath);
	UE_API const FString& GetAddressPath() const;

private:

	FString AddressPath;
};

class FKeepAliveResponse final : public FControlResponse
{
public:

	UE_API FKeepAliveResponse();
};

class FStartSessionResponse final : public FControlResponse
{
public:

	UE_API FStartSessionResponse();

	UE_API virtual TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API const FString& GetSessionId() const;

private:

	FString SessionId;
};

class FStopSessionResponse final : public FControlResponse
{
public:

	UE_API FStopSessionResponse();
};

class FGetServerInformationResponse final : public FControlResponse
{
public:

	UE_API FGetServerInformationResponse();

	UE_API virtual TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API const FString& GetId() const;
	UE_API const FString& GetName() const;
	UE_API const FString& GetModel() const;
	UE_API const FString& GetPlatformName() const;
	UE_API const FString& GetPlatformVersion() const;
	UE_API const FString& GetSoftwareName() const;
	UE_API const FString& GetSoftwareVersion() const;
	UE_API uint16 GetExportPort() const;

private:

	FString Id;
	FString Name;
	FString Model;
	FString PlatformName;
	FString PlatformVersion;
	FString SoftwareName;
	FString SoftwareVersion;
	uint16 ExportPort = 0;
};

class FSubscribeResponse final : public FControlResponse
{
public:

	UE_API FSubscribeResponse();
};

class FUnsubscribeResponse final : public FControlResponse
{
public:

	UE_API FUnsubscribeResponse();
};


class FGetStateResponse final : public FControlResponse
{
public:

	UE_API FGetStateResponse();

	UE_API virtual TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API bool IsRecording() const;
	UE_API const TSharedPtr<FJsonObject>& GetPlatformState() const;

private:

	bool bIsRecording;
	TSharedPtr<FJsonObject> PlatformState;
};

class FStartRecordingTakeResponse final : public FControlResponse
{
public:

	UE_API FStartRecordingTakeResponse();
};

class FStopRecordingTakeResponse final : public FControlResponse
{
public:

	UE_API FStopRecordingTakeResponse();

	UE_API virtual TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API const FString& GetTakeName() const;
private:

	FString TakeName;
};

class FAbortRecordingTakeResponse final : public FControlResponse
{
public:

	UE_API FAbortRecordingTakeResponse();
};

class FGetTakeListResponse final : public FControlResponse
{
public:

	UE_API FGetTakeListResponse();

	UE_API virtual TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API const TArray<FString>& GetNames() const;

private:

	TArray<FString> Names;
};

class FGetTakeMetadataResponse final : public FControlResponse
{
public:

	struct FFileObject
	{
		FString Name;
		uint64 Length = 0;
	};

	struct FVideoObject
	{
		uint64 Frames = 0;
		uint16 FrameRate = 0;
		uint32 Height = 0;
		uint32 Width = 0;
	};

	struct FAudioObject
	{
		uint8 Channels = 0;
		uint32 SampleRate = 0;
		uint8 BitsPerChannel = 0;
	};

	struct FTakeObject
	{
		FString Name;
		FString Slate;
		uint16 TakeNumber = 0;
		FString DateTime;
		FString AppVersion;
		FString Model;
		FString Subject;
		FString Scenario;
		TArray<FString> Tags;

		TArray<FFileObject> Files;

		FVideoObject Video;
		FAudioObject Audio;
	};

	UE_API FGetTakeMetadataResponse();

	UE_API virtual TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API const TArray<FTakeObject>& GetTakes() const;

private:

	UE_API TProtocolResult<void> CreateTakeObject(const TSharedPtr<FJsonObject>& InTakeObject, FTakeObject& OutTake) const;
	UE_API TProtocolResult<void> CreateFileObject(const TSharedPtr<FJsonObject>& InFileObject, FFileObject& OutFile) const;

	TArray<FTakeObject> Takes;
};

class FGetStreamingSubjectsResponse final : public FControlResponse
{
public:
	
	struct FAnimationMetadata
	{
		FString Type;
		uint16 Version;
		TArray<FString> Controls;
	};
	
	struct FSubject
	{
		FString Id;
		FString Name;
		FAnimationMetadata AnimationMetadata;
	};
	
	UE_API FGetStreamingSubjectsResponse();

	UE_API virtual TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API const TArray<FSubject>& GetSubjects() const;

private:
	
	TArray<FSubject> Subjects;
	
	UE_API TProtocolResult<void> CreateSubject(const TSharedPtr<FJsonObject>& InSubjectObject, FSubject& OutSubject) const;
	UE_API TProtocolResult<void> CreateAnimationMetadata(const TSharedPtr<FJsonObject>& InAnimationObject, FAnimationMetadata& OutAnimationMetadata) const;
};

class FStartStreamingResponse final : public FControlResponse
{
public:

	UE_API FStartStreamingResponse();
	
};

class FStopStreamingResponse final : public FControlResponse
{
public:
	UE_API FStopStreamingResponse();
};

class FListAllCapabilitiesResponse final : public FControlResponse
{
public:
	UE_API FListAllCapabilitiesResponse();

	UE_API TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API TArray<FCapability> GetCapabilities();

private:
	TArray<FCapability> Capabilities;

	TProtocolResult<FCapability> ParseCapability(const TSharedPtr<FJsonObject>& InJsonObject);
};

class FSetCapabilityPropertyResponse final : public FControlResponse
{
public:
	UE_API FSetCapabilityPropertyResponse();
};

class FGetCapabilityPropertyResponse final : public FControlResponse
{
public:
	UE_API FGetCapabilityPropertyResponse();

	UE_API TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API TProtocolResult<FCapabilityValue> GetValue(ECapabilityValue::Type InExpectedType) const;

private:
	TSharedPtr<FJsonObject> Body;
};

class FExecuteCapabilityCommandResponse final : public FControlResponse
{
public:
	UE_API FExecuteCapabilityCommandResponse();

	UE_API TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API TProtocolResult<FCapabilityValue> GetReturnValue(ECapabilityValue::Type InExpectedType) const;

private:
	TSharedPtr<FJsonObject> Body;
};

class FCapabilitySubscribeResponse final : public FControlResponse
{
public:
	UE_API FCapabilitySubscribeResponse();
};

class FCapabilityUnsubscribeResponse final : public FControlResponse
{
public:
	UE_API FCapabilityUnsubscribeResponse();
};
} // namespace UE::CaptureManager
#undef UE_API
