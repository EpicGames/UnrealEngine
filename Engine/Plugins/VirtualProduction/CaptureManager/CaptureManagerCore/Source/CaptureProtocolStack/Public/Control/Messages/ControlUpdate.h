// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlJsonUtilities.h"
#include "Containers/Map.h"

#include "Utility/Error.h"

#include "Delegates/Delegate.h"
#include "Control/Capabilities/Capability.h"

#define UE_API CAPTUREPROTOCOLSTACK_API

namespace UE::CaptureManager
{

class FControlUpdate;
class FControlUpdateCreator final
{
public:
	static TProtocolResult<TSharedRef<FControlUpdate>> Create(const FString& InAddressPath);
};

class FControlUpdate
{
public:

	DECLARE_DELEGATE_OneParam(FOnUpdateMessage, TSharedPtr<FControlUpdate> InUpdateMessage);

	UE_API FControlUpdate(FString InAddressPath);
	virtual ~FControlUpdate() = default;

	UE_API const FString& GetAddressPath() const;

	UE_API virtual TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody);

private:

	FString AddressPath;
};

class FSessionStopped final : public FControlUpdate
{
public:

	UE_API FSessionStopped();
};

class FRecordingStatusUpdate final : public FControlUpdate
{
public:

	UE_API FRecordingStatusUpdate();

	UE_API virtual TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API bool IsRecording() const;

private:

	bool bIsRecording;
};

class FBaseTakeUpdate : public FControlUpdate
{
public:

	using FControlUpdate::FControlUpdate;

	UE_API virtual TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API const FString& GetTakeName() const;

private:

	FString TakeName;
};

class FTakeAddedUpdate final : public FBaseTakeUpdate
{
public:

	UE_API FTakeAddedUpdate();
};

class FTakeRemovedUpdate final : public FBaseTakeUpdate
{
public:

	UE_API FTakeRemovedUpdate();
};

class FTakeUpdatedUpdate final : public FBaseTakeUpdate
{
public:

	UE_API FTakeUpdatedUpdate();
};

// IOS

class FDiskCapacityUpdate final : public FControlUpdate
{
public:

	UE_API FDiskCapacityUpdate();

	UE_API virtual TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API uint64 GetTotal() const;
	UE_API uint64 GetRemaining() const;

private:

	uint64 Total = 0;
	uint64 Remaining = 0;
};

class FBatteryPercentageUpdate final : public FControlUpdate
{
public:

	UE_API FBatteryPercentageUpdate();

	UE_API virtual TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API float GetLevel() const;

private:

	float Level = 0.0f;
};

class FThermalStateUpdate final : public FControlUpdate
{
public:

	enum class EState
	{
		Nominal = 0,
		Fair,
		Serious,
		Critical,

		Invalid
	};

	UE_API FThermalStateUpdate();

	UE_API virtual TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API EState GetState() const;

private:

	static UE_API EState ConvertState(const FString& InStateString);

	EState State;
};

class FEventUpdate final : public FControlUpdate
{
public:
	UE_API FEventUpdate(FString InAddressPath);

	UE_API virtual UE::CaptureManager::TProtocolResult<void> Parse(TSharedPtr<FJsonObject> InBody) override;

	UE_API const TProtocolResult<TMap<FString, FCapabilityValue>> GetArguments(const TArray<FEventArgumentDescriptor>& InArgumentsDescriptor) const;

private:
	TMap<FString, TSharedPtr<FJsonObject>> Arguments;
};
} // namespace UE::CaptureManager

#undef UE_API
