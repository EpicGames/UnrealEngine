// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DiscoveryPacket.h"

#include "Network/UdpClient.h"

#include "Utility/Error.h"
#include "Async/QueueRunner.h"

namespace UE::CaptureManager
{

class FDiscoveryCommunication final
{
private:
	struct FContext
	{
		FString ServerIp;
		FDiscoveryPacket Packet;
	};

public:

	static const uint16 MulticastPort;
	static const FString MulticastAddress;

	DECLARE_DELEGATE_TwoParams(FOnPacketReceived, FString ServerIp, FDiscoveryPacket InPacket)

	using FRunnerType = TQueueRunner<FContext>;

	FDiscoveryCommunication();

	TProtocolResult<void> Start();
	TProtocolResult<void> Stop();

	void SendMessage(FDiscoveryPacket InMessage, const FString& Endpoint);
	void SendMessage(FDiscoveryPacket InMessage, const FString& EndpointIp, const uint16 EndpointPort);

	void SetReceiveHandler(FOnPacketReceived InReceiveHandler);

private:

	void OnProcessReceivedPacket(FContext InContext);
	void OnPacketArrived(const FArrayReaderPtr& InPayload, const FIPv4Endpoint& InEndpoint);

	FUdpClient Client;

	FRunnerType SynchronizedReceiver;

	FOnPacketReceived OnPacketReceived;
};

}