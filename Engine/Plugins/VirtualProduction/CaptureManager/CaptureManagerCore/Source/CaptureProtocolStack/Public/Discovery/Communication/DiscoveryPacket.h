// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Utility/Definitions.h"
#include "Utility/Error.h"

#include "Containers/Array.h"

namespace UE::CaptureManager
{

class FDiscoveryPacket final
{
public:

	enum class EMessageType : uint8
	{
		Request = 0,
		Response = 1,
		Notify = 2,

		Invalid = 255
	};

	static const TArray<uint8> Header;
	static const uint16 Version;

	static TProtocolResult<FDiscoveryPacket> Deserialize(const TConstArrayView<uint8>& InData);
	static TProtocolResult<TArray<uint8>> Serialize(const FDiscoveryPacket& InPacket);

	FDiscoveryPacket() = default;
	FDiscoveryPacket(EMessageType InMessageType, TArray<uint8> InPayload);

	FDiscoveryPacket(const FDiscoveryPacket& InOther) = default;
	FDiscoveryPacket(FDiscoveryPacket&& InOther) = default;

	FDiscoveryPacket& operator=(const FDiscoveryPacket& InOther) = default;
	FDiscoveryPacket& operator=(FDiscoveryPacket&& InOther) = default;

	const EMessageType GetMessageType() const;
	const TArray<uint8>& GetPayload() const;

private:

	static EMessageType ToMessageType(uint8 InMessageType);
	static uint8 FromMessageType(EMessageType InMessageType);

	EMessageType MessageType;
	TArray<uint8> Payload;
};

}