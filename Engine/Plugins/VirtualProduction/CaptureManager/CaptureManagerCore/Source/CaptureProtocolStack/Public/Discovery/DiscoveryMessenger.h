// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Discovery/Communication/DiscoveryCommunication.h"
#include "Discovery/Communication/DiscoveryPacket.h"

#include "Discovery/Messages/DiscoveryRequest.h"
#include "Discovery/Messages/DiscoveryResponse.h"
#include "Discovery/Messages/DiscoveryNotify.h"

#include "Utility/Error.h"

#define UE_API CAPTUREPROTOCOLSTACK_API

namespace UE::CaptureManager
{

class FDiscoveryMessenger final
{
public:
	using FOnResponseArrived = TDelegate<void(FString, FDiscoveryResponse), FDefaultTSDelegateUserPolicy>;
	using FOnNotifyArrived = TDelegate<void(FString, FDiscoveryNotify), FDefaultTSDelegateUserPolicy>;

	UE_API FDiscoveryMessenger();
	UE_API ~FDiscoveryMessenger();

	UE_API TProtocolResult<void> Start();
	UE_API TProtocolResult<void> Stop();

	UE_API TProtocolResult<void> SendRequest();

	UE_API void SetResponseHandler(FOnResponseArrived InOnResponse);
	UE_API void SetNotifyHandler(FOnNotifyArrived InOnNotify);

private:

	UE_API void OnPacketArrived(FString InServerIp, FDiscoveryPacket InPacket);

	FDiscoveryCommunication Communication;

	FOnResponseArrived OnResponse;
	FOnNotifyArrived OnNotify;
};

}

#undef UE_API
