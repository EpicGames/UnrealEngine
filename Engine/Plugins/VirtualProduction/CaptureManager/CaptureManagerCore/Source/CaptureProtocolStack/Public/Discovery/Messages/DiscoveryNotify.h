// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Discovery/Communication/DiscoveryPacket.h"

#include "Utility/Error.h"

#include "Containers/Array.h"
#include "Containers/StaticArray.h"

#define UE_API CAPTUREPROTOCOLSTACK_API

namespace UE::CaptureManager
{

class FDiscoveryNotify
{
public:

	static UE_API const uint32 MinPayloadSize;

	using FServerId = TStaticArray<uint8, 16>;

	enum class EConnectionState : uint8
	{
		Offline = 0,
		Online = 1,

		Invalid
	};

	UE_DEPRECATED(5.6, "This constructor is no longer supported")
	UE_API FDiscoveryNotify(FServerId InServerId, uint16 InControlPort, EConnectionState InConnectionState, TArray<uint16> InSupportedVersions);
	
	UE_API FDiscoveryNotify(FServerId InServerId, FString InServerName, uint16 InControlPort, EConnectionState InConnectionState);

	static UE_API TProtocolResult<FDiscoveryNotify> Deserialize(const FDiscoveryPacket& InPacket);
	static UE_API TProtocolResult<FDiscoveryPacket> Serialize(const FDiscoveryNotify& InNotify);

	UE_API const FServerId& GetServerId() const;
	UE_API const FString& GetServerName() const;
	UE_API uint16 GetControlPort() const;
	UE_API EConnectionState GetConnectionState() const;
	
	UE_DEPRECATED(5.6, "GetSupportedVersions is no longer supported")
	UE_API const TArray<uint16>& GetSupportedVersions() const;

private:

	static UE_API EConnectionState ToConnectionState(uint8 InConnectionState);
	static UE_API uint8 FromConnectionState(EConnectionState InConnectionState);

	FServerId ServerId;
	FString ServerName;
	uint16 ControlPort;
	EConnectionState ConnectionState;

	// Deprecated in 5.6.
	TArray<uint16> SupportedVersions;
};

}

#undef UE_API
