// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Discovery/Communication/DiscoveryPacket.h"

#include "Utility/Error.h"

#include "Containers/Array.h"
#include "Containers/StaticArray.h"

namespace UE::CaptureManager
{

class FDiscoveryRequest
{
public:

	static const uint32 PayloadSize;

	FDiscoveryRequest() = default;

	static TProtocolResult<FDiscoveryRequest> Deserialize(const FDiscoveryPacket& InPacket);
	static TProtocolResult<FDiscoveryPacket> Serialize(const FDiscoveryRequest& InRequest);
};

}