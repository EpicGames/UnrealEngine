// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExportHeader.h"

#include "ExportClient/Messages/ExportRequest.h"
#include "ExportClient/Messages/ExportResponse.h"

#include "Network/TcpClient.h"

#include "Async/QueueRunner.h"

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"

namespace UE::CaptureManager
{

struct FExportRequestPacket
{
	static TProtocolResult<void> Serialize(const FExportRequestPacket& InRequestPacket, ITcpSocketWriter& InWriter);

	FExportHeader Header;
	FExportRequest Request;
};

struct FExportResponseHeader
{
	static TProtocolResult<FExportResponseHeader> Deserialize(ITcpSocketReader& InReader);

	FExportHeader Header;
	FExportResponse Response;
};

class FExportCommunication final
{
public:

	FExportCommunication();

	TProtocolResult<void> Init();
	TProtocolResult<void> Start(const FString& InServerIp, const uint16 InServerPort);
	TProtocolResult<void> Stop();

	bool IsRunning() const;

	TProtocolResult<void> SendRequest(FExportHeader InHeader, FExportRequest InRequest);
	TProtocolResult<FExportResponseHeader> ReceiveResponseHeader();
	TProtocolResult<TArray<uint8>> ReceiveResponseData(const uint64 InSize);
	TProtocolResult<TStaticArray<uint8, 16>> ReceiveFileHash();

private:

	FTcpClient Client;
};

}