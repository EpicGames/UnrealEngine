// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/TcpReaderWriter.h"

#include "Utility/Error.h"

#include "Containers/Array.h"
#include "Containers/StaticArray.h"

namespace UE::CaptureManager
{

class FExportHeader final
{
public:

	static const TArray<uint8> Header;

	FExportHeader() = default;
	FExportHeader(uint16 InVersion, uint32 InTransactionId);

	FExportHeader(const FExportHeader& InOther) = default;
	FExportHeader(FExportHeader&& InOther) = default;

	FExportHeader& operator=(const FExportHeader& InOther) = default;
	FExportHeader& operator=(FExportHeader&& InOther) = default;

	static TProtocolResult<FExportHeader> Deserialize(ITcpSocketReader& InReader);
	static TProtocolResult<void> Serialize(const FExportHeader& InHeader, ITcpSocketWriter& InWriter);

	uint16 GetVersion() const;
	uint32 GetTransactionId() const;

private:

	uint16 Version = 0;
	uint32 TransactionId = 0;
};

}