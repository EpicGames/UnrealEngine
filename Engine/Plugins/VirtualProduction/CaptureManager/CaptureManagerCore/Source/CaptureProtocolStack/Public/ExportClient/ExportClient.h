// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExportClient/Communication/ExportCommunication.h"
#include "ExportWorker.h"

#include "Containers/Map.h"

#define UE_API CAPTUREPROTOCOLSTACK_API

namespace UE::CaptureManager
{

class FExportTaskStopToken
{
public:

	FExportTaskStopToken();

	void Cancel();
	bool IsCanceled();

private:

	std::atomic_bool bCanceled;
};

class FExportClient final
	: public FExportTaskExecutor
{
public:

	using FTaskId = uint32;
	static constexpr int32 AbortedError = -20;

	using FTakeFileArray = TArray<FTakeFile>;

	UE_API FExportClient(FString InServerIp, const uint16 InExportPort);
	UE_API ~FExportClient();

	UE_API FTaskId ExportTakeFiles(FString InTakeName, FTakeFileArray InTakeFileArray, TUniquePtr<FBaseStream> InStream);
	UE_API FTaskId ExportFiles(TMap<FString, FTakeFileArray> InTakesFilesMap, TUniquePtr<FBaseStream> InStream);

	UE_API void AbortExport(const FTaskId InTaskId);
	UE_API void AbortAllExports();

private:

	UE_API FTaskId StartExport(FExportTakeTask::FExportContexts InContexts, TUniquePtr<FBaseStream> InStream);

	UE_API virtual void OnTask(TUniquePtr<FExportTakeTask> InTask) override;
	UE_API bool OnExportTask(TUniquePtr<FExportTakeTask> InTask);

	UE_API TProtocolResult<TMap<uint32, TPair<FString, FTakeFile>>> SendRequests(FExportTakeTask::FExportContexts InContexts);
	UE_API TProtocolResult<void> ReceiveResponses(TMap<uint32, TPair<FString, FTakeFile>> InResponseMap,
										   FBaseStream& InBaseStream);

	UE_API TProtocolResult<void> Start();
	UE_API TProtocolResult<void> Stop();

	FString ServerIp;
	uint16 ServerPort = 0;
	TUniquePtr<FExportCommunication> Communication;

	FExportWorker ExportWorker;
	TUniquePtr<FRunnableThread> Thread;

	std::atomic<FTaskId> CurrentTaskId{0};

	uint32 TransactionIdCounter = 0;

	TUniquePtr<FExportTaskStopToken> CurrentTaskStopToken;
};

}

#undef UE_API
