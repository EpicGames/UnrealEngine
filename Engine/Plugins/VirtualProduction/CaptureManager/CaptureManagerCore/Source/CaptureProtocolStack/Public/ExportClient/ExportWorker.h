// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExportClient/Communication/ExportCommunication.h"

#include "HAL/UESemaphore.h"

namespace UE::CaptureManager
{

class FBaseStream;
struct FTakeFile
{
	FString FileName;
	uint64 Length = 0;
	uint64 Offset = 0;
};

class FExportTakeTask
{
public:

	struct FExportContext
	{
		FString TakeName;
		FTakeFile File;
	};

	using FExportContexts = TArray<FExportContext>;

	FExportTakeTask(FExportContexts InExportContexts,
					TUniquePtr<FBaseStream> InStream);

	FExportContexts ExportContexts;
	TUniquePtr<FBaseStream> Stream;
};

class FExportTaskExecutor
{
public:
	virtual ~FExportTaskExecutor() = default;

	virtual void OnTask(TUniquePtr<FExportTakeTask> InTask) = 0;
};

class FExportQueue
{
public:
	static const int32 MaxNumberOfElements = MAX_int32;

	FExportQueue();
	~FExportQueue() = default;

	TProtocolResult<void> Add(const uint32 InTaskId, TUniquePtr<FExportTakeTask> InElement);
	TUniquePtr<FExportTakeTask> Pop();

	TProtocolResult<TUniquePtr<FExportTakeTask>> Remove(uint32 InTaskId);

	bool IsEmpty() const;

	TArray<TUniquePtr<FExportTakeTask>> GetAndEmpty();

private:

	mutable FCriticalSection Mutex;
	FSemaphore Semaphore;
	TArray<TPair<uint32, TUniquePtr<FExportTakeTask>>> Queue;
};

class FExportWorker : public FRunnable
{
public:

	FExportWorker(FExportTaskExecutor& InClient);
	virtual ~FExportWorker() override;

	TProtocolResult<void> Add(const uint32 InTaskId, TUniquePtr<FExportTakeTask> InElement);
	TProtocolResult<TUniquePtr<FExportTakeTask>> Remove(uint32 InTaskId);

	bool IsEmpty() const;

	TArray<TUniquePtr<FExportTakeTask>> GetAndEmpty();

protected:

	virtual uint32 Run() override;
	virtual void Stop() override;

private:

	std::atomic<bool> bRunning;

	FExportQueue Queue;

	FExportTaskExecutor& Client;
};

class FBaseStream
{
public:

	FBaseStream() = default;
	virtual ~FBaseStream() = default;

	virtual TProtocolResult<void> StartFile(const FString& InTakeName, const FString& InFileName) = 0;
	virtual TProtocolResult<void> ProcessData(const FString& InTakeName, const FString& InFileName, const TConstArrayView<uint8>& InData) = 0;
	virtual TProtocolResult<void> FinishFile(const FString& InTakeName, const FString& InFileName, const TStaticArray<uint8, 16>& InHash) = 0;

	virtual void Finalize(TProtocolResult<void> InResult) = 0;
};

}
