// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExportClient/Communication/ExportHeader.h"

namespace UE::CaptureManager
{

class FExportRequest final
{
public:

	FExportRequest() = default;
	FExportRequest(FString InTakeName, FString InFileName, uint64 InOffset);

	FExportRequest(const FExportRequest& InOther) = default;
	FExportRequest(FExportRequest&& InOther) = default;

	FExportRequest& operator=(const FExportRequest& InOther) = default;
	FExportRequest& operator=(FExportRequest&& InOther) = default;

	static TProtocolResult<FExportRequest> Deserialize(ITcpSocketReader& InReader);
	static TProtocolResult<void> Serialize(const FExportRequest& InRequest, ITcpSocketWriter& InWriter);

	const FString& GetTakeName() const;
	const FString& GetFileName() const;
	uint64 GetOffset() const;

private:

	FString TakeName;
	FString FileName;
	uint64 Offset = 0;
};

}