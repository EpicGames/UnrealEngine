// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExportClient/Communication/ExportHeader.h"

namespace UE::CaptureManager
{

class FExportResponse final
{
public:

	enum class EStatus : uint8
	{
		Success = 0,
		InvalidTakeName = 1,
		InvalidFileName = 2,
		InvalidOffset = 3,
		ServerError = 4,
		UnsupportedProtocolVersion = 5,

		Reserved
	};

	FExportResponse(EStatus InStatus, uint64 InLength);

	FExportResponse(const FExportResponse& InOther) = default;
	FExportResponse(FExportResponse&& InOther) = default;

	FExportResponse& operator=(const FExportResponse& InOther) = default;
	FExportResponse& operator=(FExportResponse&& InOther) = default;

	static TProtocolResult<FExportResponse> Deserialize(ITcpSocketReader& InReader);
	static TProtocolResult<TStaticArray<uint8, 16>> DeserializeHash(ITcpSocketReader& InReader);

	static TProtocolResult<void> Serialize(const FExportResponse& InResponse, ITcpSocketWriter& InWriter);
	static TProtocolResult<void> SerializeHash(const TStaticArray<uint8, 16>& InHash, ITcpSocketWriter& InWriter);

	EStatus GetStatus() const;
	uint64 GetLength() const;

private:

	FExportResponse() = default;

	static EStatus ToStatus(uint8 InStatus);
	static uint8 FromStatus(EStatus InStatus);

	EStatus Status;
	uint64 Length;
};

}