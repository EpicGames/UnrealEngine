// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

#define CPS_VERSION 1

namespace UE::CaptureManager
{

DECLARE_LOG_CATEGORY_EXTERN(LogCPSControlMessenger, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogCPSControlCommunication, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogCPSDiscoveryMessenger, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogCPSDiscoveryCommunication, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogCPSExportHandler, Log, All);

}