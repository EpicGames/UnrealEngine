// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

#include "Templates/ValueOrError.h"

#include "Network/Error.h"

#define CPS_CHECK_VOID_RESULT(Function) if (UE::CaptureManager::TProtocolResult<void> Result = Function; Result.HasError()) { return Result.StealError(); }