// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/ValueOrError.h"

namespace UE::CaptureManager
{

class FStopToken;

enum class EProcessRunnerError : int32
{
	PipeCreationFailed,
	LaunchFailed,
	Timeout,
	Cancelled,
	FailedToGetReturnCode,
	ExitedWithError
};

using FProcessRunnerResult = TValueOrError<TArray<uint8>, EProcessRunnerError>;

class UE_INTERNAL CAPTUREUTILS_API FProcessRunner
{
public:
	static FProcessRunnerResult Run(
		const FString& InProcessPath, 
		const FString& InProcessArguments, 
		const FStopToken* InStopToken = nullptr, 
		const TOptional<int32> InTimeoutSeconds = TOptional<int32>());

private:
	static TArray<uint8> ReadPipe(void* InPipe);
	static TArray<uint8> DrainPipe(void* InPipe);
};

}
