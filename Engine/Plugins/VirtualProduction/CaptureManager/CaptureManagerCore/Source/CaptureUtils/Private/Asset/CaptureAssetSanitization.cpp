// Copyright Epic Games, Inc. All Rights Reserved.

#include "Asset/CaptureAssetSanitization.h"

#include "Internationalization/Text.h"

#include "Logging/LogMacros.h"
#include "Misc/Paths.h"

DEFINE_LOG_CATEGORY_STATIC(LogCaptureUtilsAssetValidation, Log, All);

namespace UE::CaptureManager::Private
{

static void Sanitize(FString& OutPath, const FString& InInvalidChars, const FString::ElementType InReplaceWith)
{
	// Make sure the package path only contains valid characters
	FText OutError;
	if (FName::IsValidXName(OutPath, InInvalidChars, &OutError))
	{
		return;
	}
	UE_LOGF(LogCaptureUtilsAssetValidation, Display, "%ls", *OutError.ToString());

	for (const FString::ElementType& InvalidChar : InInvalidChars)
	{
		OutPath.ReplaceCharInline(InvalidChar, InReplaceWith, ESearchCase::CaseSensitive);
	}

	UE_LOGF(LogCaptureUtilsAssetValidation, Display, "Sanitized path: %ls", *OutPath);
}

}

namespace UE::CaptureManager
{

void SanitizePackagePath(FString& OutPath, const FString::ElementType InReplaceWith)
{	
	FPaths::RemoveDuplicateSlashes(OutPath);
	return Private::Sanitize(OutPath, INVALID_OBJECTPATH_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, InReplaceWith);
}

void SanitizeAssetName(FString& OutPath, const FString::ElementType InReplaceWith)
{
	return Private::Sanitize(OutPath, INVALID_OBJECTNAME_CHARACTERS, InReplaceWith);
}

}