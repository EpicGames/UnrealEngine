// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/CallbackSynchronizer.h"

namespace UE::CaptureManager
{

FCallbackSynchronizer::FCallbackSynchronizer() = default;

FCallbackSynchronizer::~FCallbackSynchronizer()
{
	check(Counter == 0);
}

void FCallbackSynchronizer::AfterAll(FAfterAllDelegate InAfterAllDelegate, bool bExecuteIfCounterZero)
{
	AfterAllDelegate = MoveTemp(InAfterAllDelegate);

	if (Counter == 0 && bExecuteIfCounterZero)
	{
		AfterAllDelegate.ExecuteIfBound();
	}
}

TSharedPtr<FCallbackSynchronizer> FCallbackSynchronizer::Create()
{
	return MakeShared<FCallbackSynchronizer>();
}

void FCallbackSynchronizer::Decrease()
{
	if (--Counter == 0)
	{
		AfterAllDelegate.ExecuteIfBound();
	}
}

}