// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/CaptureTimerManager.h"

namespace UE::CaptureManager
{

FCaptureTimerManager::FCaptureTimerManager()
	: FCaptureTimerManager(IdealTimeResolution)
{
}

FCaptureTimerManager::FCaptureTimerManager(const float Resolution)
{
	Runnable = MakeUnique<FTimerManagerRunnable>(this, Resolution);
	Thread.Reset(FRunnableThread::Create(Runnable.Get(), TEXT("Timer Manager"), 128 * 1024, TPri_Normal, FPlatformAffinity::GetPoolThreadMask()));
}

FCaptureTimerManager::~FCaptureTimerManager()
{
	Thread->Kill(true);

	Thread = nullptr;
	Runnable = nullptr;
}

FCaptureTimerManager::FTimerHandle FCaptureTimerManager::AddTimer(FTimerDelegate InDelegate, float InRate, bool InbLoop, float InFirstDelay)
{
	FCaptureTimerManager::FTimerHandle Handle = Ticker.AddTicker(TEXT("Timer"), InRate, [InbLoop, Delegate = MoveTemp(InDelegate)](float InDelay)
	{
		Delegate.Execute();

		return InbLoop;
	});

	Handle.Pin()->FireTime -= InRate - InFirstDelay;

	return Handle;
}

void FCaptureTimerManager::RemoveTimer(FTimerHandle Handle)
{
	Ticker.RemoveTicker(MoveTemp(Handle));
}

FCaptureTimerManager::FTimerManagerRunnable::FTimerManagerRunnable(FCaptureTimerManager* InOwner, const float InResolution)
	: Owner(InOwner)
	, IdealTimeResolution(InResolution)
{
}

uint32 FCaptureTimerManager::FTimerManagerRunnable::Run()
{
	double LastTime = FPlatformTime::Seconds();

	while (bShouldRun.load())
	{
		const double CurrentTime = FPlatformTime::Seconds();
		const double DeltaTime = CurrentTime - LastTime;

		Owner->Ticker.Tick(DeltaTime);

		FPlatformProcess::Sleep(FMath::Max<float>(0.0f, IdealTimeResolution - (FPlatformTime::Seconds() - LastTime)));

		LastTime = CurrentTime;
	}

	Owner->Ticker.Reset();

	return 0;
}

void FCaptureTimerManager::FTimerManagerRunnable::Stop()
{
	bShouldRun.store(false);
}

}