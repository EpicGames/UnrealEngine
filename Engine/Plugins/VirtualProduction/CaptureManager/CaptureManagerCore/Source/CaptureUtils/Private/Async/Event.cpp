// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/Event.h"
#include "Misc/ScopeRWLock.h"

namespace UE::CaptureManager
{

FCaptureEvent::FCaptureEvent(FString InName) : Name(MoveTemp(InName))
{
}

const FString& FCaptureEvent::GetName() const
{
	return Name;
}

FCaptureEvent::~FCaptureEvent() = default;

}