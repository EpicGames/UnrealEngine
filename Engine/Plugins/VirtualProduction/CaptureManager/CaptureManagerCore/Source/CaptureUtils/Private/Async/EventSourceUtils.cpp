// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/EventSourceUtils.h"
#include "Misc/ScopeRWLock.h"

namespace UE::CaptureManager::Private
{
	FCaptureEventSourceBase::~FCaptureEventSourceBase() = default;

	TArray<FString> FCaptureEventSourceBase::GetAvailableEvents() const
	{
		TArray<FString> Keys;
		FRWScopeLock Guard(HandlersLock, FRWScopeLockType::SLT_ReadOnly);
		EventToHandlersMap.GetKeys(Keys);
		return Keys;
	}

	void FCaptureEventSourceBase::SubscribeToEvent(const FString& InEventName, FCaptureEventHandler InHandler)
	{
		FRWScopeLock Guard(HandlersLock, FRWScopeLockType::SLT_Write);
		FHandlers* Handlers = EventToHandlersMap.Find(InEventName);
		check(Handlers != nullptr);
		Handlers->AddLambda(MoveTemp(InHandler));
	}

	void FCaptureEventSourceBase::UnsubscribeAll()
	{
		FRWScopeLock Guard(HandlersLock, FRWScopeLockType::SLT_Write);
		for (TPair<FString, FHandlers>& EventHandlersPair : EventToHandlersMap)
		{
			EventHandlersPair.Value.Clear();
		}
	}

	void FCaptureEventSourceBase::RegisterEvent(const FString& InEventName)
	{
		FRWScopeLock Guard(HandlersLock, FRWScopeLockType::SLT_Write);
		check(EventToHandlersMap.Find(InEventName) == nullptr);
		EventToHandlersMap.Add({ InEventName, FHandlers{} });
	}

	void FCaptureEventSourceBase::PublishEventInternal(TSharedPtr<const FCaptureEvent> InEvent) const
	{
		FRWScopeLock Guard(HandlersLock, FRWScopeLockType::SLT_ReadOnly);
		const FHandlers* Handlers = EventToHandlersMap.Find(InEvent->GetName());
		check(Handlers != nullptr);
		Handlers->Broadcast(InEvent);
	}
}