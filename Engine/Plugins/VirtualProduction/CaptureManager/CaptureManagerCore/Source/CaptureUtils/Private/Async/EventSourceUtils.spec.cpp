// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/Event.h"
#include "Async/EventSourceUtils.h"

#include "Misc/AutomationTest.h"

#include "CoreGlobals.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::CaptureManager
{

BEGIN_DEFINE_SPEC(FCaptureEventSourceTest, "System.Engine.Animation.LiveLink.CaptureUtils.CaptureSourceEvent", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter | EAutomationTestFlags::MediumPriority)

struct FTestCaptureEvent : public FCaptureEvent
{
	FTestCaptureEvent(const FString& InName, int InValue) : FCaptureEvent(InName), Value(InValue)
	{
	}

	int Value;
};

struct FTestCaptureEventSource : virtual public FCaptureEventSource
{
	FTestCaptureEventSource(const TArray<FString>& InEventsToRegister)
	{
		for (const FString& EventName : InEventsToRegister)
		{
			RegisterEvent(EventName);
		}
	}

	void DoPublish(const FString& InName, int InValue)
	{
		PublishEvent<FTestCaptureEvent>(InName, InValue);
	}
};

END_DEFINE_SPEC(FCaptureEventSourceTest)

void FCaptureEventSourceTest::Define()
{
	Describe("GetAvailableEvents()", [this]()
	{
		It("should return the list of registered events", [this]()
		{
			TArray<FString> EventsToRegister = { "TestEvent1", "TestEvent2", "TestEvent3" };
			FTestCaptureEventSource EventSource{ EventsToRegister };
			TArray<FString> AvailableEvents = EventSource.GetAvailableEvents();
			AvailableEvents.Sort();
			UTEST_EQUAL("List of available events matches the expected list of event names", AvailableEvents, EventsToRegister);

			return true;
		});
	});

	Describe("PublishEvent()", [this]()
	{
		It("should publish event to handlers registered to the event and not publish to others", [this]()
		{
			constexpr int ExpectedEventValue = 5;
			TArray<FString> EventsToRegister = { "TestEvent1", "TestEvent2" };
			FTestCaptureEventSource EventSource{ EventsToRegister };

			bool IsExpectedCalled = false;
			bool DidChecksPass = false;
			FCaptureEventHandler ExpectedHandler([&](TSharedPtr<const FCaptureEvent> InEvent)
			{
				DidChecksPass = (InEvent->GetName() == EventsToRegister[0]) &&
					(static_cast<const FTestCaptureEvent*>(InEvent.Get())->Value == ExpectedEventValue);
				IsExpectedCalled = true;
				// This needs to run on `InternalThread` because the tests are being run on the game thread so posting
				// to it and blocking it will just deadlock it.
			}, EDelegateExecutionThread::InternalThread);

			bool IsUnexpectedCalled = false;
			FCaptureEventHandler UnexpectedHandler([&](TSharedPtr<const FCaptureEvent> InEvent)
			{
				IsUnexpectedCalled = true;
			}, EDelegateExecutionThread::InternalThread);

			EventSource.SubscribeToEvent(EventsToRegister[0], MoveTemp(ExpectedHandler));
			EventSource.SubscribeToEvent(EventsToRegister[1], MoveTemp(UnexpectedHandler));

			EventSource.DoPublish(EventsToRegister[0], ExpectedEventValue);

			UTEST_TRUE("Registered handler was invoked", IsExpectedCalled);
			UTEST_TRUE("Received event should be the published one and should have the expected value", DidChecksPass);
			UTEST_FALSE("Handler registered to other event was not invoked", IsUnexpectedCalled);

			return true;
		});

		It("should publish event to all registered handlers with all handlers sharing the same event instance", [this]()
		{
			constexpr int ExpectedEventValue = 6;
			TArray<FString> EventsToRegister = { "TestEvent1" };
			FTestCaptureEventSource EventSource{ EventsToRegister };

			bool IsHandler1Called = false;
			bool DidChecksPass1 = false;
			const FCaptureEvent* EventPtr1 = nullptr;
			FCaptureEventHandler Handler1([&](TSharedPtr<const FCaptureEvent> InEvent)
			{
				DidChecksPass1 = (InEvent->GetName() == EventsToRegister[0]) &&
					(static_cast<const FTestCaptureEvent*>(InEvent.Get())->Value == ExpectedEventValue);
				IsHandler1Called = true;
				EventPtr1 = InEvent.Get();
				// This needs to run on `InternalThread` because the tests are being run on the game thread so posting
				// to it and blocking it will just deadlock it.
			}, EDelegateExecutionThread::InternalThread);

			bool IsHandler2Called = false;
			bool DidChecksPass2 = false;
			const FCaptureEvent* EventPtr2 = nullptr;
			FCaptureEventHandler Handler2([&](TSharedPtr<const FCaptureEvent> InEvent)
			{
				DidChecksPass2 = (InEvent->GetName() == EventsToRegister[0]) &&
					(static_cast<const FTestCaptureEvent*>(InEvent.Get())->Value == ExpectedEventValue);
				IsHandler2Called = true;
				EventPtr2 = InEvent.Get();
				// This needs to run on `InternalThread` because the tests are being run on the game thread so posting
				// to it and blocking it will just deadlock it.
			}, EDelegateExecutionThread::InternalThread);

			EventSource.SubscribeToEvent(EventsToRegister[0], MoveTemp(Handler1));
			EventSource.SubscribeToEvent(EventsToRegister[0], MoveTemp(Handler2));

			EventSource.DoPublish(EventsToRegister[0], ExpectedEventValue);

			UTEST_TRUE("Registered handlers were invoked", IsHandler1Called && (IsHandler1Called == IsHandler2Called));
			UTEST_TRUE("Received event should be the published one and should have the expected value", DidChecksPass1 && (DidChecksPass1 == DidChecksPass2));
			UTEST_TRUE("The published event was shared amongst handlers", EventPtr1 && (EventPtr1 == EventPtr2));

			return true;
		});
	});
}

}

#endif