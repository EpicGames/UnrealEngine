// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/HelperFunctions.h"

#include "Async/Async.h"
#include "Async/Future.h"
#include "Async/TaskGraphInterfaces.h"
#include "HAL/ThreadingBase.h"

namespace UE::CaptureManager
{

void CallOnGameThread(TFunction<void()> InFunction)
{
	if (IsInGameThread())
	{
		InFunction();
		return;
	}

	TPromise<void> Promise;
	TFuture<void> Future = Promise.GetFuture();

	AsyncTask(ENamedThreads::GameThread, [&Promise, &InFunction]
		{
			InFunction();
			Promise.SetValue();
		});

	Future.Wait();
}

}