// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/StopToken.h"

namespace UE::CaptureManager
{

struct FSharedState
{
	std::atomic_bool State = false;
};

FStopToken::FStopToken(TWeakPtr<const FSharedState> InSharedState)
	: SharedStateWeak(MoveTemp(InSharedState))
{
}

bool FStopToken::IsStopRequested() const
{
	if (TSharedPtr<const FSharedState> SharedState = SharedStateWeak.Pin())
	{
		return SharedState->State;
	}

	return true;
}

FStopRequester::FStopRequester() :
	SharedState(MakeShared<FSharedState>())
{
}

void FStopRequester::RequestStop()
{
	check(SharedState.IsValid());
	SharedState->State = true;
}

bool FStopRequester::IsStopRequested() const
{
	check(SharedState.IsValid());
	return SharedState->State;
}

FStopToken FStopRequester::CreateToken() const
{
	check(SharedState.IsValid());
	SharedState->State = false;
	return FStopToken(SharedState);
}

}