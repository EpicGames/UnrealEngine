// Copyright Epic Games, Inc. All Rights Reserved

#include "Async/Task.h"

#include "Async/AsyncWork.h"

#include <atomic>

namespace UE::CaptureManager
{

class FCancelableAsyncTask::FAsyncTaskInternal : public FNonAbandonableTask
{
public:
	FAsyncTaskInternal(FStopToken InStopToken, FTaskFunction&& InTaskFunction) :
		StopToken(MoveTemp(InStopToken)), TaskFunction(MoveTemp(InTaskFunction))
	{
		check(TaskFunction != nullptr);
	}

	void DoWork()
	{
		TaskFunction(StopToken);
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FCancelableAsyncTask_FAsyncTaskInternal, STATGROUP_ThreadPoolAsyncTasks);
	}

private:
	FStopToken StopToken;
	FTaskFunction TaskFunction;
};

FCancelableAsyncTask::FCancelableAsyncTask(FTaskFunction InTaskFunction) :
	AsyncTask(MakeUnique<FAsyncTask<FAsyncTaskInternal>>(StopRequester.CreateToken(), MoveTemp(InTaskFunction)))
{
}

FCancelableAsyncTask::~FCancelableAsyncTask()
{
	Cancel();
	AsyncTask->EnsureCompletion();
}

bool FCancelableAsyncTask::IsDone()
{
	return AsyncTask->IsDone();
}

void FCancelableAsyncTask::StartSync()
{
	AsyncTask->StartSynchronousTask();
}

void FCancelableAsyncTask::StartAsync()
{
	AsyncTask->StartBackgroundTask();
}

void FCancelableAsyncTask::Cancel()
{
	StopRequester.RequestStop();
}

}
