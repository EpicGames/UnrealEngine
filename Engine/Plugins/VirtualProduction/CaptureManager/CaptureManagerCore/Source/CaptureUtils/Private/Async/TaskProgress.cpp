// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/TaskProgress.h"

#include "Math/UnrealMathUtility.h"

namespace UE::CaptureManager
{

FTaskProgress::FTask::FTask()
	: TaskProgress(nullptr)
{
}

FTaskProgress::FTask::FTask(FTask&& InTask)
{
	*this = MoveTemp(InTask);
}

FTaskProgress::FTask& FTaskProgress::FTask::operator=(FTask&& InTask)
{
	if (this == &InTask)
	{
		return *this;
	}

	TaskProgress = MoveTemp(InTask.TaskProgress);
	Id = InTask.Id;

	InTask.TaskProgress = nullptr;
	InTask.Id = INDEX_NONE;

	return *this;
}

FTaskProgress::FTask::FTask(const FTask& InTask)
{
	*this = InTask;
}

FTaskProgress::FTask& FTaskProgress::FTask::operator=(const FTask& InTask)
{
	if (this == &InTask)
	{
		return *this;
	}

	TaskProgress = InTask.TaskProgress;
	Id = InTask.Id;

	return *this;
}

FTaskProgress::FTask::FTask(TWeakPtr<FTaskProgress> InTaskProgress, int32 InId)
	: TaskProgress(MoveTemp(InTaskProgress))
	, Id(InId)
{
}

void FTaskProgress::FTask::Update(double InProgress)
{
	if (TSharedPtr<FTaskProgress> TaskProgressShared = TaskProgress.Pin())
	{
		TaskProgressShared->Update(Id, InProgress);
		TaskProgressShared->Report();
	}
}

FTaskProgress::FTaskProgress(uint32 InAmountOfWork, FProgressReporter InReporter)
	: Reporter(MoveTemp(InReporter))
	, CurrentTask(-1)
{
	checkf(InAmountOfWork != 0, TEXT("Number of tasks must NOT be 0"));
	CurrentProgressValues.Lock()->AddZeroed(InAmountOfWork);
}

void FTaskProgress::SetReportThreshold(double InReportThresholdPercent)
{
	ReportThreshold = InReportThresholdPercent / 100.0;
}

FTaskProgress::FTask FTaskProgress::StartTask()
{
	int32 NewTask = ++CurrentTask;
	checkf(CurrentProgressValues.Lock()->IsValidIndex(NewTask), TEXT("Unexpected start task"));

	return FTaskProgress::FTask(AsWeak(), NewTask);
}

double FTaskProgress::GetTotalProgress() const
{
	TMonitor<TArray<double>>::FHelper ProgressValues = CurrentProgressValues.Lock();

	double TotalProgress = 0.0;

	for (double TaskProgress : *ProgressValues)
	{
		TotalProgress += TaskProgress;
	}

	return TotalProgress / ProgressValues->Num();
}

void FTaskProgress::Update(int32 InTaskToUpdate, double InProgress)
{
	TMonitor<TArray<double>>::FHelper ProgressValues = CurrentProgressValues.Lock();

	checkf(InTaskToUpdate < ProgressValues->Num(), TEXT("Current task exceeds the number of expected tasks"));

	InProgress = FMath::Clamp(InProgress, 0.0f, 1.0f);

	(*ProgressValues)[InTaskToUpdate] = InProgress;
}

void FTaskProgress::Report()
{
	float TotalProgress = GetTotalProgress();
	bool bShouldReport = false;

	if (!FMath::IsNearlyZero(ReportThreshold))
	{	
		if (FMath::IsNearlyEqual(TotalProgress, 1.0))
		{
			bShouldReport = true;
		} 
		else if (TotalProgress > LastReportedValue && 
			(TotalProgress - LastReportedValue) > ReportThreshold)
		{
			bShouldReport = true;
		}
	}
	else
	{
		bShouldReport = true;
	}

	if (bShouldReport)
	{
		Reporter.ExecuteIfBound(TotalProgress);
		LastReportedValue = TotalProgress;
	}
}

}