// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/TaskWaiter.h"

#include "HAL/PlatformProcess.h"

namespace UE::CaptureManager
{

FTaskWaiter::FTaskWaiter() : TaskCounter(0)
{

}

FTaskWaiter::~FTaskWaiter()
{
	check(TaskCounter.load() <= CanCreateTaskFlag);
}

bool FTaskWaiter::CreateTask()
{
	uint32 OldCounter = TaskCounter.fetch_add(1);

	if ((OldCounter & CanCreateTaskFlag) > 0)
	{
		TaskCounter.fetch_sub(1);
		return false;
	}

	return true;
}

void FTaskWaiter::FinishTask()
{
	TaskCounter.fetch_sub(1);
}

void FTaskWaiter::WaitForAll()
{
	TaskCounter.fetch_add(CanCreateTaskFlag);

	while (TaskCounter > CanCreateTaskFlag)
	{
		FPlatformProcess::Sleep(0.1f);
	}
}

}