// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureUtilsModule.h"

void FCaptureUtilsModule::StartupModule()
{
	TimerManager = MakeShared<UE::CaptureManager::FCaptureTimerManager>();
}

void FCaptureUtilsModule::ShutdownModule()
{
	TimerManager = nullptr;
}

TSharedRef<UE::CaptureManager::FCaptureTimerManager> FCaptureUtilsModule::GetTimerManager()
{
	return TimerManager.ToSharedRef();
}

IMPLEMENT_MODULE(FCaptureUtilsModule, CaptureUtils)
