// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Error.h"

namespace UE::CaptureManager
{

FCaptureProtocolError::FCaptureProtocolError()
    : Message("")
    , Code(0)
{
}

FCaptureProtocolError::FCaptureProtocolError(FString InMessage, int32 InCode)
    : Message(MoveTemp(InMessage))
    , Code(InCode)
{
}

const FString& FCaptureProtocolError::GetMessage() const
{
    return Message;
}

int32 FCaptureProtocolError::GetCode() const
{
    return Code;
}

}