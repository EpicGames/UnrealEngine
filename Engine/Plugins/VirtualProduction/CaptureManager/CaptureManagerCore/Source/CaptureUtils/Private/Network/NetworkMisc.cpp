// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/NetworkMisc.h"
#include "SocketSubsystem.h"
#include "Misc/OutputDeviceRedirector.h"

namespace UE::CaptureManager
{

TOptional<FString> GetLocalIpAddress()
{
	TOptional<FString> LocalHostIPAddress;

	if (ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM))
	{
		bool bCanBindAll;
		TSharedPtr<FInternetAddr> LocalHostAddr = SocketSubsystem->GetLocalHostAddr(*GLog, bCanBindAll);

		if (LocalHostAddr->IsValid())
		{
			constexpr bool bAppendPort = false;
			LocalHostIPAddress = LocalHostAddr->ToString(bAppendPort);
		}
	}

	return LocalHostIPAddress;
}

TOptional<FString> GetLocalHostName()
{
	if (ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM))
	{
		FString HostName;
		bool bSuccess = SocketSubsystem->GetHostName(HostName);

		if (bSuccess)
		{
			return HostName;
		}
	}

	return {};
}

CAPTUREUTILS_API FString GetLocalHostNameChecked()
{
	TOptional<FString> LocalHostNameOpt = GetLocalHostName();

	if (LocalHostNameOpt.IsSet())
	{
		return *LocalHostNameOpt;
	}

	check(false);
	return TEXT("Unknown host");
}

}