// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/TcpClient.h"

namespace UE::CaptureManager
{

FTcpClient::FTcpClient()
	: bRunning(false)
{
}

FTcpClient::~FTcpClient()
{
	Stop();
}

TProtocolResult<void> FTcpClient::Init()
{
	if (bRunning)
	{
		return FCaptureProtocolError(TEXT("Can't initialize the client while running."));
	}

	// Prepare socket
	FSocket* RawSocket = FTcpSocketBuilder(TEXT("CPS TCP Socket"))
		.AsNonBlocking()
		.AsReusable()
		.WithReceiveBufferSize(BufferSize)
		.WithSendBufferSize(BufferSize)
		.Build();

	if (!RawSocket)
	{
		return FCaptureProtocolError(TEXT("Failed to create a client socket"));
	}

	// Wrapping the socket in UniquePtr
	TcpSocket = FSocketPtr(RawSocket, FSocketDeleter(ISocketSubsystem::Get()));

	return ResultOk;
}

// Blocking function until connection is established
TProtocolResult<void> FTcpClient::Start(const FString& InServerAddress)
{
	if (bRunning)
	{
		return FCaptureProtocolError(TEXT("The client is already started"));
	}

	FIPv4Endpoint Endpoint;
	FIPv4Endpoint::FromHostAndPort(InServerAddress, Endpoint);

	bool IsConnected = TcpSocket->Connect(*Endpoint.ToInternetAddr());
	
	// Wait 500 milliseconds before reporting an error
	FDateTime Time = FDateTime::UtcNow();
	while (IsConnected && TcpSocket->GetConnectionState() != SCS_Connected)
	{
		FPlatformProcess::Sleep(0.20f);
		if ((FDateTime::UtcNow() - Time).GetTotalMilliseconds() > 1000)
		{
			IsConnected = false;
		}
	}

	if (!IsConnected)
	{
		return FCaptureProtocolError(TEXT("Failed to connect the client"));
	}

	bRunning = true;

	return ResultOk;
}

TProtocolResult<void> FTcpClient::Stop()
{
	if (!bRunning)
	{
		return FCaptureProtocolError(TEXT("The client is already stopped"));
	}

	if (TcpSocket.IsValid())
	{
		TcpSocket->Shutdown(ESocketShutdownMode::ReadWrite);
		TcpSocket->Close();
	}

	bRunning = false;

	return ResultOk;
}

bool FTcpClient::IsRunning() const
{
	return bRunning;
}

TProtocolResult<void> FTcpClient::SendMessage(const TArray<uint8>& InPayload)
{
	if (!TcpSocket)
	{
		return FCaptureProtocolError(TEXT("Invalid TCP socket"));
	}

	int32 TotalSent = 0;
	while (TotalSent != InPayload.Num())
	{
		int32 Sent = 0;
		bool Result = TcpSocket->Send(InPayload.GetData() + TotalSent, InPayload.Num() - TotalSent, Sent);
		if (!Result)
		{
			ESocketErrors Error = ISocketSubsystem::Get()->GetLastErrorCode();

			FString SocketError = ISocketSubsystem::Get()->GetSocketError(Error);

			if (Error == ESocketErrors::SE_EWOULDBLOCK ||
				Error == ESocketErrors::SE_TRY_AGAIN)
			{
				continue;
			}

			return FCaptureProtocolError(FString::Printf(TEXT("Failed to send the data: %s"), *SocketError), static_cast<int32>(Error));
		}

		TotalSent += Sent;
	}

	return ResultOk;
}

TProtocolResult<TArray<uint8>> FTcpClient::ReceiveMessage(const uint64 InSize, const uint32 InWaitTimeoutMs)
{
	if (!TcpSocket)
	{
		return FCaptureProtocolError(TEXT("Invalid TCP socket"));
	}

	TArray<uint8> ReadData;
	ReadData.AddUninitialized(InSize);

	uint32 LeftToRead = InSize;

	while (LeftToRead != 0)
	{
		uint32 PendingSize = 0;

		if (!TcpSocket->Wait(ESocketWaitConditions::WaitForRead, FTimespan::FromMilliseconds(InWaitTimeoutMs)))
		{
			return FCaptureProtocolError(TEXT("Timeout has expired"), TimeoutError);
		}

		if (!TcpSocket->HasPendingData(PendingSize))
		{
			return FCaptureProtocolError(TEXT("No pending data on socket"), NoPendingDataError);
		}

		int32 SizeToRead = FMath::Min(LeftToRead, PendingSize);

		const uint32 Offset = InSize - LeftToRead;
		int32 ReadSize = 0;
		if (!TcpSocket->Recv(ReadData.GetData() + Offset, SizeToRead, ReadSize))
		{
			return FCaptureProtocolError(TEXT("Failed to read the data from the TCP socket"), ReadError);
		}

		if (PendingSize == 0 && ReadSize == 0)
		{
			return FCaptureProtocolError(TEXT("Host has been disconnected"), DisconnectedError);
		}

		LeftToRead -= ReadSize;
	}

	return ReadData;
}

FTcpClientReader::FTcpClientReader(FTcpClient& InClient)
	: Client(InClient)
{
}

TProtocolResult<TArray<uint8>> FTcpClientReader::ReceiveMessage(const uint64 InSize, const uint32 InWaitTimeoutMs)
{
	return Client.ReceiveMessage(InSize, InWaitTimeoutMs);
}


FTcpClientWriter::FTcpClientWriter(FTcpClient& InClient)
	: Client(InClient)
{
}

TProtocolResult<void> FTcpClientWriter::SendMessage(const TArray<uint8>& InPayload)
{
	return Client.SendMessage(InPayload);
}

}