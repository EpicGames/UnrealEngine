// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/TcpServer.h"

namespace UE::CaptureManager
{

FTcpClientHandler::FTcpClientHandler(FSocketPtr InSocket, FString InEndpoint)
	: Socket(MoveTemp(InSocket))
	, Endpoint(MoveTemp(InEndpoint))
{
}

FTcpClientHandler::~FTcpClientHandler()
{
	if (Socket)
	{
		Socket->Shutdown(ESocketShutdownMode::ReadWrite);
		Socket->Close();
		Socket = nullptr;
	}
}

TProtocolResult<void> FTcpClientHandler::SendMessage(const TArray<uint8>& InData)
{
	if (!Socket)
	{
		return FCaptureProtocolError(TEXT("Invalid TCP socket"));
	}

	int32 TotalSent = 0;
	while (TotalSent != InData.Num())
	{
		int32 Sent = 0;
		bool Result = Socket->Send(InData.GetData() + TotalSent, InData.Num() - TotalSent, Sent);
		if (!Result)
		{
			return FCaptureProtocolError(TEXT("Failed to send the data"));
		}

		TotalSent += Sent;
	}

	return ResultOk;
}

TProtocolResult<TArray<uint8>> FTcpClientHandler::ReceiveMessage(const uint64 InSize, const uint32 InWaitTimeoutMs)
{
	if (!Socket)
	{
		return FCaptureProtocolError(TEXT("Invalid TCP socket"));
	}

	TArray<uint8> ReadData;
	ReadData.AddUninitialized(InSize);

	uint32 LeftToRead = InSize;

	while (LeftToRead != 0)
	{
		uint32 PendingSize = 0;
		int32 ReadSize = 0;

		if (!Socket->Wait(ESocketWaitConditions::WaitForRead, FTimespan::FromMilliseconds(InWaitTimeoutMs)))
		{
			return FCaptureProtocolError(TEXT("Timeout has expired"), TimeoutError);
		}

		if (!Socket->HasPendingData(PendingSize))
		{
			return FCaptureProtocolError(TEXT("Host has been disconnected"), DisconnectedError);
		}

		int32 Size = FMath::Min(LeftToRead, PendingSize);

		const uint32 Offset = InSize - LeftToRead;
		if (!Socket->Recv(ReadData.GetData() + Offset, Size, ReadSize))
		{
			return FCaptureProtocolError(TEXT("Failed to read the data from the TCP socket"));
		}

		if (PendingSize == 0 && ReadSize == 0)
		{
			return FCaptureProtocolError(TEXT("Host has been disconnected"), DisconnectedError);
		}

		LeftToRead -= ReadSize;
	}

	return ReadData;
}

TProtocolResult<uint32> FTcpClientHandler::HasPendingData() const
{
	if (!Socket)
	{
		return FCaptureProtocolError(TEXT("Invalid TCP socket"));
	}

	uint32 PendingSize = 0;
	if (!Socket->HasPendingData(PendingSize))
	{
		return FCaptureProtocolError(TEXT("Host has been disconnected"), DisconnectedError);
	}

	return PendingSize;
}

const FString& FTcpClientHandler::GetEndpoint() const
{
	return Endpoint;
}

bool FTcpClientHandler::operator==(const FTcpClientHandler& InOther)
{
	return InOther.Endpoint == Endpoint;
}


FTcpServer::FTcpServer(const uint32 InMaxNumberOfClients)
	: Listener(nullptr)
	, MaxNumberOfClients(InMaxNumberOfClients)
	, bRunning(false)
{
	Clients.Reserve(MaxNumberOfClients);
}

TProtocolResult<uint16> FTcpServer::Start(const uint16 InListeningPort)
{
	if (bRunning)
	{
		return FCaptureProtocolError(TEXT("The server is already started"));
	}

	FIPv4Endpoint ListenEndpoint(FIPv4Address::Any, InListeningPort);

	FSocket* RawPointer = FTcpSocketBuilder(TEXT("FTcpListener server"))
		.AsReusable(true)
		.BoundToEndpoint(ListenEndpoint)
		.Listening(8) // Max number of connections in queue before refusing them
		.WithSendBufferSize(2 * 1024 * 1024).Build();

	if (!RawPointer)
	{
		return FCaptureProtocolError(TEXT("Failed to create a server socket"));
	}

	// Wrapping the socket in UniquePtr
	Socket = FSocketPtr(RawPointer, FSocketDeleter(ISocketSubsystem::Get()));

	Listener.Reset(new FTcpListener(*Socket.Get(), FTimespan::FromMilliseconds(ThreadWaitTime)));

	if (!Listener)
	{
		return FCaptureProtocolError(TEXT("The server is failed to create a listener"));
	}

	Listener->OnConnectionAccepted().BindLambda([this](FSocket* InSocket, const FIPv4Endpoint& InEndpoint) mutable
	{
		if (Clients.Num() >= (int32) MaxNumberOfClients)
		{
			return false;
		}

		FSocketPtr ClientSocket(InSocket, FSocketDeleter(ISocketSubsystem::Get()));

		int32 NewSize = 0;
		ClientSocket->SetReceiveBufferSize(FTcpClientHandler::MaxBufferSize, NewSize);

		TSharedPtr<FTcpClientHandler> NewClient = MakeShared<FTcpClientHandler>(MoveTemp(ClientSocket), InEndpoint.ToString());

		TWeakPtr<FTcpClientHandler> ClientObserver(NewClient);
		OnConnectionHandler.ExecuteIfBound(MoveTemp(ClientObserver), true);

		FRWScopeLock Lock(Mutex, FRWScopeLockType::SLT_Write);
		Clients.Add(InEndpoint.ToString(), MoveTemp(NewClient));

		return true;
	});

	bRunning = true;

	return GetPort();
}

TProtocolResult<void> FTcpServer::Stop()
{
	if (!bRunning)
	{
		return FCaptureProtocolError(TEXT("The server is already stopped"));
	}

	Listener->Stop();
	Listener = nullptr;

	Socket->Close();
	Socket = nullptr;

	bRunning = false;

	FRWScopeLock Lock(Mutex, FRWScopeLockType::SLT_Write);

	Clients.Empty();

	return ResultOk;
}

bool FTcpServer::IsRunning() const
{
	return bRunning;
}

TProtocolResult<void> FTcpServer::SendMessage(const TArray<uint8>& InMessage, const FString& InEndpoint)
{
	FRWScopeLock Lock(Mutex, FRWScopeLockType::SLT_ReadOnly);

	TSharedPtr<FTcpClientHandler>* Iterator = Clients.Find(InEndpoint);
	verifyf(Iterator != nullptr, TEXT("Client doesn't exist"));

	const TSharedPtr<FTcpClientHandler>& Client = *Iterator;

	return Client->SendMessage(InMessage);
}

void FTcpServer::DisconnectClient(const FString& InEndpoint)
{
	FRWScopeLock Lock(Mutex, FRWScopeLockType::SLT_Write);

	TSharedPtr<FTcpClientHandler>* Iterator = Clients.Find(InEndpoint);

	if (Iterator)
	{
		OnConnectionHandler.ExecuteIfBound(*Iterator, false);
		Clients.Remove(InEndpoint);
	}
}

void FTcpServer::SetConnectionHandler(FConnectionHandler InOnConnectionHandler)
{
	OnConnectionHandler = MoveTemp(InOnConnectionHandler);
}

int32 FTcpServer::GetPort() const
{
	return Socket->GetPortNo();
}

FTcpConnectionReader::FTcpConnectionReader(FTcpClientHandler& InClient)
	: Client(InClient)
{
}

TProtocolResult<TArray<uint8>> FTcpConnectionReader::ReceiveMessage(const uint64 InSize, const uint32 InWaitTimeoutMs)
{
	return Client.ReceiveMessage(InSize, InWaitTimeoutMs);
}


FTcpConnectionWriter::FTcpConnectionWriter(FTcpClientHandler& InClient)
	: Client(InClient)
{
}

TProtocolResult<void> FTcpConnectionWriter::SendMessage(const TArray<uint8>& InPayload)
{
	return Client.SendMessage(InPayload);
}

}