// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProcessRunner/ProcessRunner.h"

#include "Async/StopToken.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"

DEFINE_LOG_CATEGORY_STATIC(LogCaptureManagerProcessRunner, Log, All);

namespace UE::CaptureManager
{

FProcessRunnerResult FProcessRunner::Run(
	const FString& InProcessPath, 
	const FString& InProcessArguments, 
	const FStopToken* InStopToken, 
	const TOptional<int32> InTimeoutSeconds)
{
	UE_LOGF(LogCaptureManagerProcessRunner, Log, "Executing \"%ls\" with arguments \"%ls\"", *FPaths::GetCleanFilename(InProcessPath), *InProcessArguments);

	void* StdOutReadPipe = nullptr;
	void* StdOutWritePipe = nullptr;

	void* StdErrReadPipe = nullptr;
	void* StdErrWritePipe = nullptr;

	void* StdInReadPipe = nullptr;
	void* StdInWritePipe = nullptr;

	ON_SCOPE_EXIT
	{
		FPlatformProcess::ClosePipe(StdOutReadPipe, StdOutWritePipe);
		FPlatformProcess::ClosePipe(StdErrReadPipe, StdErrWritePipe);
		FPlatformProcess::ClosePipe(StdInReadPipe, StdInWritePipe);
	};

	/* Create the read/write pipes for process I/O
	 * - StdOut: Parent process owns the read pipe, child owns write.
	 * - StdErr: Same as StdOut
	 * - StdIn: Parent process owns the write pipe, child owns read.
	 * For StdIn we must specify 'true' for the third argument (bWritePipeLocal) in order
	 * to swap ownership so that the child can inherit the read pipe.
	*/
	if (!FPlatformProcess::CreatePipe(StdOutReadPipe, StdOutWritePipe, false)
		|| !FPlatformProcess::CreatePipe(StdErrReadPipe, StdErrWritePipe, false)
		|| !FPlatformProcess::CreatePipe(StdInReadPipe, StdInWritePipe, true))
	{
		return MakeError(EProcessRunnerError::PipeCreationFailed);
	}

	constexpr bool bLaunchDetached = false;
	constexpr bool bLaunchHidden = true;
	constexpr bool bLaunchReallyHidden = true;
	FProcHandle ProcHandle = FPlatformProcess::CreateProc(
		*InProcessPath, 
		*InProcessArguments, 
		bLaunchDetached, 
		bLaunchHidden, 
		bLaunchReallyHidden, 
		nullptr, 
		0, 
		nullptr, 
		StdOutWritePipe, 
		StdInReadPipe, 
		StdErrWritePipe);

	bool bTerminateOnScopeExit = false;

	ON_SCOPE_EXIT
	{
		if (bTerminateOnScopeExit)
		{
			FPlatformProcess::TerminateProc(ProcHandle);
		}
		FPlatformProcess::CloseProc(ProcHandle);
	};

	if (!ProcHandle.IsValid())
	{
		UE_LOGF(LogCaptureManagerProcessRunner, Error, "Failed to launch \"%ls\" with arguments \"%ls\"", *FPaths::GetCleanFilename(InProcessPath), *InProcessArguments);
		return MakeError(EProcessRunnerError::LaunchFailed);
	}

	const FDateTime StartDateTime = FDateTime::Now();

	TArray<uint8> StandardOutput;
	TArray<uint8> StandardError;
	while (FPlatformProcess::IsProcRunning(ProcHandle))
	{
		TArray<uint8> StandardOutputFragment = ReadPipe(StdOutReadPipe);
		TArray<uint8> StandardErrorFragment = ReadPipe(StdErrReadPipe);

		StandardOutput.Append(MoveTemp(StandardOutputFragment));
		StandardError.Append(MoveTemp(StandardErrorFragment));

		if (StandardOutputFragment.IsEmpty() && StandardErrorFragment.IsEmpty())
		{
			FPlatformProcess::Sleep(0.01f);
		}

		if (InTimeoutSeconds.IsSet())
		{
			int32 TimeoutSeconds = InTimeoutSeconds.GetValue();
			if ((FDateTime::Now() - StartDateTime).GetSeconds() > TimeoutSeconds)
			{
				bTerminateOnScopeExit = true;
				UE_LOGF(LogCaptureManagerProcessRunner, Error, "\"%ls\" timed out after %d seconds", *FPaths::GetCleanFilename(InProcessPath), TimeoutSeconds);
				return MakeError(EProcessRunnerError::Timeout);
			}
		}

		if (InStopToken && InStopToken->IsStopRequested())
		{
			bTerminateOnScopeExit = true;
			UE_LOGF(LogCaptureManagerProcessRunner, Warning, "\"%ls\" cancelled by the user", *FPaths::GetCleanFilename(InProcessPath));
			return MakeError(EProcessRunnerError::Cancelled);
		}
	}

	// Drain pipes of all remaining data
	StandardOutput.Append(DrainPipe(StdOutReadPipe));
	StandardError.Append(DrainPipe(StdErrReadPipe));

	int32 ReturnCode = 0;
	if (!FPlatformProcess::GetProcReturnCode(ProcHandle, &ReturnCode))
	{
		UE_LOGF(LogCaptureManagerProcessRunner, Error, "Failed to get return code when running \"%ls\"", *FPaths::GetCleanFilename(InProcessPath))
		return MakeError(EProcessRunnerError::FailedToGetReturnCode);
	}

	if (ReturnCode != 0)
	{
		UE_LOGF(LogCaptureManagerProcessRunner, Error, "\"%ls\" exited with error code %d", *FPaths::GetCleanFilename(InProcessPath), ReturnCode);

		if (!StandardOutput.IsEmpty())
		{
			FString StandardOutputString = FString::ConstructFromPtrSize(reinterpret_cast<const UTF8CHAR*>(StandardOutput.GetData()), StandardOutput.Num());
			UE_LOGF(LogCaptureManagerProcessRunner, Log, "Standard output:\n>>>>>>\n%ls<<<<<<", *StandardOutputString);
		}

		if (!StandardError.IsEmpty())
		{
			FString StandardErrorString = FString::ConstructFromPtrSize(reinterpret_cast<const UTF8CHAR*>(StandardError.GetData()), StandardError.Num());
			UE_LOGF(LogCaptureManagerProcessRunner, Log, "Error output:\n>>>>>>\n%ls<<<<<<\n", *StandardErrorString);
		}

		return MakeError(EProcessRunnerError::ExitedWithError);
	}

	return MakeValue(StandardOutput);
}

TArray<uint8> FProcessRunner::ReadPipe(void* InPipe)
{
	TArray<uint8> Output;
	bool Read = FPlatformProcess::ReadPipeToArray(InPipe, Output);
	if (!Read)
	{
		Output.Empty();
	}
	return Output;
}

TArray<uint8> FProcessRunner::DrainPipe(void* InPipe)
{
	TArray<uint8> Drained;
	TArray<uint8> Fragment;
	do 
	{
		Fragment = ReadPipe(InPipe);
		Drained.Append(Fragment);
	} while (!Fragment.IsEmpty());
	return Drained;
}

}
