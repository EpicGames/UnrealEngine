// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

namespace UE::CaptureManager
{
// Replaces any invalid package path characters 
CAPTUREUTILS_API void SanitizePackagePath(FString& OutPath, FString::ElementType InReplaceWith = TEXT('_'));

// Replaces any invalid object name characters 
CAPTUREUTILS_API void SanitizeAssetName(FString& OutPath, FString::ElementType InReplaceWith = TEXT('_'));

}
