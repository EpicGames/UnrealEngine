// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Delegates/Delegate.h"

#include <atomic>

#define UE_API CAPTUREUTILS_API

namespace UE::CaptureManager
{

template<typename ...>
struct ArgsBundle
{
};

template<typename T>
struct FFunctionTraits : public FFunctionTraits<decltype(&T::operator())>
{
};

template<typename Return, typename Object, typename ... Args>
struct FFunctionTraits<Return(Object::*)(Args ...)>
{
	using ReturnType = Return;
	using FnArgs = ArgsBundle<Args ...>;
	using FnType = Return(Args ...);
};

template<typename Return, typename Object, typename ... Args>
struct FFunctionTraits<Return(Object::*)(Args ...) const>
{
	using ReturnType = Return;
	using FnArgs = ArgsBundle<Args ...>;
	using FnType = Return(Args ...);
};

template<typename Return, typename ... Args>
struct FFunctionTraits<Return(*)(Args ...)>
{
	using ReturnType = Return;
	using FnArgs = ArgsBundle<Args ...>;
	using FnType = Return(Args ...);
};

class FCallbackSynchronizer : public TSharedFromThis<FCallbackSynchronizer>
{
public:
	DECLARE_DELEGATE(FAfterAllDelegate);

	UE_API FCallbackSynchronizer();
	UE_API ~FCallbackSynchronizer();

	template<typename FCallback>
	auto CreateCallback(FCallback&& InCallback)
	{
		using FIdentityFnArgs = typename FFunctionTraits<FCallback>::FnArgs;
		return FMakeCallbackInternal<FIdentityFnArgs>::Get(AsShared(), MoveTemp(InCallback));
	}

	UE_API void AfterAll(FAfterAllDelegate InAfterAllDelegate, bool bExecuteIfCounterZero = true);

	static UE_API TSharedPtr<FCallbackSynchronizer> Create();

private:

	template<typename Args>
	struct FMakeCallbackInternal;

	template<typename ... Args>
	struct FMakeCallbackInternal<ArgsBundle<Args ...> >
	{
		template<typename LT>
		static auto Get(TSharedPtr<FCallbackSynchronizer> InCallbackSynchronizer, LT&& InUserFn)
		{
			InCallbackSynchronizer->Counter++;

			return [CallbackSynchronizer = MoveTemp(InCallbackSynchronizer),
				UserFn(MoveTemp(InUserFn))](Args&& ... InArgs) mutable
			{
				UserFn(Forward<Args>(InArgs) ...);

				CallbackSynchronizer->Decrease();
			};
		}
	};

	UE_API void Decrease();

	FAfterAllDelegate AfterAllDelegate;

	std::atomic_int Counter;
};

}

#undef UE_API
