// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/RunnableThread.h"
#include "HAL/Runnable.h"
#include "Containers/Ticker.h"

#include "TimerManager.h"

#define UE_API CAPTUREUTILS_API

namespace UE::CaptureManager
{

class FCaptureTimerManager
{
public:

	using FTimerHandle = FTSTicker::FDelegateHandle;

	UE_API FCaptureTimerManager();
	UE_API FCaptureTimerManager(const float Resolution);

	UE_API ~FCaptureTimerManager();

	UE_API FTimerHandle AddTimer(FTimerDelegate InDelegate, float InRate, bool InbLoop = false, float InFirstDelay = 0.f);
	UE_API void RemoveTimer(FTimerHandle Handle);

private:
	static constexpr float IdealTimeResolution = 0.1f;

	class FTimerManagerRunnable final : public FRunnable
	{
	public:

		FTimerManagerRunnable(FCaptureTimerManager* InOwner, const float InResolution);

		virtual uint32 Run() override;
		virtual void Stop() override;

	private:

		FCaptureTimerManager* Owner;
		std::atomic_bool bShouldRun = true;
		const float IdealTimeResolution;
	};

	FTSTicker Ticker;
	TUniquePtr<FRunnableThread> Thread;
	TUniquePtr<FTimerManagerRunnable> Runnable;
};

}

#undef UE_API
