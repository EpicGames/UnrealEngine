// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/ManagedDelegate.h"

#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"
#include "Containers/UnrealString.h"
#include "Misc/DateTime.h"
#include "Misc/Timespan.h"

#include <atomic>

#define UE_API CAPTUREUTILS_API

namespace UE::CaptureManager
{

// Base class for all event sublasses
class FCaptureEvent
{
public:
	UE_API const FString& GetName() const;
	UE_API virtual ~FCaptureEvent();

protected:
	UE_API FCaptureEvent(FString InName);


private:
	FString Name;
};

// Macro for easy definition of events that don't carry any data
#define CAPTURE_DEFINE_EMPTY_EVENT(ClassName, EventName) \
	struct CAPTUREUTILS_API ClassName : public FCaptureEvent \
	{ \
		inline static const FString Name = TEXT(EventName); \
		ClassName() : FCaptureEvent(Name) \
		{ \
		} \
	};

// The SharedPtr points to a `const` event object because the shared event might end up on multiple
// threads in which case we'd have a thread safety issue without the event being `const`
using FCaptureEventHandler = TManagedDelegate<TSharedPtr<const FCaptureEvent>>;

// Interface for classes that wish to provide capture event subscription to their clients
class ICaptureEventSource
{
public:
	virtual TArray<FString> GetAvailableEvents() const = 0;
	virtual void SubscribeToEvent(const FString& InEventName, FCaptureEventHandler InHandler) = 0;
	virtual void UnsubscribeAll() = 0;
};

}

#undef UE_API
