// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Event.h"

#define UE_API CAPTUREUTILS_API

namespace UE::CaptureManager
{
namespace Private
{
	class FCaptureEventSourceBase : public ICaptureEventSource
	{
	public:
		UE_API virtual ~FCaptureEventSourceBase();

	public:
		UE_API virtual TArray<FString> GetAvailableEvents() const override;
		// NOTE: SubscribeToEvent() must not be called from an event handler that is being executed on the same thread
		UE_API virtual void SubscribeToEvent(const FString& InEventName, FCaptureEventHandler InHandler) override;
		UE_API virtual void UnsubscribeAll() override;

	protected:
		UE_API void RegisterEvent(const FString& InEventName);
		UE_API void PublishEventInternal(TSharedPtr<const FCaptureEvent> InEvent) const;

	private:
		using FHandlers = TTSMulticastDelegate<void(TSharedPtr<const FCaptureEvent>)>;

		mutable FRWLock HandlersLock;
		TMap<FString, FHandlers> EventToHandlersMap;

	};
}

// Class to be publicly inherited to get basic event source functionality. All functions are thread-safe.
class FCaptureEventSource : public Private::FCaptureEventSourceBase
{
protected:
	template<typename EventType, typename ... Args>
	void PublishEvent(Args&&... InArgs) const
	{
		PublishEventInternal(MakeShared<const EventType>(Forward<Args>(InArgs)...));
	}

	void PublishEventPtr(TSharedPtr<const FCaptureEvent> InEvent) const
	{
		PublishEventInternal(MoveTemp(InEvent));
	}

private:
	using Private::FCaptureEventSourceBase::PublishEventInternal;
};

// Class to be publicly inherited to get event source functionality where rate of events published is limited. All functions are thread-safe.
class FCaptureEventSourceWithLimiter : public Private::FCaptureEventSourceBase
{
public:
	FCaptureEventSourceWithLimiter(int32 InThresholdMillis) : ThresholdMillis(InThresholdMillis), LastPublish(0)
	{
	}

protected:
	// When PublishIfThresholdReached() is called the publishing will mostly only occur if the time since last
	// publish is greater than the threshold (although this isn't guaranteed and multiple events can still be
	// published in rare cases). The unpublished events are simply dropped (i.e. not buffered so the client has
	// to make sure that won't cause problems to the subscribers.
	// The client can also force publish an event which will do the publishing no matter what the current time is and
	// reset the time measurement. This is handy for publishing events whose dropping will break continuity for the
	// client or for publishing the last event in a line of optional events to trigger the final update.
	template<typename EventType, typename ... EventArgs>
	bool PublishIfThresholdReached(bool bInForcePublish, EventArgs&& ... InEventArgs)
	{
		int64 LastPublishLocal = LastPublish.load();
		int64 Now = FDateTime::Now().GetTicks();

		if (bInForcePublish || IsThresholdPassed(LastPublishLocal, Now))
		{
			// Only update if nobody updated after we loaded the value. This way we don't overwrite newer time
			// if another thread had a chance to do that before this one.
			LastPublish.compare_exchange_strong(LastPublishLocal, Now);
			PublishEventInternal(MakeShared<const EventType>(Forward<EventArgs>(InEventArgs)...));
			return true;
		}
		return false;
	}

	// Always publishes the event while completely ignoring the threshold mechanism. If you want to force publishing
	// of an event AND update the "last publish" timestamp, please use PublishIfThresholdReached(true, ...)
	void PublishEventIgnoreThresholdPtr(TSharedPtr<const FCaptureEvent> InEvent)
	{
		PublishEventInternal(MoveTemp(InEvent));
	}

	template<typename EventType, typename ... EventArgs>
	void PublishEventIgnoreThreshold(EventArgs&&... InEventArgs)
	{
		PublishEventPtr(MakeShared<const EventType>(Forward<EventArgs>(InEventArgs)...));
	}

private:
	using Private::FCaptureEventSourceBase::PublishEventInternal;

	bool IsThresholdPassed(const FDateTime& InLastPublish, const FDateTime& InNow) const
	{
		FTimespan TimespanElapsed = InNow - InLastPublish;
		return TimespanElapsed.GetTotalMilliseconds() >= ThresholdMillis;
	}

	const int32 ThresholdMillis;
	std::atomic<int64> LastPublish;
};

}

#undef UE_API
