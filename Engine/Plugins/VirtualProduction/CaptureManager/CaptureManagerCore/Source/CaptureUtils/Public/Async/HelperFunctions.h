// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"

namespace UE::CaptureManager
{

// Sends InFunction to be called on the GameThread and waits for the result
void CAPTUREUTILS_API CallOnGameThread(TFunction<void()> InFunction);

}