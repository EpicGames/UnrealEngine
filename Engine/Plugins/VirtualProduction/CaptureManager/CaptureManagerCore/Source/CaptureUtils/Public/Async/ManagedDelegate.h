// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Async/Async.h"

namespace UE::CaptureManager
{

enum class EDelegateExecutionThread
{
	GameThread = 0,
	InternalThread, // Thread on which the function invoking the delegate is being run on
	AnyThread
};

namespace Private
{

static ENamedThreads::Type GetThreadType(EDelegateExecutionThread InThread)
{
	switch (InThread)
	{
		case EDelegateExecutionThread::GameThread:
			return ENamedThreads::GameThread;
		case EDelegateExecutionThread::AnyThread:
			return ENamedThreads::AnyThread;
		case EDelegateExecutionThread::InternalThread:
		default:
			return ENamedThreads::UnusedAnchor;
	}
}

template<typename ... Args>
static void ExecuteDelegate(TDelegate<void(Args...)> InDelegate, EDelegateExecutionThread InThread, Args&&... InArgs)
{
	ENamedThreads::Type ThreadToExecuteOn = GetThreadType(InThread);

	if (ThreadToExecuteOn == ENamedThreads::UnusedAnchor || (ThreadToExecuteOn == ENamedThreads::GameThread && IsInGameThread()))
	{
		InDelegate.Execute(Forward<Args>(InArgs)...);

		return;
	}

	TDelegate<void()> Task = TDelegate<void()>::CreateLambda([InDelegate = (MoveTemp(InDelegate))](Args... InArgs)
	{
		InDelegate.Execute(Forward<Args>(InArgs)...);
	}, Forward<Args>(InArgs)...);

	AsyncTask(ThreadToExecuteOn, [InTask = MoveTemp(Task)]()
	{
		InTask.Execute();
	});
}

}

template<typename ... Args>
class TManagedDelegate
{
public:
	using Type = TDelegate<void(Args...)>;

	TManagedDelegate() = default;

	template<typename Func>
	TManagedDelegate(Func InFunc, EDelegateExecutionThread InThread = EDelegateExecutionThread::GameThread)
		: TManagedDelegate(Type::CreateLambda(MoveTemp(InFunc)), InThread)
	{
	}

	TManagedDelegate(Type InDelegate, EDelegateExecutionThread InThread = EDelegateExecutionThread::GameThread)
		: Delegate(MoveTemp(InDelegate))
		, ExecutionThread(InThread)
	{
	}

	void operator()(Args... InArgs)
	{
		if (Delegate.IsBound())
		{
			Private::ExecuteDelegate(Delegate, ExecutionThread, Forward<Args>(InArgs)...);
		}
	}

	void operator()(Args... InArgs) const
	{
		if (Delegate.IsBound())
		{
			Private::ExecuteDelegate(Delegate, ExecutionThread, Forward<Args>(InArgs)...);
		}
	}

private:

	Type Delegate;
	EDelegateExecutionThread ExecutionThread;
};

template<typename ... Args>
class TManagedMulticastDelegate
{
public:
	using DelegateType = TDelegate<void(Args...)>;

	TManagedMulticastDelegate() = default;

	template<typename Func>
	void Add(Func InFunc, EDelegateExecutionThread InThread = EDelegateExecutionThread::GameThread)
	{
		Add(DelegateType::CreateLambda(MoveTemp(InFunc)), InThread);
	}

	void Add(DelegateType InDelegate, EDelegateExecutionThread InThread = EDelegateExecutionThread::GameThread)
	{
		Delegate.AddLambda([InDelegate = MoveTemp(InDelegate), InThread](Args... InArgs) mutable
		{
			Private::ExecuteDelegate(InDelegate, InThread, Forward<Args>(InArgs)...);
		});
	}

	void operator()(Args... InArgs)
	{
		Delegate.Broadcast(Forward<Args>(InArgs)...);
	}

	void operator()(Args... InArgs) const
	{
		Delegate.Broadcast(Forward<Args>(InArgs)...);
	}

private:

	using Type = TMulticastDelegate<void(Args...)>;

	Type Delegate;
};

}