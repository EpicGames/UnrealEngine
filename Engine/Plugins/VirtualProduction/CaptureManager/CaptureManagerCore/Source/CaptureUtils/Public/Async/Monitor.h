// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ScopeLock.h"

namespace UE::CaptureManager
{

template<typename T>
class TMonitor
{
public:

	UE_NONCOPYABLE(TMonitor)

	template <typename... Args>
	TMonitor(Args&&... InArgs)
		: Object(Forward<Args>(InArgs)...)
	{
	}

	TMonitor(T InObject)
		: Object(MoveTemp(InObject))
	{
	}

	class FHelper
	{
	public:
		FHelper(TMonitor* InOwner)
			: Owner(InOwner)
			, ScopeLock(&InOwner->Mutex)
		{
		}

		T* operator->()
		{
			return &Owner->Object;
		}

		T& operator*()
		{
			return Owner->Object;
		}

	private:

		TMonitor* Owner;
		FScopeLock ScopeLock;
	};

	FHelper operator->()
	{
		return FHelper(this);
	}

	FHelper Lock()
	{
		return FHelper(this);
	}

	T& GetUnsafe()
	{
		return Object;
	}

	T Claim()
	{
		return MoveTemp(Object);
	}

private:

	FCriticalSection Mutex;
	T Object;
};

}