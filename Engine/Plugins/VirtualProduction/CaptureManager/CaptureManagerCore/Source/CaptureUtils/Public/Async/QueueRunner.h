// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"

#include "HAL/Event.h"
#include "HAL/RunnableThread.h"
#include "HAL/Runnable.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformProcess.h"

namespace UE::CaptureManager
{

template<class T>
class TQueueRunner : public FRunnable
{
public:

	DECLARE_DELEGATE_OneParam(FOnProcess, T InElem)

	TQueueRunner(FOnProcess InOnProcess = nullptr)
		: bRunning(false)
		, OnProcess(MoveTemp(InOnProcess))
	{
		Thread.Reset(FRunnableThread::Create(this, TEXT("Queue Runner"), 128 * 1024, TPri_Normal, FPlatformAffinity::GetPoolThreadMask()));
	}

	virtual ~TQueueRunner() override
	{
		if (Thread)
		{
			Thread->Kill(true);
		}
	}

	void Add(T InElement)
	{
		{
			FScopeLock lock(&Mutex);
			Queue.Enqueue(MoveTemp(InElement));
		}

		Event->Trigger();
	}

	void Empty()
	{
		FScopeLock lock(&Mutex);
		Queue.Empty();
	}

	bool IsRunning() const
	{
		return bRunning.load();
	}

protected:

	virtual uint32 Run() override
	{
		bRunning = true;

		T Elem;
	
		while (bRunning.load())
		{
			Event->Wait();

			for (;;)
			{
				{
					FScopeLock Lock(&Mutex);
					if (!Queue.Dequeue(Elem))
					{
						break;
					}
				}

				OnProcess.ExecuteIfBound(MoveTemp(Elem));
			}
		}

		return 0;
	}

	virtual void Stop() override
	{
		bRunning.store(false);

		Event->Trigger();
	}

private:

	std::atomic<bool> bRunning;
	FOnProcess OnProcess;

	FCriticalSection Mutex;
	FEventRef Event;
	TQueue<T> Queue;
	TUniquePtr<FRunnableThread> Thread;
};

}