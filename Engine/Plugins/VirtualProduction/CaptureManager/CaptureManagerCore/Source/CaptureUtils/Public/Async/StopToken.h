// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API CAPTUREUTILS_API

namespace UE::CaptureManager
{

struct FSharedState;

class FStopToken
{
public:

	FStopToken() = default;
	FStopToken(const FStopToken& InOther) = default;
	FStopToken(FStopToken&& InOther) = default;
	FStopToken& operator=(const FStopToken& InOther) = default;
	FStopToken& operator=(FStopToken&& InOther) = default;

	UE_API bool IsStopRequested() const;

private:

	friend class FStopRequester;

	UE_API FStopToken(TWeakPtr<const FSharedState> InSharedState);

	TWeakPtr<const FSharedState> SharedStateWeak;
};

class FStopRequester
{
public:
	UE_API FStopRequester();

	FStopRequester(const FStopRequester& InOther) = default;
	FStopRequester(FStopRequester&& InOther) = default;
	FStopRequester& operator=(const FStopRequester& InOther) = default;
	FStopRequester& operator=(FStopRequester&& InOther) = default;

	UE_API void RequestStop();
	UE_API bool IsStopRequested() const;

	UE_API FStopToken CreateToken() const;

private:

	TSharedPtr<FSharedState> SharedState;
};

}

#undef UE_API
