// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"
#include "Async/StopToken.h"

#define UE_API CAPTUREUTILS_API

template<typename TTask>
class FAsyncTask;

namespace UE::CaptureManager
{

class FCancelableAsyncTask
{
public:
	using FTaskFunction = TFunction<void(const FStopToken&)>;

	UE_API explicit FCancelableAsyncTask(FTaskFunction InTaskFunction);

	UE_API ~FCancelableAsyncTask();

	UE_API bool IsDone();

	UE_API void StartSync();
	UE_API void StartAsync();

	UE_API void Cancel();

private:
	class FAsyncTaskInternal;

	FStopRequester StopRequester;
	TUniquePtr<FAsyncTask<FAsyncTaskInternal>> AsyncTask;
};

}

#undef UE_API
