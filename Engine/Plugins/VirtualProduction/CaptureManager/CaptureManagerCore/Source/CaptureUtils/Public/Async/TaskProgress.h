// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/CriticalSection.h"
#include "Delegates/Delegate.h"

#include "Monitor.h"

#define UE_API CAPTUREUTILS_API

namespace UE::CaptureManager
{

class FTaskProgress : public TSharedFromThis<FTaskProgress>
{
public:

	DECLARE_DELEGATE_OneParam(FProgressReporter, double InProgress);

	class FTask
	{
	public:

		UE_API FTask();

		UE_API FTask(const FTask& InTask);
		UE_API FTask& operator=(const FTask& InTask);

		UE_API FTask(FTask&& InTask);
		UE_API FTask& operator=(FTask&& InTask);

		UE_API void Update(double InProgress);

	private:

		UE_API FTask(TWeakPtr<FTaskProgress> InTaskProgress, int32 InId);

		TWeakPtr<FTaskProgress> TaskProgress;
		int32 Id;

		friend FTaskProgress;
	};

	UE_API FTaskProgress(uint32 InAmountOfWork, FProgressReporter InReport);
	UE_API void SetReportThreshold(double InReportThresholdPercent);

	UE_API FTask StartTask();

	UE_API double GetTotalProgress() const;

private:

	UE_API void Update(int32 InTaskToUpdate, double InProgress);
	UE_API void Report();

	FProgressReporter Reporter;

	std::atomic<int32> CurrentTask;
	double ReportThreshold = 0.0;

	std::atomic<double> LastReportedValue = 0.0;

	mutable TMonitor<TArray<double>> CurrentProgressValues;
};

}

#undef UE_API
