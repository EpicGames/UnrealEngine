// Copyright Epic Games, Inc. All Rights Reserved.

#include <atomic>
#include "HAL/Platform.h"

#define UE_API CAPTUREUTILS_API

namespace UE::CaptureManager
{

class FTaskWaiter
{
public:

	UE_API FTaskWaiter();
	UE_API ~FTaskWaiter();

	UE_API bool CreateTask();
	UE_API void FinishTask();
	UE_API void WaitForAll();

private:

	std::atomic<uint32> TaskCounter;
	const uint32 CanCreateTaskFlag = 0x80000000;
};

}

#undef UE_API
