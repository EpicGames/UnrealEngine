// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

#include "Async/CaptureTimerManager.h"

#define UE_API CAPTUREUTILS_API

class FCaptureUtilsModule : public IModuleInterface
{
public:

	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	UE_API TSharedRef<UE::CaptureManager::FCaptureTimerManager> GetTimerManager();

private:

	TSharedPtr<UE::CaptureManager::FCaptureTimerManager> TimerManager;
};

#undef UE_API
