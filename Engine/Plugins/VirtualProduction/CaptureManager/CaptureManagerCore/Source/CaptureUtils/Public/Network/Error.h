// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

#include "Templates/ValueOrError.h"

#define UE_API CAPTUREUTILS_API

namespace UE::CaptureManager
{

class FCaptureProtocolError
{
public:

	UE_API FCaptureProtocolError();
	UE_API FCaptureProtocolError(FString InMessage, int32 InCode = 0);

	FCaptureProtocolError(const FCaptureProtocolError& InOther) = default;
	FCaptureProtocolError(FCaptureProtocolError&& InOther) = default;

	FCaptureProtocolError& operator=(const FCaptureProtocolError& InOther) = default;
	FCaptureProtocolError& operator=(FCaptureProtocolError&& InOther) = default;

	UE_API const FString& GetMessage() const;
	UE_API int32 GetCode() const;

private:

	FString Message;
	int32 Code = 0;
};

template <typename ValueType>
class TProtocolResult : public TValueOrError<ValueType, FCaptureProtocolError>
{
private:

	using Base = TValueOrError<ValueType, FCaptureProtocolError>;

public:

	TProtocolResult(ValueType&& InValueType)
		: Base(MakeValue(MoveTemp(InValueType)))
	{
	}

	TProtocolResult(const ValueType& InValueType)
		: Base(MakeValue(InValueType))
	{
	}

	TProtocolResult(FCaptureProtocolError&& InErrorType)
		: Base(MakeError(MoveTemp(InErrorType)))
	{
	}

	TProtocolResult(const FCaptureProtocolError& InErrorType)
		: Base(MakeError(InErrorType))
	{
	}

	template <typename... ArgTypes>
	TProtocolResult(TInPlaceType<ValueType>&&, ArgTypes&&... Args)
		: Base(MakeValue(Forward<ArgTypes>(Args)...))
	{
	}

	template <typename... ArgTypes>
	TProtocolResult(TInPlaceType<FCaptureProtocolError>&&, ArgTypes&&... Args)
		: Base(MakeError(Forward<ArgTypes>(Args)...))
	{
	}
};

template <>
class TProtocolResult<void> : public TValueOrError<void, FCaptureProtocolError>
{
private:

	using Base = TValueOrError<void, FCaptureProtocolError>;

public:

	TProtocolResult(TInPlaceType<void>)
		: Base(MakeValue())
	{
	}

	TProtocolResult(FCaptureProtocolError&& InErrorType)
		: Base(MakeError(MoveTemp(InErrorType)))
	{
	}

	TProtocolResult(const FCaptureProtocolError& InErrorType)
		: Base(MakeError(InErrorType))
	{
	}

	template <typename... ArgTypes>
	TProtocolResult(ArgTypes&&... Args)
		: Base(MakeError(Forward<ArgTypes>(Args)...))
	{
	}
};

const TProtocolResult<void> ResultOk = TInPlaceType<void>{};

}

#undef UE_API
