// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/Optional.h"

namespace UE::CaptureManager
{

CAPTUREUTILS_API TOptional<FString> GetLocalIpAddress();
CAPTUREUTILS_API TOptional<FString> GetLocalHostName();
CAPTUREUTILS_API FString GetLocalHostNameChecked();

}