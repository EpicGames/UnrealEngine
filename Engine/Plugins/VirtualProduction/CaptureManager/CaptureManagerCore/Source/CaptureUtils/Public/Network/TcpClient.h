// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TcpReaderWriter.h"

#include "Network/Error.h"

#include "Runtime/Sockets/Public/IPAddress.h"

#include "Common/TcpSocketBuilder.h"

#include "EngineLogs.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"

#include "Containers/UnrealString.h"

#include "Templates/UniquePtr.h"

#include "Misc/DateTime.h"

#define UE_API CAPTUREUTILS_API

namespace UE::CaptureManager
{

using FSocketPtr = TUniquePtr<FSocket, FSocketDeleter>;

class FTcpClient final
{
public:

	static constexpr uint32 BufferSize = 2 * 1024 * 1024;
	static constexpr int32 DisconnectedError = -10;
	static constexpr int32 ReadError = -3;
	static constexpr int32 NoPendingDataError = -2;
	static constexpr int32 TimeoutError = -1;

	UE_API FTcpClient();
	UE_API ~FTcpClient();

	UE_API TProtocolResult<void> Init();
	// Blocking function until connection is established
	UE_API TProtocolResult<void> Start(const FString& InServerAddress);
	UE_API TProtocolResult<void> Stop();

	UE_API bool IsRunning() const;

	UE_API TProtocolResult<void> SendMessage(const TArray<uint8>& InPayload);
	UE_API TProtocolResult<TArray<uint8>> ReceiveMessage(const uint64 InSize, const uint32 InWaitTimeoutMs = ITcpSocketReader::DefaultWaitTimeoutMs);

private:

	FSocketPtr TcpSocket;

	bool bRunning;
};

class FTcpClientReader final : public ITcpSocketReader
{
public:
	UE_API FTcpClientReader(FTcpClient& InClient);

	UE_API virtual TProtocolResult<TArray<uint8>> ReceiveMessage(const uint64 InSize, const uint32 InWaitTimeoutMs = DefaultWaitTimeoutMs) override;

private:
	FTcpClient& Client;
};

class FTcpClientWriter final : public ITcpSocketWriter
{
public:
	UE_API FTcpClientWriter(FTcpClient& InClient);

	UE_API virtual TProtocolResult<void> SendMessage(const TArray<uint8>& InPayload) override;

private:

	FTcpClient& Client;
};

}

#undef UE_API
