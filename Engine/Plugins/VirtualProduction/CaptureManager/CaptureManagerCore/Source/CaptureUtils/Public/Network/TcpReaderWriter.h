// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Error.h"

#include "Containers/Array.h"

namespace UE::CaptureManager
{

class ITcpSocketReader
{
public:
	static constexpr uint32 DefaultWaitTimeoutMs = 1000;

	virtual ~ITcpSocketReader() = default;

	virtual TProtocolResult<TArray<uint8>> ReceiveMessage(const uint64 InSize, const uint32 InWaitTimeoutMs = DefaultWaitTimeoutMs) = 0;
};

class ITcpSocketWriter
{
public:
	virtual ~ITcpSocketWriter() = default;

	virtual TProtocolResult<void> SendMessage(const TArray<uint8>& InPayload) = 0;
};

}