// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/TcpReaderWriter.h"

#include "Network/Error.h"

#include "Common/TcpSocketBuilder.h"
#include "Common/TcpListener.h"

#include "EngineLogs.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "Async/Async.h"

#include "Containers/UnrealString.h"

#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

#include "Delegates/Delegate.h"

#define UE_API CAPTUREUTILS_API

namespace UE::CaptureManager
{

using FSocketPtr = TUniquePtr<FSocket, FSocketDeleter>;

class FTcpClientHandler final
{
public:

	static constexpr uint32 MaxBufferSize = 500 * 1024;
	static constexpr int32 DisconnectedError = -10;
	static constexpr int32 TimeoutError = -1;

	UE_API FTcpClientHandler(FSocketPtr InSocket, FString InEndpoint);
	UE_API ~FTcpClientHandler();

	UE_API TProtocolResult<void> SendMessage(const TArray<uint8>& InData);
	UE_API TProtocolResult<TArray<uint8>> ReceiveMessage(const uint64 InSize, const uint32 InWaitTimeoutMs = ITcpSocketReader::DefaultWaitTimeoutMs);

	UE_API TProtocolResult<uint32> HasPendingData() const;

	UE_API const FString& GetEndpoint() const;

	UE_API bool operator==(const FTcpClientHandler& InOther);

private:

	FSocketPtr Socket;
	FString Endpoint;
};

class FTcpServer
{
public:

	static constexpr uint32 ThreadWaitTime = 500; // Milliseconds
	static constexpr uint16 AnyPort = 0; // OS assigned port

	DECLARE_DELEGATE_TwoParams(FConnectionHandler, TWeakPtr<FTcpClientHandler> InClient, bool bConnected)

	UE_API FTcpServer(const uint32 InMaxNumberOfClients);

	UE_API TProtocolResult<uint16> Start(const uint16 InListenPort = AnyPort);
	UE_API TProtocolResult<void> Stop();

	UE_API bool IsRunning() const;

	UE_API TProtocolResult<void> SendMessage(const TArray<uint8>& InMessage, const FString& InEndpoint);

	UE_API void DisconnectClient(const FString& InEndpoint);
	UE_API void SetConnectionHandler(FConnectionHandler InOnConnectionHandler);

	UE_API int32 GetPort() const;

private:

	TUniquePtr<FTcpListener> Listener;
	FSocketPtr Socket;

	uint32 MaxNumberOfClients;
	TMap<FString, TSharedPtr<FTcpClientHandler>> Clients;

	FConnectionHandler OnConnectionHandler;
	FRWLock Mutex;

	std::atomic_bool bRunning;
};

class FTcpConnectionReader final : public ITcpSocketReader
{
public:
	UE_API FTcpConnectionReader(FTcpClientHandler& InClient);

	UE_API virtual TProtocolResult<TArray<uint8>> ReceiveMessage(const uint64 InSize, const uint32 InWaitTimeoutMs = DefaultWaitTimeoutMs) override;

private:
	FTcpClientHandler& Client;
};

class FTcpConnectionWriter final : public ITcpSocketWriter
{
public:
	UE_API FTcpConnectionWriter(FTcpClientHandler& InClient);

	UE_API virtual TProtocolResult<void> SendMessage(const TArray<uint8>& InPayload) override;

private:

	FTcpClientHandler& Client;
};

}

#undef UE_API
