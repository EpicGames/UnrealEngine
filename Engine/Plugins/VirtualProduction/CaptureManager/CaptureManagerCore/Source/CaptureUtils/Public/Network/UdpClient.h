// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Error.h"

#include "Runtime/Sockets/Public/IPAddress.h"

#include "Common/UdpSocketReceiver.h"
#include "Common/UdpSocketSender.h"
#include "Common/UdpSocketBuilder.h"

#include "EngineLogs.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"

#include "Containers/UnrealString.h"

#include "Templates/UniquePtr.h"

#define UE_API CAPTUREUTILS_API

namespace UE::CaptureManager
{

struct FUdpClientConfigure
{
	uint16 ListenPort = 0;
	FString MulticastIpAddress = "";
};

#define CHECK_BOOL(Function) if (bool Result = Function; !Result) { return FCaptureProtocolError("Failed to execute function"); }

class FUdpClient final
{
public:

	UE_API FUdpClient();
	UE_API ~FUdpClient();

	UE_API TProtocolResult<void> Init(FUdpClientConfigure InConfig, FOnSocketDataReceived InReceiveHandler);
	UE_API TProtocolResult<void> Start();
	UE_API TProtocolResult<void> Stop();

	UE_API TProtocolResult<int32> SendMessage(const TArray<uint8>& InPayload, const FString& InEndpoint);

private:

	static constexpr uint32 ThreadWaitTime = 500; // Milliseconds
	static constexpr uint32 BufferSize = 2 * 1024 * 1024;

	TUniquePtr<FSocket> UdpSocket;

	TUniquePtr<FUdpSocketReceiver> UdpReceiver;

	bool bRunning;
};

}

#undef UE_API
