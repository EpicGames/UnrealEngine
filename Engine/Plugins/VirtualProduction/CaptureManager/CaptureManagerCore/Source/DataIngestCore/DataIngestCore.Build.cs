// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DataIngestCore : ModuleRules
{
	public DataIngestCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"Engine",
			"CaptureUtils",
			"Media"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CoreUObject",
			"InputCore",
			"PropertyEditor",
			"Json",
			"CaptureDataCore"
		});
	}
}
