// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

using FDataIngestCoreModule = IModuleInterface;
IMPLEMENT_MODULE(FDataIngestCoreModule, DataIngestCore)
