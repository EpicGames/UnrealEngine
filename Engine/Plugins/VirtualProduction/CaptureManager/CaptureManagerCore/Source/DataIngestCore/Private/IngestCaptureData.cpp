// Copyright Epic Games, Inc. All Rights Reserved.

#include "IngestCaptureData.h"

#include "Serialization/JsonSerializer.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "IngestCaptureData"

#define INGEST_CAPTURE_DATA_SUPPORTED_VERSION_MIN 1
#define INGEST_CAPTURE_DATA_SUPPORTED_VERSION_MAX 1

DEFINE_LOG_CATEGORY_STATIC(LogIngestCaptureData, Log, All);

#define INGEST_CAPTURE_DATA_CHECK_AND_RETURN(Condition, Message)						 \
	if (!(Condition))																     \
	{                                                                                    \
		FText ErrorMessage = FText::Format(FText::FromString(TEXT("{0}")), Message);     \
		UE_LOGF(LogIngestCaptureData, Error, "%ls", *ErrorMessage.ToString());       \
		return MakeError(MoveTemp(ErrorMessage));                                        \
	}

const FString FIngestCaptureData::Extension = TEXT("cparch");

namespace UE::CaptureManager::IngestCaptureData
{

bool DoesSupportVersion(uint32 InVersion)
{
	return InVersion >= INGEST_CAPTURE_DATA_SUPPORTED_VERSION_MIN && InVersion <= INGEST_CAPTURE_DATA_SUPPORTED_VERSION_MAX;
}

TValueOrError<FIngestCaptureData::FVideo, FText> ParseVideoObject(const TSharedPtr<FJsonValue>& InVideo)
{
	FIngestCaptureData::FVideo Video;

	const TSharedPtr<FJsonObject>* Object = nullptr;
	InVideo->TryGetObject(Object);

	const TSharedPtr<FJsonObject>& ObjectRef = *Object;

	INGEST_CAPTURE_DATA_CHECK_AND_RETURN(ObjectRef->TryGetStringField(TEXT("Name"), Video.Name),
										 LOCTEXT("Parse_MissingVideoName", "Video object doesn't contain Name field"));

	INGEST_CAPTURE_DATA_CHECK_AND_RETURN(ObjectRef->TryGetStringField(TEXT("Path"), Video.Path),
										 LOCTEXT("Parse_MissingVideoPath", "Video object doesn't contain Path field"));
	// Optional
	float FrameRate = 0.0f;
	if (ObjectRef->TryGetNumberField(TEXT("FrameRate"), FrameRate))
	{
		Video.FrameRate = FrameRate;
	}

	uint32 FrameWidth = 0;
	if (ObjectRef->TryGetNumberField(TEXT("FrameWidth"), FrameWidth))
	{
		Video.FrameWidth = FrameWidth;
	}

	uint32 FrameHeight = 0;
	if (ObjectRef->TryGetNumberField(TEXT("FrameHeight"), FrameHeight))
	{
		Video.FrameHeight = FrameHeight;
	}

	FString TimecodeStart;
	if (ObjectRef->TryGetStringField(TEXT("TimecodeStart"), TimecodeStart))
	{
		Video.TimecodeStart = MoveTemp(TimecodeStart);
	}

	const TArray<TSharedPtr<FJsonValue>>* DroppedFrames;
	if (ObjectRef->TryGetArrayField(TEXT("DroppedFrames"), DroppedFrames))
	{
		for (const TSharedPtr<FJsonValue>& DroppedFrame : *DroppedFrames)
		{
			uint32 DroppedFrameNumber = 0;
			DroppedFrame->TryGetNumber(DroppedFrameNumber);

			Video.DroppedFrames.Add(DroppedFrameNumber);
		}
	}

	return MakeValue(MoveTemp(Video));
}

TValueOrError<FIngestCaptureData::FAudio, FText> ParseAudioObject(const TSharedPtr<FJsonValue>& InAudio)
{
	FIngestCaptureData::FAudio Audio;

	const TSharedPtr<FJsonObject>* Object = nullptr;
	InAudio->TryGetObject(Object);

	const TSharedPtr<FJsonObject>& ObjectRef = *Object;

	INGEST_CAPTURE_DATA_CHECK_AND_RETURN(ObjectRef->TryGetStringField(TEXT("Name"), Audio.Name),
										 LOCTEXT("Parse_MissingAudioName", "Audio object doesn't contain Name field"));

	INGEST_CAPTURE_DATA_CHECK_AND_RETURN(ObjectRef->TryGetStringField(TEXT("Path"), Audio.Path),
										 LOCTEXT("Parse_MissingAudioPath", "Audio object doesn't contain Path field"));

	FString TimecodeStart;
	if (ObjectRef->TryGetStringField(TEXT("TimecodeStart"), TimecodeStart))
	{
		Audio.TimecodeStart = MoveTemp(TimecodeStart);
	}

	float TimecodeRate = 0.0f;
	if (ObjectRef->TryGetNumberField(TEXT("TimecodeRate"), TimecodeRate))
	{
		Audio.TimecodeRate = TimecodeRate;
	}

	return MakeValue(MoveTemp(Audio));
}

TValueOrError<FIngestCaptureData::FCalibration, FText> ParseCalibrationObject(const TSharedPtr<FJsonValue>& InCalibration)
{
	FIngestCaptureData::FCalibration Calibration;

	const TSharedPtr<FJsonObject>* Object = nullptr;
	InCalibration->TryGetObject(Object);

	const TSharedPtr<FJsonObject>& ObjectRef = *Object;

	INGEST_CAPTURE_DATA_CHECK_AND_RETURN(ObjectRef->TryGetStringField(TEXT("Name"), Calibration.Name),
										 LOCTEXT("Parse_MissingCalibrationName", "Calibration object doesn't contain Name field"));

	INGEST_CAPTURE_DATA_CHECK_AND_RETURN(ObjectRef->TryGetStringField(TEXT("Path"), Calibration.Path),
										 LOCTEXT("Parse_MissingCalibrationPath", "Calibration object doesn't contain Path field"));

	return MakeValue(MoveTemp(Calibration));
}

FParseResult ParseFile(const FString& InFilePath)
{
	FString Extension = FPaths::GetExtension(InFilePath, false);

	INGEST_CAPTURE_DATA_CHECK_AND_RETURN(Extension == FIngestCaptureData::Extension, 
										 FText::Format(LOCTEXT("Parse_InvalidExtension", "Provided file has invalid extension (found '{0}', expected '{1}'"),
													   FText::FromString(Extension), FText::FromString(FIngestCaptureData::Extension)));

	FString Content;
	INGEST_CAPTURE_DATA_CHECK_AND_RETURN(FFileHelper::LoadFileToString(Content, *InFilePath), 
										 FText::Format(LOCTEXT("Parse_FailedToOpenFile", "Provided file doesn't exist {0}"),
													   FText::FromString(InFilePath)));

	TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<>::Create(Content);

	TSharedPtr<FJsonObject> FormatObject;
	INGEST_CAPTURE_DATA_CHECK_AND_RETURN(FJsonSerializer::Deserialize(Reader, FormatObject), 
										 FText::Format(LOCTEXT("Parse_NotJson", "Invalid json file {0}"),
													   FText::FromString(InFilePath)));

	uint32 Version = 0;
	INGEST_CAPTURE_DATA_CHECK_AND_RETURN(FormatObject->TryGetNumberField(TEXT("Version"), Version),
										 LOCTEXT("Parse_InvalidFormatVersion", "Json file doesn't contain version number"));

	INGEST_CAPTURE_DATA_CHECK_AND_RETURN(DoesSupportVersion(Version),
										 LOCTEXT("Parse_UnsupportedFormatVersion", "Parser doesn't support specified version"));

	FIngestCaptureData IngestCaptureData;
	IngestCaptureData.Version = Version;

	INGEST_CAPTURE_DATA_CHECK_AND_RETURN(FormatObject->TryGetStringField(TEXT("DeviceModel"), IngestCaptureData.DeviceModel),
										 LOCTEXT("Parse_MissingDeviceModel", "Json doesn't contain Device Model field"));
	INGEST_CAPTURE_DATA_CHECK_AND_RETURN(FormatObject->TryGetStringField(TEXT("Slate"), IngestCaptureData.Slate),
										 LOCTEXT("Parse_MissingTakeSlate", "Json doesn't contain Slate field"));
	INGEST_CAPTURE_DATA_CHECK_AND_RETURN(FormatObject->TryGetNumberField(TEXT("TakeNumber"), IngestCaptureData.TakeNumber),
										 LOCTEXT("Parse_MissingTakeNumber", "Json doesn't contain Take Number field"));

	// Optional
	const TArray<TSharedPtr<FJsonValue>>* VideoArray;
	if (FormatObject->TryGetArrayField(TEXT("Video"), VideoArray))
	{
		for (const TSharedPtr<FJsonValue>& Video : *VideoArray)
		{
			TValueOrError<FIngestCaptureData::FVideo, FText> Result = ParseVideoObject(Video);

			INGEST_CAPTURE_DATA_CHECK_AND_RETURN(Result.HasValue(), Result.StealError());

			FIngestCaptureData::FVideo ParsedVideo = Result.StealValue();
			IngestCaptureData.Video.Emplace(MoveTemp(ParsedVideo));
		}
	}

	// Optional
	const TArray<TSharedPtr<FJsonValue>>* DepthArray;
	if (FormatObject->TryGetArrayField(TEXT("Depth"), DepthArray))
	{
		for (const TSharedPtr<FJsonValue>& Depth : *DepthArray)
		{
			TValueOrError<FIngestCaptureData::FVideo, FText> Result = ParseVideoObject(Depth);

			INGEST_CAPTURE_DATA_CHECK_AND_RETURN(Result.HasValue(), Result.StealError());

			FIngestCaptureData::FVideo ParsedDepth = Result.StealValue();
			IngestCaptureData.Depth.Emplace(MoveTemp(ParsedDepth));
		}
	}

	// Optional
	const TArray<TSharedPtr<FJsonValue>>* AudioArray;
	if (FormatObject->TryGetArrayField(TEXT("Audio"), AudioArray))
	{
		for (const TSharedPtr<FJsonValue>& Audio : *AudioArray)
		{
			TValueOrError<FIngestCaptureData::FAudio, FText> Result = ParseAudioObject(Audio);

			INGEST_CAPTURE_DATA_CHECK_AND_RETURN(Result.HasValue(), Result.StealError());

			FIngestCaptureData::FAudio ParsedAudio = Result.StealValue();
			IngestCaptureData.Audio.Emplace(MoveTemp(ParsedAudio));
		}
	}

	const TArray<TSharedPtr<FJsonValue>>* CalibrationArray;
	if (FormatObject->TryGetArrayField(TEXT("Calibration"), CalibrationArray))
	{
		for (const TSharedPtr<FJsonValue>& Calibration : *CalibrationArray)
		{
			TValueOrError<FIngestCaptureData::FCalibration, FText> Result = ParseCalibrationObject(Calibration);

			INGEST_CAPTURE_DATA_CHECK_AND_RETURN(Result.HasValue(), Result.StealError());

			FIngestCaptureData::FCalibration ParsedCalibration = Result.StealValue();
			IngestCaptureData.Calibration.Emplace(MoveTemp(ParsedCalibration));
		}
	}

	return MakeValue(MoveTemp(IngestCaptureData));
}

void SerializeVideo(TSharedPtr<TJsonWriter<TCHAR>> InJsonWriter, const FIngestCaptureData::FVideo& InVideo)
{
	InJsonWriter->WriteObjectStart();

	InJsonWriter->WriteValue(TEXT("Name"), InVideo.Name);
	InJsonWriter->WriteValue(TEXT("Path"), InVideo.Path);

	if (InVideo.FrameRate.IsSet())
	{
		InJsonWriter->WriteValue(TEXT("FrameRate"), InVideo.FrameRate.GetValue());
	}

	if (InVideo.FrameWidth.IsSet())
	{
		InJsonWriter->WriteValue(TEXT("FrameWidth"), InVideo.FrameWidth.GetValue());
	}

	if (InVideo.FrameHeight.IsSet())
	{
		InJsonWriter->WriteValue(TEXT("FrameHeight"), InVideo.FrameHeight.GetValue());
	}

	if (InVideo.TimecodeStart.IsSet())
	{
		InJsonWriter->WriteValue(TEXT("TimecodeStart"), InVideo.TimecodeStart.GetValue());
	}

	if (!InVideo.DroppedFrames.IsEmpty())
	{ 
		InJsonWriter->WriteArrayStart(TEXT("DroppedFrames"));

		for (uint32 DroppedFrame : InVideo.DroppedFrames)
		{
			InJsonWriter->WriteValue(DroppedFrame);
		}

		InJsonWriter->WriteArrayEnd();
	}

	InJsonWriter->WriteObjectEnd();
}

void SerializeAudio(TSharedPtr<TJsonWriter<TCHAR>> InJsonWriter, const FIngestCaptureData::FAudio& InAudio)
{
	InJsonWriter->WriteObjectStart();

	InJsonWriter->WriteValue(TEXT("Name"), InAudio.Name);
	InJsonWriter->WriteValue(TEXT("Path"), InAudio.Path);

	if (InAudio.TimecodeStart.IsSet())
	{
		InJsonWriter->WriteValue(TEXT("TimecodeStart"), InAudio.TimecodeStart.GetValue());
	}

	if (InAudio.TimecodeRate.IsSet())
	{
		InJsonWriter->WriteValue(TEXT("TimecodeRate"), InAudio.TimecodeRate.GetValue());
	}

	InJsonWriter->WriteObjectEnd();
}

void SerializeCalibration(TSharedPtr<TJsonWriter<TCHAR>> InJsonWriter, const FIngestCaptureData::FCalibration& InCalibration)
{
	InJsonWriter->WriteObjectStart();

	InJsonWriter->WriteValue(TEXT("Name"), InCalibration.Name);
	InJsonWriter->WriteValue(TEXT("Path"), InCalibration.Path);

	InJsonWriter->WriteObjectEnd();
}

TOptional<FText> Serialize(const FString& InFilePath, const FString& InFileName, const FIngestCaptureData& InIngestCaptureData)
{
	FString FileName = FPaths::SetExtension(InFileName, FIngestCaptureData::Extension);

	FString Content;
	TSharedPtr<TJsonWriter<TCHAR>> JsonWriter = TJsonWriterFactory<TCHAR>::Create(&Content);

	JsonWriter->WriteObjectStart();

	JsonWriter->WriteValue(TEXT("Version"), InIngestCaptureData.Version);
	JsonWriter->WriteValue(TEXT("DeviceModel"), InIngestCaptureData.DeviceModel);
	JsonWriter->WriteValue(TEXT("Slate"), InIngestCaptureData.Slate);
	JsonWriter->WriteValue(TEXT("TakeNumber"), InIngestCaptureData.TakeNumber);

	if (!InIngestCaptureData.Video.IsEmpty())
	{
		JsonWriter->WriteArrayStart(TEXT("Video"));

		for (const FIngestCaptureData::FVideo& Video : InIngestCaptureData.Video)
		{
			SerializeVideo(JsonWriter, Video);
		}

		JsonWriter->WriteArrayEnd();
	}

	if (!InIngestCaptureData.Depth.IsEmpty())
	{
		JsonWriter->WriteArrayStart(TEXT("Depth"));

		for (const FIngestCaptureData::FVideo& Depth : InIngestCaptureData.Depth)
		{
			SerializeVideo(JsonWriter, Depth);
		}

		JsonWriter->WriteArrayEnd();
	}

	if (!InIngestCaptureData.Audio.IsEmpty())
	{
		JsonWriter->WriteArrayStart(TEXT("Audio"));

		for (const FIngestCaptureData::FAudio& Audio : InIngestCaptureData.Audio)
		{
			SerializeAudio(JsonWriter, Audio);
		}

		JsonWriter->WriteArrayEnd();
	}

	if (!InIngestCaptureData.Calibration.IsEmpty())
	{
		JsonWriter->WriteArrayStart(TEXT("Calibration"));

		for (const FIngestCaptureData::FCalibration& Calibration : InIngestCaptureData.Calibration)
		{
			SerializeCalibration(JsonWriter, Calibration);
		}

		JsonWriter->WriteArrayEnd();
	}

	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();

	FString FullFilePath = InFilePath / FileName;
	if (!FFileHelper::SaveStringToFile(Content, *FullFilePath))
	{
		return FText::Format(LOCTEXT("Serialize_FailedToWrite", "Failed to serialize json file {0}"), FText::FromString(FullFilePath));
	}

	return {};
}

}

void FIngestCaptureData::MakePathsAbsolute(const FString& InBasePath)
{
	for (FVideo& VideoEntry : Video)
	{
		VideoEntry.Path = FPaths::ConvertRelativePathToFull(InBasePath, VideoEntry.Path);
	}

	for (FVideo& DepthEntry : Depth)
	{
		DepthEntry.Path = FPaths::ConvertRelativePathToFull(InBasePath, DepthEntry.Path);
	}

	for (FAudio& AudioEntry : Audio)
	{
		AudioEntry.Path = FPaths::ConvertRelativePathToFull(InBasePath, AudioEntry.Path);
	}

	for (FCalibration& CalibrationEntry : Calibration)
	{
		CalibrationEntry.Path = FPaths::ConvertRelativePathToFull(InBasePath, CalibrationEntry.Path);
	}
}

#undef LOCTEXT_NAMESPACE