// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/UnrealCalibrationParser.h"

#include "Serialization/JsonSerializer.h"

#include "Misc/FileHelper.h"

#define LOCTEXT_NAMESPACE "UnrealCalibrationParser"

DEFINE_LOG_CATEGORY_STATIC(LogUnrealCalibrationParser, Log, All);

#define CALIB_CHECK_AND_RETURN(Condition, Message) \
	if (!(Condition))																     \
	{                                                                                    \
		FText ErrorMessage = FText::Format(FText::FromString(TEXT("{0}")), Message);     \
		UE_LOGF(LogUnrealCalibrationParser, Error, "%ls", *ErrorMessage.ToString()); \
		return MakeError(MoveTemp(ErrorMessage));                                        \
	}

#define UNREAL_FORMAT_SUPPORTED_VERSION_MIN 1
#define UNREAL_FORMAT_SUPPORTED_VERSION_MAX 1

bool DoesSupportVersion(uint32 InVersion)
{
	return InVersion >= UNREAL_FORMAT_SUPPORTED_VERSION_MIN && InVersion <= UNREAL_FORMAT_SUPPORTED_VERSION_MAX;
}

FUnrealCalibrationParser::FParseResult FUnrealCalibrationParser::Parse(const FString& InFile)
{
	UE_LOGF(LogUnrealCalibrationParser, Display, "Parsing the calibration file: %ls", *InFile);

	FString Content;

	CALIB_CHECK_AND_RETURN(FFileHelper::LoadFileToString(Content, *InFile), LOCTEXT("Parse_FailedToOpenFile", "Provided file doesn't exist"));

	TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<>::Create(Content);

	TArray<FCameraCalibration> Result;

	TSharedPtr<FJsonObject> FormatObject;
	CALIB_CHECK_AND_RETURN(FJsonSerializer::Deserialize(Reader, FormatObject), LOCTEXT("Parse_NotJson", "Invalid json file"));

	uint32 Version = 0; 
	CALIB_CHECK_AND_RETURN(FormatObject->TryGetNumberField(TEXT("Version"), Version),
						   LOCTEXT("Parse_InvalidFormatVersion", "Json file doesn't contain version number"));

	CALIB_CHECK_AND_RETURN(DoesSupportVersion(Version),
						   LOCTEXT("Parse_UnsupportedFormatVersion", "Parser doesn't support specified version"));

	const TArray<TSharedPtr<FJsonValue>>* CameraArray;
	FormatObject->TryGetArrayField(TEXT("Calibrations"), CameraArray);

	CALIB_CHECK_AND_RETURN(!CameraArray->IsEmpty(), LOCTEXT("Parse_InvalidArrayFormat", "Json doesn't contain camera array"));

	for (const TSharedPtr<FJsonValue>& Camera : *CameraArray)
	{
		FCameraCalibration CameraCalibration;

		const TSharedPtr<FJsonObject>* CameraObjectPtr = nullptr;
		CALIB_CHECK_AND_RETURN(Camera->TryGetObject(CameraObjectPtr), 
							   LOCTEXT("Parse_InvalidObjectFormat", "Json file doesn't contain camera objects within the array"));

		const TSharedPtr<FJsonObject>& CameraObject = *CameraObjectPtr;
		CALIB_CHECK_AND_RETURN(CameraObject->TryGetStringField(TEXT("CameraId"), CameraCalibration.CameraId),
							   LOCTEXT("Parse_MissingCameraId", "Json doesn't contain Camera Id field"));

		std::underlying_type_t<FCameraCalibration::EType> CameraType;
		CALIB_CHECK_AND_RETURN(CameraObject->TryGetNumberField(TEXT("CameraType"), CameraType),
							   LOCTEXT("Parse_MissingCameraType", "Json doesn't contain Camera Type field"));

		CameraCalibration.CameraType = static_cast<FCameraCalibration::EType>(CameraType);

		const TSharedPtr<FJsonObject>* DimensionsObjectPtr = nullptr;
		CALIB_CHECK_AND_RETURN(CameraObject->TryGetObjectField(TEXT("Dimensions"), DimensionsObjectPtr), 
							   LOCTEXT("Parse_MissingDimensions", "Json doesn't contain Dimensions field"));

		const TSharedPtr<FJsonObject>& DimensionsObject = *DimensionsObjectPtr;
		CALIB_CHECK_AND_RETURN(DimensionsObject->TryGetNumberField(TEXT("Width"), CameraCalibration.ImageSize.X), 
							   LOCTEXT("Parse_MissingWidth", "Json doesn't contain Width field"));
		CALIB_CHECK_AND_RETURN(DimensionsObject->TryGetNumberField(TEXT("Height"), CameraCalibration.ImageSize.Y), 
							   LOCTEXT("Parse_MissingHeight", "Json doesn't contain Height field"));

		std::underlying_type_t<EMediaOrientation> Orientation;
		CALIB_CHECK_AND_RETURN(CameraObject->TryGetNumberField(TEXT("Orientation"), Orientation), 
							   LOCTEXT("Parse_MissingOrientation", "Json doesn't contain Orientation field"));

		CameraCalibration.Orientation = static_cast<EMediaOrientation>(Orientation);

		const TSharedPtr<FJsonObject>* DistortionModelPtr = nullptr;
		bool bResult = CameraObject->TryGetObjectField(TEXT("DistortionModel"), DistortionModelPtr);
		if (bResult)
		{
			const TSharedPtr<FJsonObject>& DistortionModel = *DistortionModelPtr;

			FString Name;
			CALIB_CHECK_AND_RETURN(DistortionModel->TryGetStringField(TEXT("Name"), Name),
								   LOCTEXT("Parse_MissingModelName", "Json doesn't contain Name field within Distortion Model object"));

			if (Name == TEXT("opencv"))
			{
				const TSharedPtr<FJsonObject>* RadialDistortionPtr = nullptr;
				CALIB_CHECK_AND_RETURN(DistortionModel->TryGetObjectField(TEXT("Radial"), RadialDistortionPtr),
									   LOCTEXT("Parse_MissingRadialDistortion", "Json doesn't contain Radial field within Distortion Model object"));
				const TSharedPtr<FJsonObject>& RadialDistortion = *RadialDistortionPtr;

				CALIB_CHECK_AND_RETURN(RadialDistortion->TryGetNumberField(TEXT("K1"), CameraCalibration.K1),
									   LOCTEXT("Parse_MissingRadialDistortionK1", "Json doesn't contain K1 field within Radial distortion object"));
				CALIB_CHECK_AND_RETURN(RadialDistortion->TryGetNumberField(TEXT("K2"), CameraCalibration.K2),
									   LOCTEXT("Parse_MissingRadialDistortionK2", "Json doesn't contain K2 field within Radial distortion object"));
				CALIB_CHECK_AND_RETURN(RadialDistortion->TryGetNumberField(TEXT("K3"), CameraCalibration.K3),
									   LOCTEXT("Parse_MissingRadialDistortionK3", "Json doesn't contain K3 field within Radial distortion object"));

				const TSharedPtr<FJsonObject>* TangentialDistortionPtr = nullptr;
				CALIB_CHECK_AND_RETURN(DistortionModel->TryGetObjectField(TEXT("Tangential"), TangentialDistortionPtr),
									   LOCTEXT("Parse_MissingTangentialDistortion", "Json doesn't contain Tangential field within Distortion Model object"));
				const TSharedPtr<FJsonObject>& TangentialDistortion = *TangentialDistortionPtr;

				CALIB_CHECK_AND_RETURN(TangentialDistortion->TryGetNumberField(TEXT("P1"), CameraCalibration.P1),
									   LOCTEXT("Parse_MissingTangentialDistortionP1", "Json doesn't contain P1 field within Tangential distortion object"));
				CALIB_CHECK_AND_RETURN(TangentialDistortion->TryGetNumberField(TEXT("P2"), CameraCalibration.P2),
									   LOCTEXT("Parse_MissingTangentialDistortionP2", "Json doesn't contain P2 field within Tangential distortion object"));
			}
		}

		const TArray<TSharedPtr<FJsonValue>>* FocalLengthPtr;
		CALIB_CHECK_AND_RETURN(CameraObject->TryGetArrayField(TEXT("FocalLength"), FocalLengthPtr),
							   LOCTEXT("Parse_MissingFocalLength", "Json doesn't contain Focal Length field"));

		const TArray<TSharedPtr<FJsonValue>>& FocalLength = *FocalLengthPtr;
		FocalLength[0]->TryGetNumber(CameraCalibration.FocalLengthNormalized.X);
		FocalLength[1]->TryGetNumber(CameraCalibration.FocalLengthNormalized.Y);

		const TArray<TSharedPtr<FJsonValue>>* PrincipalPointPtr;
		CALIB_CHECK_AND_RETURN(CameraObject->TryGetArrayField(TEXT("PrincipalPoint"), PrincipalPointPtr),
							   LOCTEXT("Parse_MissingPrincipalPoint", "Json doesn't contain Principal Point field"));

		const TArray<TSharedPtr<FJsonValue>>& PrincipalPoint = *PrincipalPointPtr;
		PrincipalPoint[0]->TryGetNumber(CameraCalibration.PrincipalPointNormalized.X);
		PrincipalPoint[1]->TryGetNumber(CameraCalibration.PrincipalPointNormalized.Y);

		const TArray<TSharedPtr<FJsonValue>>* TransformPtr;
		CALIB_CHECK_AND_RETURN(CameraObject->TryGetArrayField(TEXT("Transform"), TransformPtr),
							   LOCTEXT("Parse_MissingTransform", "Json doesn't contain Transform field"));

		const TArray<TSharedPtr<FJsonValue>>& TransformArray = *TransformPtr;

		for (int32 Row = 0; Row < 4; ++Row)
		{
			for (int32 Column = 0; Column < 4; ++Column)
			{
				TransformArray[Row * 4 + Column]->TryGetNumber(CameraCalibration.Transform.M[Row][Column]);
			}
		}

		Result.Add(MoveTemp(CameraCalibration));
	}

	return MakeValue(MoveTemp(Result));
}

#undef LOCTEXT_NAMESPACE