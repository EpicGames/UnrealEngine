// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/ValueOrError.h"
#include "Containers/UnrealString.h"

#define UE_API DATAINGESTCORE_API

/** Capture archive data object for import into UE. */
struct FIngestCaptureData
{
	/** Expected extension for capture archive files. */
	static UE_API const FString Extension;

	/** Video/image sequence information. */
	struct FVideo
	{
		FString Name;
		FString Path;
		
		TOptional<float> FrameRate;
		TOptional<uint32> FrameWidth;
		TOptional<uint32> FrameHeight;

		TArray<uint32> DroppedFrames;

		TOptional<FString> TimecodeStart;
	};

	/** Audio info. */
	struct FAudio
	{
		FString Name;
		FString Path;

		TOptional<FString> TimecodeStart;
		TOptional<float> TimecodeRate;
	};

	/** Calibration info. */
	struct FCalibration
	{
		FString Name;
		FString Path;
	};

	/** Archive format version. */
	uint32 Version = 1;
	FString DeviceModel;
	FString Slate;
	uint32 TakeNumber;
	
	TArray<FVideo> Video;
	TArray<FVideo> Depth;
	TArray<FAudio> Audio;
	TArray<FCalibration> Calibration;

	/** Converts all relative paths to absolute using InBasePath. */
	UE_API void MakePathsAbsolute(const FString& InBasePath);
};

namespace UE::CaptureManager::IngestCaptureData
{

/** Parse input file into capture archive data object. */
using FParseResult = TValueOrError<FIngestCaptureData, FText>;
DATAINGESTCORE_API FParseResult ParseFile(const FString& InFilePath);

/** Serialise capture archive data object to file. */
DATAINGESTCORE_API TOptional<FText> Serialize(const FString& InFilePath, const FString& InFileName, const FIngestCaptureData& InIngestCaptureData);

}

#undef UE_API
