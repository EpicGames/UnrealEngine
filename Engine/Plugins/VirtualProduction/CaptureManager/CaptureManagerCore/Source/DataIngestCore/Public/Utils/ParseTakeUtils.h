// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Timecode.h"
#include "Misc/FrameRate.h"

#include "Containers/UnrealString.h"

namespace UE::CaptureManager
{

/** Parse timecode from string. */
DATAINGESTCORE_API FTimecode ParseTimecode(const FString& InTimecodeString);

/** Parse frame rate from double. */
DATAINGESTCORE_API FFrameRate ParseFrameRate(double InFrameRate);

}