// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraCalibration.h"

#define UE_API DATAINGESTCORE_API

class FUnrealCalibrationParser
{
public:
	using FParseResult = TValueOrError<TArray<FCameraCalibration>, FText>;

	/** Parse input file into unreal calibration format. */
	static UE_API FParseResult Parse(const FString& InFile);
};

#undef UE_API
