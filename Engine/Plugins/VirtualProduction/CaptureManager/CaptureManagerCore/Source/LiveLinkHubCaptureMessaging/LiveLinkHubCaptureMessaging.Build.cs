// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class LiveLinkHubCaptureMessaging : ModuleRules
{
	public LiveLinkHubCaptureMessaging(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
			"CaptureUtils",
			"CaptureProtocolStack"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"MessagingCommon"
		});
	}
}
