// Copyright Epic Games, Inc. All Rights Reserved.

#include "Features/ConnectAcceptor.h"

#include "CaptureUtilsModule.h"

namespace UE::CaptureManager::ConnectAcceptor
{

static TSharedRef<FCaptureTimerManager> GetTimerManager()
{
	FCaptureUtilsModule& CaptureUtilsModule =
		FModuleManager::LoadModuleChecked<FCaptureUtilsModule>("CaptureUtils");
	return CaptureUtilsModule.GetTimerManager();
}

}

FConnectAcceptor::FConnectAcceptor()
	: TimerManager(UE::CaptureManager::ConnectAcceptor::GetTimerManager())
{
}

FConnectAcceptor::~FConnectAcceptor()
{
	TimerManager->RemoveTimer(ClientActivity);
}

void FConnectAcceptor::SetConnectionHandler(FConnectAccepted InConnectHandler, FConnectionLostHandler InConnectionLostHandler)
{
	ConnectHandler = InConnectHandler;
	ConnectionLostHandler = InConnectionLostHandler;
}

void FConnectAcceptor::Disconnect()
{
	if (bConnected)
	{
		bConnected = false;
		TimerManager->RemoveTimer(ClientActivity);

		LastPingRequest = 0.0f;

		ConnectionLostHandler.ExecuteIfBound(GetAddress());
	}
}

bool FConnectAcceptor::IsConnected() const
{
	return bConnected;
}

void FConnectAcceptor::Initialize(FMessageEndpointBuilder& InBuilder)
{
	InBuilder
		.Handling<FConnectRequest>(this, &FConnectAcceptor::ConnectRequestHandler)
		.Handling<FCaptureManagerHangUp>(this, &FConnectAcceptor::OnHangUpReceived)
		.Handling<FPingMessage>(this, &FConnectAcceptor::HandleKeepAlive);
}

void FConnectAcceptor::ConnectRequestHandler(const FConnectRequest& InRequest, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{	
	using namespace UE::CaptureManager;

	FConnectResponse* Response = nullptr;

	if (!bConnected)
	{
		Response = ConnectHandler.Execute(InRequest, InContext->GetSender());

		if (Response->Status == EStatus::Ok)
		{
			LastPingRequest = FPlatformTime::Seconds();

			static const float CheckConnectionActivityInterval = 5.0f;
			ClientActivity = TimerManager->AddTimer(FTimerDelegate::CreateRaw(this, &FConnectAcceptor::CheckConnectionActivity), CheckConnectionActivityInterval, true);
		}
	}
	else
	{
		LastPingRequest = FPlatformTime::Seconds();

		// Already connected isn't considered an error
		Response = FMessageEndpoint::MakeMessage<FConnectResponse>();
	}

	Response->Guid = FGuid::NewGuid();
	Response->RequestGuid = InRequest.Guid;

	SendMessage(Response);

	if (Response->Status == EStatus::Ok)
	{
		bConnected = true;
	}
}

void FConnectAcceptor::OnHangUpReceived(
	const FCaptureManagerHangUp& InRequest,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext
)
{
	// The connection we currently have has been dropped from the other side.
	// Disconnect so we're ready to accept new connections.
	Disconnect();
}

void FConnectAcceptor::HandleKeepAlive(const FPingMessage& InRequest, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	if (!bConnected)
	{
		return;
	}

	FPongMessage* Response = FMessageEndpoint::MakeMessage<FPongMessage>();

	Response->Guid = FGuid::NewGuid();
	Response->RequestGuid = InRequest.Guid;

	SendMessage(Response);

	LastPingRequest = FPlatformTime::Seconds();
}

void FConnectAcceptor::CheckConnectionActivity()
{
	double CurrentTime = FPlatformTime::Seconds();
	if (CurrentTime - LastPingRequest >= InactivityTimeout)
	{
		Disconnect();
	}
}