// Copyright Epic Games, Inc. All Rights Reserved.

#include "Features/ConnectStarter.h"

#include "Async/Async.h"

#include "CaptureUtilsModule.h"

namespace UE::CaptureManager::ConnectStarter
{

static TSharedRef<FCaptureTimerManager> GetTimerManager()
{
	FCaptureUtilsModule& CaptureUtilsModule =
		FModuleManager::LoadModuleChecked<FCaptureUtilsModule>("CaptureUtils");
	return CaptureUtilsModule.GetTimerManager();
}

}

const float FConnectStarter::KeepAliveInterval = 7.5f; // seconds KeepAliveTimeout * 2 + KeepAliveTimeout / 2
const float FConnectStarter::KeepAliveTimeout = 3.0f; // seconds

FConnectStarter::FConnectStarter()
	: TimerManager(UE::CaptureManager::ConnectStarter::GetTimerManager())
{
}

FConnectStarter::~FConnectStarter()
{
	TimerManager->RemoveTimer(KeepAlive);

	{
		FScopeLock Lock(&PingMutex);
		PingContexts.Empty();
	}

	{
		FScopeLock Lock(&Mutex);
		Contexts.Empty();
	}
}

void FConnectStarter::Connect(FConnectHandler InConnectHandler)
{
	if (bConnected)
	{
		return;
	}

	FConnectRequest* Request = FMessageEndpoint::MakeMessage<FConnectRequest>();
	Request->Guid = FGuid::NewGuid();

	AddContext(Request->Guid, MoveTemp(InConnectHandler));

	SendMessage(Request);
}

void FConnectStarter::Disconnect()
{
	using namespace UE::CaptureManager;

	bConnected = false;

	FScopeLock Lock(&Mutex);

	// Take a copy, so we can execute the handler without holding the lock
	FDisconnectHandler LocalDisconnectHandler = Handler;
	Contexts.Empty();
	Lock.Unlock();

	// Send a hangup message so that UE can drop its side of the connection
	FCaptureManagerHangUp* HangUp = FMessageEndpoint::MakeMessage<FCaptureManagerHangUp>();
	SendMessage(HangUp);

	TimerManager->RemoveTimer(KeepAlive);
	LocalDisconnectHandler.ExecuteIfBound();
}

void FConnectStarter::SetDisconnectHandler(FDisconnectHandler InHandler)
{
	FScopeLock ScopeLock(&Mutex);
	Handler = MoveTemp(InHandler);
}

bool FConnectStarter::IsConnected() const
{
	return bConnected;
}

void FConnectStarter::Initialize(FMessageEndpointBuilder& InBuilder)
{
	InBuilder.Handling<FConnectResponse>(this, &FConnectStarter::HandleConnectResponse)
		.Handling<FPongMessage>(this, &FConnectStarter::HandlePingResponse);
}

FGuid FConnectStarter::SendPingRequest(FPingCallback InResponseCb)
{
	if (!bConnected)
	{
		return FGuid();
	}

	FPingMessage* Request = FMessageEndpoint::MakeMessage<FPingMessage>();
	Request->Guid = FGuid::NewGuid();

	FGuid RequestGuid = Request->Guid;

	{
		FScopeLock Lock(&PingMutex);
		PingContexts.Add(MoveTemp(RequestGuid), MoveTemp(InResponseCb));
	}

	SendMessage(Request);

	return RequestGuid;
}

void FConnectStarter::HandleConnectResponse(const FConnectResponse& InResponse, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FScopeLock Lock(&Mutex);
	if (Contexts.Contains(InResponse.RequestGuid))
	{
		FConnectHandler Callback;
		Contexts.RemoveAndCopyValue(InResponse.RequestGuid, Callback);

		Callback.ExecuteIfBound(InResponse);
	}

	if (InResponse.Status == EStatus::Ok)
	{
		// Remove previous keep alive entry (reset keep alive)
		TimerManager->RemoveTimer(KeepAlive);

		KeepAlive = 
			TimerManager->AddTimer(
				FTimerDelegate::CreateSP(this, &FConnectStarter::OnKeepAliveInterval), KeepAliveInterval, false, KeepAliveInterval);

		bConnected = true;
	}
}

void FConnectStarter::HandlePingResponse(const FPongMessage& InResponse, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FScopeLock Lock(&Mutex);
	if (PingContexts.Contains(InResponse.RequestGuid))
	{
		FPingCallback Callback;
		PingContexts.RemoveAndCopyValue(InResponse.RequestGuid, Callback);

		Callback.ExecuteIfBound(InResponse);
	}
}

void FConnectStarter::AddContext(FGuid InGuid, FConnectHandler InHandler)
{
	FScopeLock Lock(&Mutex);
	Contexts.Add(MoveTemp(InGuid), MoveTemp(InHandler));
}

void FConnectStarter::OnKeepAliveInterval()
{
	AsyncTask(ENamedThreads::Type::AnyThread, [This = AsShared()]()
	{
		FSharedEventRef PongEvent;

		FGuid RequestGuid = This->SendPingRequest(FPingCallback::CreateLambda([PongEvent](const FPongMessage& InResponse) mutable
		{
			PongEvent->Trigger();
		}));

		if (!RequestGuid.IsValid())
		{
			// We already disconnected
			return;
		}

		bool bResult = PongEvent->Wait(FTimespan::FromSeconds(KeepAliveTimeout));

		if (!bResult)
		{
			This->bConnected = false;

			FScopeLock Lock(&This->Mutex);
			This->Contexts.Remove(RequestGuid);
			FDisconnectHandler LocalDisconnectHandler = This->Handler;
			Lock.Unlock();

			LocalDisconnectHandler.ExecuteIfBound();
		}

		This->TimerManager->RemoveTimer(This->KeepAlive);

		if (bResult)
		{
			This->KeepAlive = 
				This->TimerManager->AddTimer(
					FTimerDelegate::CreateSP(This, &FConnectStarter::OnKeepAliveInterval), KeepAliveInterval, false, KeepAliveInterval);
		}
	});
}