// Copyright Epic Games, Inc. All Rights Reserved.

#include "Features/UploadStateHandler.h"

FUploadStateHandler::FUploadStateHandler() = default;

void FUploadStateHandler::SetUploadCallbacks(FUploadStateCallback InStateCallback,
											 FUploadFinishedCallback InFinishedCallback)
{
	FScopeLock ScopeLock(&CriticalSection);
	StateCallback = MoveTemp(InStateCallback);
	FinishedCallback = MoveTemp(InFinishedCallback);
}

void FUploadStateHandler::Initialize(FMessageEndpointBuilder& InBuilder)
{
	InBuilder
		.Handling<FUploadState>(this, &FUploadStateHandler::HandleUploadStateMessage)
		.Handling<FUploadFinished>(this, &FUploadStateHandler::HandleUploadFinishedMessage);
}

void FUploadStateHandler::HandleUploadStateMessage(const FUploadState& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FScopeLock ScopeLock(&CriticalSection);
	// Take a copy, so we can execute the callback without holding the lock
	FUploadStateCallback LocalCallback = StateCallback;
	ScopeLock.Unlock();

	LocalCallback.ExecuteIfBound(InMessage.CaptureSourceId, InMessage.TakeUploadId, InMessage.Progress);
}

void FUploadStateHandler::HandleUploadFinishedMessage(const FUploadFinished& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FScopeLock ScopeLock(&CriticalSection);
	// Take a copy, so we can execute the callback without holding the lock
	FUploadFinishedCallback LocalCallback = FinishedCallback;
	ScopeLock.Unlock();

	LocalCallback.ExecuteIfBound(InMessage.CaptureSourceId, InMessage.TakeUploadId, InMessage.Message, static_cast<int32>(InMessage.Status));
}
