// Copyright Epic Games, Inc. All Rights Reserved.

#include "Features/UploadStateSender.h"

FUploadStateSender::FUploadStateSender() = default;

void FUploadStateSender::SendUploadStateMessage(const FGuid& InCaptureSourceId, const FGuid& InTakeUploadId, double InProgress)
{
	FUploadState* Message = FMessageEndpoint::MakeMessage<FUploadState>();

	Message->CaptureSourceId = InCaptureSourceId;
	Message->TakeUploadId = InTakeUploadId;
	Message->Progress = InProgress;

	SendMessage(Message);
}

void FUploadStateSender::SendUploadDoneMessage(const FGuid& InCaptureSourceId, const FGuid& InTakeUploadId, FString InMessage, int32 InCode)
{
	FUploadFinished* Message = FMessageEndpoint::MakeMessage<FUploadFinished>();

	Message->CaptureSourceId = InCaptureSourceId;
	Message->TakeUploadId = InTakeUploadId;
	Message->Message = MoveTemp(InMessage);
	Message->Status = ConvertStatus(InCode);

	SendMessage(Message);
}

void FUploadStateSender::Initialize(FMessageEndpointBuilder& InBuilder)
{
}

EStatus FUploadStateSender::ConvertStatus(const int32 InCode)
{
	switch (InCode)
	{
		case 0:
			return EStatus::Ok;
		default:
			return EStatus::InternalError;
	}
}
