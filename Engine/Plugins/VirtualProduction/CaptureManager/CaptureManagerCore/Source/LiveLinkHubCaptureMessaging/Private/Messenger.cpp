// Copyright Epic Games, Inc. All Rights Reserved.

#include "Messenger.h"

void FFeatureBase::SetEndpoint(TSharedPtr<FMessageEndpoint> InEndpoint)
{
	FScopeLock ScopeLock(&CriticalSection);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Endpoint = InEndpoint;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FFeatureBase::SetAddress(const FMessageAddress& InAddress)
{
	FScopeLock ScopeLock(&CriticalSection);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Address = InAddress;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FMessageAddress FFeatureBase::GetAddress() const
{
	FScopeLock ScopeLock(&CriticalSection);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Address;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
