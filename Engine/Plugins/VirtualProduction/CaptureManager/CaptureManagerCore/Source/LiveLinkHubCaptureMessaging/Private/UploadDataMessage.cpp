// Copyright Epic Games, Inc. All Rights Reserved.

#include "UploadDataMessage.h"

#include "Utility/Error.h"

#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "UploadDataMessage"

#define CHECK_AND_CONVERT_ERROR(Result, Message)                                \
	if (Result.HasError())                                                      \
	{                                                                           \
		FCaptureProtocolError Error = Result.StealError();                      \
		FText TextMessage =                                                     \
			FText::Format(FText::FromString("{0}: {1}"),                        \
						  Message,                                              \
						  FText::FromString(Error.GetMessage()));               \
		return MakeError(MoveTemp(TextMessage), Error.GetCode());               \
	}

#define CHECK_AND_RET_ERROR(Result)                                             \
	if (Result.HasError())                                                      \
	{                                                                           \
		FUploadError Error = Result.StealError();                               \
		return MakeError(MoveTemp(Error));                                      \
	}

FUploadError::FUploadError(FText InMessage, int32 InCode)
	: Message(MoveTemp(InMessage))
	, Code(InCode)
{
}

const FText& FUploadError::GetText() const
{
	return Message;
}

int32 FUploadError::GetCode() const
{
	return Code;
}

const TArray<uint8> FUploadDataHeader::Header = {'U', 'P', 'L', 'O', 'A', 'D', '\0'};

FUploadVoidResult FUploadDataMessage::SerializeHeader(const FUploadDataHeader InHeader, UE::CaptureManager::ITcpSocketWriter& InWriter)
{
	using namespace UE::CaptureManager;

	const FString ClientId = InHeader.ClientId.ToString(EGuidFormats::Digits);
	const FString CaptureSourceId = InHeader.CaptureSourceId.ToString(EGuidFormats::Digits);;
	const FString TakeUploadId = InHeader.TakeUploadId.ToString(EGuidFormats::Digits);;
	const FString& CaptureSourceName = InHeader.CaptureSourceName;
	const FString& Slate = InHeader.Slate;
	const uint32 TakeNumber = InHeader.TakeNumber;
	
	TArray<uint8> Data;

	auto ClientIdUTF8 = StringCast<UTF8CHAR>(*ClientId);
	auto CaptureSourceIdUTF8 = StringCast<UTF8CHAR>(*CaptureSourceId);
	auto TakeUploadIdUTF8 = StringCast<UTF8CHAR>(*TakeUploadId);
	auto CaptureSourceNameUTF8 = StringCast<UTF8CHAR>(*CaptureSourceName);
	auto SlateUTF8 = StringCast<UTF8CHAR>(*Slate);
	
	const uint16 ClientIdLen = ClientIdUTF8.Length();
	const uint16 CaptureSourceIdLen = CaptureSourceIdUTF8.Length();
	const uint16 TakeUploadIdLen = TakeUploadIdUTF8.Length();
	const uint16 CaptureSourceNameLen = CaptureSourceNameUTF8.Length();
	const uint16 SlateLen = SlateUTF8.Length();

	Data.Append(FUploadDataHeader::Header.GetData(), FUploadDataHeader::Header.Num());
	Data.Append(reinterpret_cast<const uint8*>(&ClientIdLen), sizeof(ClientIdLen));
	Data.Append(reinterpret_cast<const uint8*>(ClientIdUTF8.Get()), ClientIdLen);
	Data.Append(reinterpret_cast<const uint8*>(&CaptureSourceIdLen), sizeof(CaptureSourceIdLen));
	Data.Append(reinterpret_cast<const uint8*>(CaptureSourceIdUTF8.Get()), CaptureSourceIdLen);
	Data.Append(reinterpret_cast<const uint8*>(&CaptureSourceNameLen), sizeof(CaptureSourceNameLen));
	Data.Append(reinterpret_cast<const uint8*>(CaptureSourceNameUTF8.Get()), CaptureSourceNameLen);
	Data.Append(reinterpret_cast<const uint8*>(&TakeUploadIdLen), sizeof(TakeUploadIdLen));
	Data.Append(reinterpret_cast<const uint8*>(TakeUploadIdUTF8.Get()), TakeUploadIdLen);
	Data.Append(reinterpret_cast<const uint8*>(&SlateLen), sizeof(SlateLen));
	Data.Append(reinterpret_cast<const uint8*>(SlateUTF8.Get()), SlateLen);
	Data.Append(reinterpret_cast<const uint8*>(&TakeNumber), sizeof(TakeNumber));
	Data.Append(reinterpret_cast<const uint8*>(&(InHeader.TotalLength)), sizeof(InHeader.TotalLength));

	TProtocolResult<void> Result = InWriter.SendMessage(Data);
	CHECK_AND_CONVERT_ERROR(Result, LOCTEXT("SerializeHeader_Error", "Error while writing the Upload Data header"));

	return MakeValue();
}

FUploadVoidResult FUploadDataMessage::SerializeFileHeader(FUploadFileDataHeader InFileHeader, UE::CaptureManager::ITcpSocketWriter& InWriter)
{
	using namespace UE::CaptureManager;

	const FString& FileName = InFileHeader.FileName;

	TArray<uint8> Data;

	auto FileNameUTF8 = StringCast<UTF8CHAR>(*FileName);

	const uint16 FileNameLen = FileNameUTF8.Length();

	Data.Append(reinterpret_cast<const uint8*>(&FileNameLen), sizeof(FileNameLen));
	Data.Append(reinterpret_cast<const uint8*>(FileNameUTF8.Get()), FileNameLen);
	Data.Append(reinterpret_cast<const uint8*>(&(InFileHeader.Length)), sizeof(InFileHeader.Length));

	TProtocolResult<void> Result = InWriter.SendMessage(Data);
	CHECK_AND_CONVERT_ERROR(Result, LOCTEXT("SerializeFileHeader_Error", "Error while writing a file header"));

	return MakeValue();
}

FUploadVoidResult FUploadDataMessage::SerializeData(TArray<uint8> InData, UE::CaptureManager::ITcpSocketWriter& InWriter)
{
	using namespace UE::CaptureManager;

	TProtocolResult<void> Result = InWriter.SendMessage(InData);
	CHECK_AND_CONVERT_ERROR(Result, LOCTEXT("SerializeData_Error", "Error while writing the data"));

	return MakeValue();
}

FUploadVoidResult FUploadDataMessage::SerializeHash(const TStaticArray<uint8, 16> InHash, UE::CaptureManager::ITcpSocketWriter& InWriter)
{
	using namespace UE::CaptureManager;

	TArray<uint8> Data;
	Data.Append(InHash.GetData(), InHash.Num());

	TProtocolResult<void> Result = InWriter.SendMessage(Data);
	CHECK_AND_CONVERT_ERROR(Result, LOCTEXT("SerializeHash_Error", "Error while writing the hash"));
	return MakeValue();
}

FUploadResult<FUploadDataHeader> FUploadDataMessage::DeserializeHeader(UE::CaptureManager::ITcpSocketReader& InReader)
{
	FUploadDataHeader Header;

	{
		FUploadVoidResult Result = DeserializeStartHeader(InReader);
		CHECK_AND_RET_ERROR(Result);
	}

	{
		FUploadVoidResult Result = DeserializeGuid(InReader, Header.ClientId);
		CHECK_AND_RET_ERROR(Result);
	}

	{
		FUploadVoidResult Result = DeserializeGuid(InReader, Header.CaptureSourceId);
		CHECK_AND_RET_ERROR(Result);
	}

	{
		FUploadVoidResult Result = DeserializeString(InReader, Header.CaptureSourceName);
		CHECK_AND_RET_ERROR(Result);
	}

	{
		FUploadVoidResult Result = DeserializeGuid(InReader, Header.TakeUploadId);
		CHECK_AND_RET_ERROR(Result);
	}

	{
		FUploadVoidResult Result = DeserializeString(InReader, Header.Slate);
		CHECK_AND_RET_ERROR(Result);
	}

	{
		FUploadVoidResult Result = DeserializeTakeNumber(InReader, Header.TakeNumber);
		CHECK_AND_RET_ERROR(Result);
	}

	{
		FUploadVoidResult Result = DeserializeTotalLength(InReader, Header.TotalLength);
		CHECK_AND_RET_ERROR(Result);
	}

	return MakeValue(MoveTemp(Header));
}

FUploadResult<FUploadFileDataHeader> FUploadDataMessage::DeserializeFileHeader(UE::CaptureManager::ITcpSocketReader& InReader)
{
	FUploadFileDataHeader FileHeader;

	{
		FUploadVoidResult Result = DeserializeFileName(InReader, FileHeader.FileName);
		CHECK_AND_RET_ERROR(Result);
	}

	{
		FUploadVoidResult Result = DeserializeLength(InReader, FileHeader.Length);
		CHECK_AND_RET_ERROR(Result);
	}

	return MakeValue(MoveTemp(FileHeader));
}

FUploadVoidResult FUploadDataMessage::DeserializeStartHeader(UE::CaptureManager::ITcpSocketReader& InReader)
{
	using namespace UE::CaptureManager;

	TProtocolResult<TArray<uint8>> HeaderResult = InReader.ReceiveMessage(FUploadDataHeader::Header.Num(), InactivityTimeout);
	CHECK_AND_CONVERT_ERROR(HeaderResult, LOCTEXT("DeserializeHeader_HeaderError", "Failed to read the header"));

	TArray<uint8> ReadHeader = HeaderResult.StealValue();
	if (FUploadDataHeader::Header != ReadHeader)
	{
		return MakeError(LOCTEXT("DeserializeHeader_InvalidHeaderError", "Invalid header read"));
	}

	return MakeValue();
}

FUploadVoidResult FUploadDataMessage::DeserializeGuid(UE::CaptureManager::ITcpSocketReader& InReader, FGuid& OutFGuid)
{
	using namespace UE::CaptureManager;

	uint16 Length;

	TProtocolResult<TArray<uint8>> GuidLengthResult = InReader.ReceiveMessage(sizeof(Length), InactivityTimeout);
	CHECK_AND_CONVERT_ERROR(GuidLengthResult, LOCTEXT("DeserializeHeader_GuidLengthError", "Failed to read client id length"));

	Length = *reinterpret_cast<uint16*>(GuidLengthResult.StealValue().GetData());

	TProtocolResult<TArray<uint8>> GuidResult = InReader.ReceiveMessage(Length);
	CHECK_AND_CONVERT_ERROR(GuidResult, LOCTEXT("DeserializeHeader_GuidError", "Failed to read the client id"));

	TArray<uint8> GuidData = GuidResult.StealValue();
	FString GuidString = FString::ConstructFromPtrSize(reinterpret_cast<const UTF8CHAR*>(GuidData.GetData()), Length);

	bool ParseSucceded = FGuid::ParseExact(GuidString, EGuidFormats::Digits, OutFGuid);
	if (!ParseSucceded)
	{
		return MakeError(LOCTEXT("DeserializeHeader_InvalidGuidError", "Invalid client id format"));
	}
	
	return MakeValue();
}

FUploadVoidResult FUploadDataMessage::DeserializeTakeNumber(UE::CaptureManager::ITcpSocketReader& InReader, uint32& OutTakeNumber)
{
	using namespace UE::CaptureManager;

	uint32 TakeNumber;

	TProtocolResult<TArray<uint8>> TakeNumberResult = InReader.ReceiveMessage(sizeof(TakeNumber), InactivityTimeout);
	CHECK_AND_CONVERT_ERROR(TakeNumberResult, LOCTEXT("DeserializeHeader_TakeNumberError", "Failed to read take number"));

	TArray<uint8> TakeNumberData = TakeNumberResult.StealValue();

	OutTakeNumber = *reinterpret_cast<const uint32*>(TakeNumberData.GetData());

	return MakeValue();
}

FUploadVoidResult FUploadDataMessage::DeserializeString(UE::CaptureManager::ITcpSocketReader& InReader, FString& OutTakeName)
{
	using namespace UE::CaptureManager;

	uint16 Length;

	TProtocolResult<TArray<uint8>> TakeNameLengthResult = InReader.ReceiveMessage(sizeof(Length), InactivityTimeout);
	CHECK_AND_CONVERT_ERROR(TakeNameLengthResult, LOCTEXT("DeserializeHeader_TakeNameLengthError", "Failed to read take name length"));

	Length = *reinterpret_cast<uint16*>(TakeNameLengthResult.StealValue().GetData());

	TProtocolResult<TArray<uint8>> TakeNameResult = InReader.ReceiveMessage(Length);
	CHECK_AND_CONVERT_ERROR(TakeNameResult, LOCTEXT("DeserializeHeader_TakeNameError", "Failed to read take name"));

	TArray<uint8> TakeNameData = TakeNameResult.StealValue();
	OutTakeName = FString::ConstructFromPtrSize(reinterpret_cast<const UTF8CHAR*>(TakeNameData.GetData()), Length);

	return MakeValue();
}

FUploadVoidResult FUploadDataMessage::DeserializeTotalLength(UE::CaptureManager::ITcpSocketReader& InReader, uint64& OutTotalLength)
{
	using namespace UE::CaptureManager;

	uint64 TotalLength;

	TProtocolResult<TArray<uint8>> TotalLengthResult = InReader.ReceiveMessage(sizeof(TotalLength), InactivityTimeout);
	CHECK_AND_CONVERT_ERROR(TotalLengthResult, LOCTEXT("DeserializeHeader_TotalLengthError", "Failed to read total file length"));

	TArray<uint8> TotalLengthData = TotalLengthResult.StealValue();

	OutTotalLength = *reinterpret_cast<const uint64*>(TotalLengthData.GetData());

	return MakeValue();
}

FUploadVoidResult FUploadDataMessage::DeserializeFileName(UE::CaptureManager::ITcpSocketReader& InReader, FString& OutFileName)
{
	using namespace UE::CaptureManager;

	uint16 Length;

	TProtocolResult<TArray<uint8>> FileNameLengthResult = InReader.ReceiveMessage(sizeof(Length));
	CHECK_AND_CONVERT_ERROR(FileNameLengthResult, LOCTEXT("DeserializeHeader_FileNameLengthError", "Failed to read file name length"));

	Length = *reinterpret_cast<uint16*>(FileNameLengthResult.StealValue().GetData());

	TProtocolResult<TArray<uint8>> FileNameResult = InReader.ReceiveMessage(Length);
	CHECK_AND_CONVERT_ERROR(FileNameResult, LOCTEXT("DeserializeHeader_FileNameError", "Failed to read file name"));

	TArray<uint8> FileNameData = FileNameResult.StealValue();
	OutFileName = FString::ConstructFromPtrSize(reinterpret_cast<const UTF8CHAR*>(FileNameData.GetData()), Length);

	return MakeValue();
}

FUploadVoidResult FUploadDataMessage::DeserializeLength(UE::CaptureManager::ITcpSocketReader& InReader, uint64& OutLength)
{
	using namespace UE::CaptureManager;

	uint64 Length;

	TProtocolResult<TArray<uint8>> LengthResult = InReader.ReceiveMessage(sizeof(Length), InactivityTimeout);
	CHECK_AND_CONVERT_ERROR(LengthResult, LOCTEXT("DeserializeHeader_LengthError", "Failed to read file length"));

	TArray<uint8> LengthData = LengthResult.StealValue();

	OutLength = *reinterpret_cast<const uint64*>(LengthData.GetData());

	return MakeValue();
}

FUploadResult<TArray<uint8>> FUploadDataMessage::DeserializeData(uint32 InSize, UE::CaptureManager::ITcpSocketReader& InReader)
{
	using namespace UE::CaptureManager;

	TProtocolResult<TArray<uint8>> DataResult = InReader.ReceiveMessage(InSize, InactivityTimeout);
	CHECK_AND_CONVERT_ERROR(DataResult, LOCTEXT("DeserializeData_DataError", "Failed to read the data"));

	return MakeValue(DataResult.StealValue());
}

FUploadResult<TStaticArray<uint8, FUploadDataMessage::HashSize>> FUploadDataMessage::DeserializeHash(UE::CaptureManager::ITcpSocketReader& InReader)
{
	using namespace UE::CaptureManager;

	TProtocolResult<TArray<uint8>> HashResult = InReader.ReceiveMessage(HashSize, InactivityTimeout);
	CHECK_AND_CONVERT_ERROR(HashResult, LOCTEXT("DeserializeHash_HashError", "Failed to read the hash"));

	TArray<uint8> HashData = HashResult.StealValue();

	TStaticArray<uint8, HashSize> Hash;

	FMemory::Memcpy(Hash.GetData(), HashData.GetData(), Hash.Num());

	return MakeValue(MoveTemp(Hash));
}

#undef LOCTEXT_NAMESPACE