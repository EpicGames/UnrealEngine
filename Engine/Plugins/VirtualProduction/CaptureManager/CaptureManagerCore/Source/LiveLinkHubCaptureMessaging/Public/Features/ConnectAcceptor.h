// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MessageEndpointBuilder.h"
#include "MessageEndpoint.h"

#include "LiveLinkHubCaptureMessages.h"

#include "Messenger.h"

#include "Async/CaptureTimerManager.h"

#define UE_API LIVELINKHUBCAPTUREMESSAGING_API

class FConnectAcceptor : public FFeatureBase
{
public:

	DECLARE_DELEGATE_RetVal_TwoParams(FConnectResponse*, FConnectAccepted, const FConnectRequest& InRequest, const FMessageAddress& InAddress);
	DECLARE_DELEGATE_OneParam(FConnectionLostHandler, const FMessageAddress& InAddress);

	UE_API FConnectAcceptor();
	UE_API virtual ~FConnectAcceptor() override;

	UE_API void SetConnectionHandler(FConnectAccepted InConnectHandler,
							  FConnectionLostHandler InConnectionLostHandler);
	UE_API void Disconnect();
	UE_API bool IsConnected() const;

protected:

	UE_API void Initialize(FMessageEndpointBuilder& InBuilder);

private:

	UE_API void ConnectRequestHandler(const FConnectRequest& InRequest, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	UE_API void OnHangUpReceived(const FCaptureManagerHangUp& InRequest, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	UE_API void HandleKeepAlive(const FPingMessage& InRequest, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);

	UE_API void CheckConnectionActivity();

	double LastPingRequest = 0.0f;
	double InactivityTimeout = 20.0f; // seconds

	TSharedRef<UE::CaptureManager::FCaptureTimerManager> TimerManager;
	UE::CaptureManager::FCaptureTimerManager::FTimerHandle ClientActivity;

	std::atomic_bool bConnected = false;

	FCriticalSection CriticalSection;
	FConnectAccepted ConnectHandler;
	FConnectionLostHandler ConnectionLostHandler;
};

#undef UE_API
