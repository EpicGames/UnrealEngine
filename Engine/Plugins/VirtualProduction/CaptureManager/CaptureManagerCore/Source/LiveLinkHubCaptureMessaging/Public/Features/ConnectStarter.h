// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MessageEndpointBuilder.h"
#include "MessageEndpoint.h"

#include "Messenger.h"

#include "Async/CaptureTimerManager.h"

#include "LiveLinkHubCaptureMessages.h"

#define UE_API LIVELINKHUBCAPTUREMESSAGING_API

class FConnectStarter : 
	public FFeatureBase, 
	public TSharedFromThis<FConnectStarter>
{
public:

	DECLARE_DELEGATE_OneParam(FConnectHandler, const FConnectResponse& InResponse);
	DECLARE_DELEGATE(FDisconnectHandler);

	UE_API FConnectStarter();
	UE_API virtual ~FConnectStarter() override;

	UE_API void Connect(FConnectHandler InConnectHandler);
	UE_API void Disconnect();
	UE_API void SetDisconnectHandler(FDisconnectHandler InHandler);

	UE_API bool IsConnected() const;

protected:

	UE_API void Initialize(FMessageEndpointBuilder& InBuilder);

private:

	static UE_API const float KeepAliveInterval;
	static UE_API const float KeepAliveTimeout;

	DECLARE_DELEGATE_OneParam(FPingCallback, const FPongMessage& InResponse);
	UE_API FGuid SendPingRequest(FPingCallback InCallback);

	UE_API void HandleConnectResponse(const FConnectResponse& InResponse, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	UE_API void HandlePingResponse(const FPongMessage& InResponse, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	UE_API void AddContext(FGuid InGuid, FConnectHandler InHandler);

	UE_API void OnKeepAliveInterval();

	FCriticalSection Mutex;
	TMap<FGuid, FConnectHandler> Contexts;

	FCriticalSection PingMutex;
	TMap<FGuid, FPingCallback> PingContexts;

	TSharedRef<UE::CaptureManager::FCaptureTimerManager> TimerManager;
	UE::CaptureManager::FCaptureTimerManager::FTimerHandle KeepAlive;
	FDisconnectHandler Handler;

	std::atomic_bool bConnected = false;
};

#undef UE_API
