// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MessageEndpointBuilder.h"
#include "MessageEndpoint.h"

#include "Messenger.h"

#include "LiveLinkHubCaptureMessages.h"

#define UE_API LIVELINKHUBCAPTUREMESSAGING_API

class FUploadStateHandler : public FFeatureBase
{
public:

	DECLARE_DELEGATE_ThreeParams(FUploadStateCallback, const FGuid& InCaptureSourceId, const FGuid& InTakeUploadId, double InProgress);
	DECLARE_DELEGATE_FourParams(FUploadFinishedCallback, const FGuid& InCaptureSourceId, const FGuid& InTakeUploadId, FString InMessage, int32 InCode);

	UE_API FUploadStateHandler();

	UE_API void SetUploadCallbacks(FUploadStateCallback InStateCallback,
							FUploadFinishedCallback InFinishedCallback);

protected:

	UE_API void Initialize(FMessageEndpointBuilder& InBuilder);

private:
	UE_API void HandleUploadStateMessage(const FUploadState& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	UE_API void HandleUploadFinishedMessage(const FUploadFinished& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);

	FCriticalSection CriticalSection;
	FUploadStateCallback StateCallback;
	FUploadFinishedCallback FinishedCallback;
};

#undef UE_API
