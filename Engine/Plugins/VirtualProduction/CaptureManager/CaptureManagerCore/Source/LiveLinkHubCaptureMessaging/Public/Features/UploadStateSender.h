// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MessageEndpointBuilder.h"
#include "MessageEndpoint.h"

#include "Messenger.h"

#include "LiveLinkHubCaptureMessages.h"

#define UE_API LIVELINKHUBCAPTUREMESSAGING_API

class FUploadStateSender : public FFeatureBase
{
public:

	UE_API FUploadStateSender();

	UE_API void SendUploadStateMessage(const FGuid& InCaptureSourceId, const FGuid& InTakeUploadId, double InProgress);
	UE_API void SendUploadDoneMessage(const FGuid& InCaptureSourceId, const FGuid& InTakeUploadId, FString InMessage, int32 InCode);

protected:

	UE_API void Initialize(FMessageEndpointBuilder& InBuilder);

private:

	UE_API EStatus ConvertStatus(int32 InCode);
};

#undef UE_API
