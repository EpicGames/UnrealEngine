// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"

#include "Containers/UnrealString.h"

#include "Misc/Guid.h"

#include "LiveLinkHubCaptureMessages.generated.h"

namespace UE::LiveLinkHubCaptureMessages
{
	constexpr uint16 Version = 1;
	constexpr uint16 DiscoveryVersion = 1;
}

UENUM()
enum class EStatus
{
	Ok = 0,
	InvalidArgument = 1,
	InternalError = 2,
};

USTRUCT()
struct FBaseMessage
{
	GENERATED_BODY()

	UPROPERTY()
	uint16 Version = UE::LiveLinkHubCaptureMessages::Version;

	UPROPERTY()
	FGuid Guid;
};

USTRUCT()
struct FBaseResponse : public FBaseMessage
{
	GENERATED_BODY()

	UPROPERTY()
	EStatus Status = EStatus::Ok;

	UPROPERTY()
	FString Message;

	UPROPERTY()
	FGuid RequestGuid;
};

USTRUCT()
struct FConnectRequest : public FBaseMessage
{
	GENERATED_BODY()
};

USTRUCT()
struct FConnectResponse : public FBaseResponse
{
	GENERATED_BODY()
};

USTRUCT()
struct FCaptureManagerHangUp : public FBaseMessage
{
	GENERATED_BODY()
};

USTRUCT()
struct FDiscoveryRequest
{
	GENERATED_BODY()

	UPROPERTY()
	uint16 MessageVersion = UE::LiveLinkHubCaptureMessages::DiscoveryVersion;

	UPROPERTY()
	FString HostName;
};

USTRUCT()
struct FDiscoveryResponse
{
	GENERATED_BODY()

	UPROPERTY()
	FString HostName;

	UPROPERTY()
	FString IPAddress;

	UPROPERTY()
	uint16 ExportPort = 0;
};

USTRUCT()
struct FPingMessage : public FBaseMessage
{
	GENERATED_BODY();
};

USTRUCT()
struct FPongMessage : public FBaseResponse
{
	GENERATED_BODY();
};

USTRUCT()
struct FUploadState
{
	GENERATED_BODY();

	UPROPERTY()
	FGuid CaptureSourceId;

	UPROPERTY()
	FGuid TakeUploadId;

	UPROPERTY()
	double Progress = 0.0f;
};

USTRUCT()
struct FUploadFinished
{
	GENERATED_BODY();

	UPROPERTY()
	FGuid CaptureSourceId;

	UPROPERTY()
	FGuid TakeUploadId;

	UPROPERTY()
	EStatus Status = EStatus::Ok;

	UPROPERTY()
	FString Message;
};
