// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkHubCaptureMessages.h"

#include "MessageEndpointBuilder.h"
#include "MessageEndpoint.h"

#include "Async/Future.h"

#include "Templates/ValueOrError.h"

#define UE_API LIVELINKHUBCAPTUREMESSAGING_API


class FFeatureBase
{
protected:

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FFeatureBase() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void Initialize(FMessageEndpointBuilder& InBuilder) = 0;

	template <typename MessageType>
	void SendMessage(MessageType* InMessage);

	UE_API void SetEndpoint(TSharedPtr<FMessageEndpoint> InEndpoint);
	UE_API void SetAddress(const FMessageAddress& InAddress);
	UE_API FMessageAddress GetAddress() const;

	// When removing this deprecation, make the member private and remove warning suppression in this class
	UE_DEPRECATED(5.7, "Accessing the endpoint directly is deprecated and likely to result in a crash, use SendMessage instead. Access to this member will be removed in a future version")
	TSharedPtr<FMessageEndpoint> Endpoint;

	// When removing this deprecation, make the member private and remove warning suppression in this classs
	UE_DEPRECATED(5.7, "Accessing the address directly is deprecated and likely to result in a crash, use Set/GetAddress instead. Access to this member will be removed in a future version")
	FMessageAddress Address;

private:
	mutable FCriticalSection CriticalSection;
};

template <typename MessageType>
void FFeatureBase::SendMessage(MessageType* InMessage)
{
	if (!ensureMsgf(InMessage, TEXT("SendMessage() requires a valid message (it was nullptr)")))
	{
		return;
	}

	FScopeLock ScopeLock(&CriticalSection);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TSharedPtr<FMessageEndpoint> MessageEndpoint = Endpoint;
	FMessageAddress MessageAddress = Address;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// We don't hold the lock during Send(), just in case the caller has bound a message handler which calls back into
	// the public API of this class.
	ScopeLock.Unlock();

	if (ensureMsgf(MessageEndpoint, TEXT("A valid endpoint must be set before calling SendMessage")))
	{
		MessageEndpoint->Send(InMessage, MessageAddress);
	}
	else
	{
		// Since we didn't call Send, we have to free the memory for the message ourselves
		FMemory::Free(InMessage);
	}
}

template<class ... Features>
class FMessenger
	: public Features...
{
public:

	inline static const FString Name = TEXT("Messenger");

	FMessenger()
		: Builder(*Name)
	{
		Builder.ReceivingOnAnyThread();

		(Features::Initialize(Builder), ...);

		Endpoint = Builder.Build();

		(Features::SetEndpoint(Endpoint), ...);
	}

	~FMessenger()
	{
		FMessageEndpoint::SafeRelease(Endpoint);
	}

	void SetAddress(FMessageAddress InAddress)
	{
		Address = MoveTemp(InAddress);

		(Features::SetAddress(Address), ...);
	}

	FMessageAddress GetAddress() const
	{
		return Address;
	}

	FMessageAddress GetOwnAddress() const
	{
		return Endpoint->GetAddress();
	}

	void SendDiscoveryResponse(FDiscoveryResponse* InResponse, FMessageAddress InReceiver)
	{
		Endpoint->Send(InResponse, InReceiver);
	}

private:

	FMessageEndpointBuilder Builder;
	TSharedPtr<FMessageEndpoint> Endpoint;
	FMessageAddress Address;
};

#undef UE_API
