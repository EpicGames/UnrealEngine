// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Containers/StaticArray.h"

#include "Internationalization/Text.h"

#include "Templates/ValueOrError.h"

#include "Network/TcpReaderWriter.h"

#include "Misc/Guid.h"

#define UE_API LIVELINKHUBCAPTUREMESSAGING_API

class FUploadError
{
public:

	UE_API FUploadError(FText InMessage, int32 InCode = 0);

	UE_API const FText& GetText() const;
	UE_API int32 GetCode() const;

private:

	FText Message;
	int32 Code;
};

template<typename T>
using FUploadResult = TValueOrError<T, FUploadError>;

using FUploadVoidResult = FUploadResult<void>;

struct FUploadDataHeader
{
	static const TArray<uint8> Header;

	FGuid ClientId;
	FGuid CaptureSourceId;
	FGuid TakeUploadId;

	FString CaptureSourceName;
	FString Slate;
	uint32 TakeNumber = 0;
	uint64 TotalLength = 0;
};

struct FUploadFileDataHeader
{
	FString FileName;
	uint64 Length = 0;
};

class FUploadDataMessage
{
public:

	static constexpr int32 HashSize = 16;

	static UE_API FUploadVoidResult SerializeHeader(FUploadDataHeader InHeader, UE::CaptureManager::ITcpSocketWriter& InWriter);
	static UE_API FUploadVoidResult SerializeFileHeader(FUploadFileDataHeader InFileHeader, UE::CaptureManager::ITcpSocketWriter& InWriter);
	static UE_API FUploadVoidResult SerializeData(TArray<uint8> InData, UE::CaptureManager::ITcpSocketWriter& InWriter);
	static UE_API FUploadVoidResult SerializeHash(TStaticArray<uint8, HashSize> InHash, UE::CaptureManager::ITcpSocketWriter& InWriter);

	static UE_API FUploadResult<FUploadDataHeader> DeserializeHeader(UE::CaptureManager::ITcpSocketReader& InReader);
	static UE_API FUploadResult<FUploadFileDataHeader> DeserializeFileHeader(UE::CaptureManager::ITcpSocketReader& InReader);
	static UE_API FUploadResult<TArray<uint8>> DeserializeData(uint32 InSize, UE::CaptureManager::ITcpSocketReader& InReader);
	static UE_API FUploadResult<TStaticArray<uint8, HashSize>> DeserializeHash(UE::CaptureManager::ITcpSocketReader& InReader);

private:

	static UE_API FUploadVoidResult DeserializeStartHeader(UE::CaptureManager::ITcpSocketReader& InReader);
	static UE_API FUploadVoidResult DeserializeGuid(UE::CaptureManager::ITcpSocketReader& InReader, FGuid& OutFGuid);
	static UE_API FUploadVoidResult DeserializeTakeNumber(UE::CaptureManager::ITcpSocketReader& InReader, uint32& OutTakeNumber);
	static UE_API FUploadVoidResult DeserializeString(UE::CaptureManager::ITcpSocketReader& InReader, FString& OutTakeName);
	static UE_API FUploadVoidResult DeserializeTotalLength(UE::CaptureManager::ITcpSocketReader& InReader, uint64& OutTotalLength);
	static UE_API FUploadVoidResult DeserializeFileName(UE::CaptureManager::ITcpSocketReader& InReader, FString& OutFileName);
	static UE_API FUploadVoidResult DeserializeLength(UE::CaptureManager::ITcpSocketReader& InReader, uint64& OutLength);

	static constexpr uint32 InactivityTimeout = 15 * 1000; // Miliseconds
};

#undef UE_API
